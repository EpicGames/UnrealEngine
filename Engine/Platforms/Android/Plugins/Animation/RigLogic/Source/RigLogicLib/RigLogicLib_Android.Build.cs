// Copyright Epic Games, Inc. All Rights Reserved.
namespace UnrealBuildTool.Rules
{
	public class RigLogicLib_Android : RigLogicLib
	{
		public RigLogicLib_Android(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDefinitions.Add("RL_AUTODETECT_SSE");
			PrivateDefinitions.Add("RL_AUTODETECT_NEON");
			PrivateDefinitions.Add("RL_AUTODETECT_HALF_FLOATS");
		}
	}
}