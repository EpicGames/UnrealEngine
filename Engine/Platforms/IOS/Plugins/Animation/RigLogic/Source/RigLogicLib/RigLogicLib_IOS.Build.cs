// Copyright Epic Games, Inc. All Rights Reserved.
namespace UnrealBuildTool.Rules
{
	public class RigLogicLib_IOS : RigLogicLib
	{
		public RigLogicLib_IOS(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDefinitions.Add("RL_AUTODETECT_NEON");
			PrivateDefinitions.Add("RL_AUTODETECT_HALF_FLOATS");
		}
	}
}