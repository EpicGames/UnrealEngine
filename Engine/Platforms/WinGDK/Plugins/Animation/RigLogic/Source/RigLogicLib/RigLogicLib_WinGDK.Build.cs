// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RigLogicLib_WinGDK : RigLogicLib
	{
		public RigLogicLib_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDefinitions.Add("TERSE_ENABLE_SSE");
			PrivateDefinitions.Add("RL_BUILD_WITH_SSE");
			PrivateDefinitions.Add("RL_AUTODETECT_HALF_FLOATS");
		}
	}
}
