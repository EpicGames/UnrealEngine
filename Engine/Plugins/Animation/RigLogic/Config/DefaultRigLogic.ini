[CoreRedirects]


+ClassRedirects=(OldName="/Script/RigLogicEditor.UDNAAssetImportUI", NewName="/Script/RigLogicEditor.UDEPRECATED_DNAAssetImportUI")