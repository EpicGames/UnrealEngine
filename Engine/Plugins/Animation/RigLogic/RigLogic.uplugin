{
	"FileVersion": 3,
	"Version": 13.2,
	"VersionName": "13.2.5",
	"FriendlyName": "RigLogic Plugin",
	"Description": "RigLogic Plugin for Facial Animation",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "RigLogicLib",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "RigLogicLibTest",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "RigLogicModule",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name" : "RigLogicDeveloper",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name": "RigLogicEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		}
	]
}