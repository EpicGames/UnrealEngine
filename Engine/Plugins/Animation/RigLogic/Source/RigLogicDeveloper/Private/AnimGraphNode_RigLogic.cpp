// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RigLogic.h"

#include "AnimGraphNode_Base.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RigLogic)

#define LOCTEXT_NAMESPACE "AnimGraphNode_RigLogic_DeveloperModule"

UAnimGraphNode_RigLogic::UAnimGraphNode_RigLogic(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

void UAnimGraphNode_RigLogic::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

FText UAnimGraphNode_RigLogic::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_RigLogic_Tooltip", "Rig Logic Animation Node");
}

FText UAnimGraphNode_RigLogic::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_RigLogic_Title", "AnimNode_RigLogic");
}

#undef LOCTEXT_NAMESPACE

