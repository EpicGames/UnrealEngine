// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicDeveloper.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "RigLogicDeveloperModule"

DEFINE_LOG_CATEGORY(LogRigLogicDeveloper);

void FRigLogicDeveloperModule::StartupModule()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = false;
	InitOptions.bAllowClear = true;
	MessageLogModule.RegisterLogListing("RigLogicLog", LOCTEXT("RigLogicLog", "RigLogic Log"), InitOptions);
}

void FRigLogicDeveloperModule::ShutdownModule()
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.UnregisterLogListing("RigLogicLog");
}

IMPLEMENT_MODULE(FRigLogicDeveloperModule, RigLogicDeveloper)

#undef LOCTEXT_NAMESPACE
