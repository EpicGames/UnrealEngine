// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"
#include "AnimNode_RigLogic.h"
#include "AnimGraphNode_RigLogic.generated.h"

#define UE_API RIGLOGICDEVELOPER_API

UCLASS(MinimalAPI, meta = (Keywords = "Rig Logic Animation Node"))
class UAnimGraphNode_RigLogic : public UAnimGraphNode_Base
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_RigLogic Node;

public:
	UE_API FText GetTooltipText() const;
	UE_API FText GetNodeTitle(ENodeTitleType::Type TitleType) const;

protected:
	UE_API void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog);
};

#undef UE_API
