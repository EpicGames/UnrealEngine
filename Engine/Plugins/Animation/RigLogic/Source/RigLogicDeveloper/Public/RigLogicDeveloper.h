// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API RIGLOGICDEVELOPER_API

DECLARE_LOG_CATEGORY_EXTERN(LogRigLogicDeveloper, Log, All);

class FRigLogicDeveloperModule : public IModuleInterface 
{
public:
	UE_API void StartupModule() override;
	UE_API void ShutdownModule() override;
};

#undef UE_API
