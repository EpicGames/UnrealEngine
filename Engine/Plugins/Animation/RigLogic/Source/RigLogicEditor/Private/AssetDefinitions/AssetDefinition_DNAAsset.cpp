// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DNAAsset.h"
#include "DNAAsset.h"
#include "DNAUtils.h"
#include "DNA.h"
#include "DNAAssetUserData.h"
#include "DNAImporter.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ContentBrowserMenuContexts.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailHelpers.h"
#include "Engine/SkeletalMesh.h"
#include "EditorFramework/AssetImportData.h"


DEFINE_LOG_CATEGORY_STATIC(LogDNAAssetDefinition, Log, All);

#define LOCTEXT_NAMESPACE "AssetDefinition_DNAAsset"

namespace MenuExtention_SkelMeshDNAAsset
{
	static void ExecuteDNAImport(const UContentBrowserAssetContextMenuContext* InCBContext)
	{
		UDNAImporter* DNAImporter = NewObject<UDNAImporter>();
		TArray<USkeletalMesh*> SkeletalMeshes;
		for(const FAssetData& SelectedSkeletalMesh : InCBContext->SelectedAssets)
		{
			SkeletalMeshes.Add(Cast<USkeletalMesh>(SelectedSkeletalMesh.GetAsset()));
		}

		if(!DNAImporter->ImportDNAWithPrompt(SkeletalMeshes))
		{
			const FText Message = LOCTEXT("DNAImportFromSkeletalMeshMenuFail", "Importing of DNA failed");
			UE_LOGFMT(LogDNAAssetDefinition, Error, "{Name}", *Message.ToString());
		}
	}

	static void ExecuteDNAExport(const UContentBrowserAssetContextMenuContext* InCBContext)
	{
		UDNA* DNAAsset = Cast<UDNA>(InCBContext->SelectedAssets[0].GetAsset());

		UDNAImporter* DNAImporter = NewObject<UDNAImporter>();

		DNAImporter->ExportDNAWithPrompt(DNAAsset);
	}

	static void ExecuteDNAConversion(const UContentBrowserAssetContextMenuContext* InCBContext)
	{
		UDNAImporter* DNAImporter = NewObject<UDNAImporter>();
		for (const FAssetData& SelectedSkeletalMesh : InCBContext->SelectedAssets)
		{
			USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(SelectedSkeletalMesh.GetAsset());
			if(SkelMesh->GetAssetUserData<UDNAAsset>())
			{
				DNAImporter->ConvertFromLegacyAssetUserData(SkelMesh);
			}
		}
	}

	static void ExtendDNASkelMeshAssetActions()
	{
		FToolMenuOwnerScoped OwnderScoped(UE_MODULE_NAME);
		{
			UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USkeletalMesh::StaticClass())
				->AddDynamicSection(
					NAME_None,
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* InMenu)
						{
							const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext< UContentBrowserAssetContextMenuContext>();
							if (Context && Context->SelectedAssets.Num() > 0)
							{
								FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("GetAssetActions"));
								Section.AddSubMenu(
									"SkeletalMeshDNASubmenu",
									LOCTEXT("DNASkeletalMeshSubmenu", "MetaHuman DNA"),
									LOCTEXT("DNAImportSubmenu_ToolTip", "DNA related actions"),
									FNewToolMenuDelegate::CreateLambda([Context](UToolMenu* SubMenu)
									{
											FToolMenuSection& SubSection = SubMenu->AddSection(
												"DNA Asset Actions", LOCTEXT("DNAAssetActions", "DNA Actions"));
											{
												const FText Label = LOCTEXT("DNAAsset_Import", "Import DNA Asset");
												const FText Tooltip = LOCTEXT("DNAAsset_ImportTooltip", "Import a new DNA Asset and attach it to this Skeletal Mesh");
												const FSlateIcon Icon{ FAppStyle::GetAppStyleSetName(), "Icons.MetaHuman" };
												const FUIAction UIAction = FUIAction(
													FExecuteAction::CreateStatic(&ExecuteDNAImport, Context)
												);
												SubSection.AddMenuEntry(TEXT("DNAAsset_Import"), Label, Tooltip, Icon, UIAction);
											}
											{
												for (FAssetData SelectedSkeletalMesh : Context->SelectedAssets)
												{
													USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(SelectedSkeletalMesh.GetAsset());
													if(SkelMesh->GetAssetUserData<UDNAAsset>())
													{
														const FText Label = LOCTEXT("DNAAsset_Convert", "Create DNA Asset");
														const FText Tooltip = LOCTEXT("DNAAsset_ConvertTooltip", "Convert DNA asset user data to full DNA asset which is attached to this Skeletal Mesh");
														const FSlateIcon Icon{ FAppStyle::GetAppStyleSetName(), "Icons.MetaHuman" };
														const FUIAction UIAction = FUIAction(
															FExecuteAction::CreateStatic(&ExecuteDNAConversion, Context)
														);
														SubSection.AddMenuEntry(TEXT("DNAAsset_Convert"), Label, Tooltip, Icon, UIAction);
														break;
													}
												}
											}
									}),
									false,
									FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import")
								);
							}
						}
					)
				);

			UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UDNA::StaticClass())
				->AddDynamicSection(
					NAME_None,
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* InMenu)
						{
							const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext< UContentBrowserAssetContextMenuContext>();
							if (Context && Context->SelectedAssets.Num() > 0)
							{
								FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("GetAssetActions"));
								{
									{
										const FText Label = LOCTEXT("DNAAssetExport", "Export DNA");
										const FText Tooltip = LOCTEXT("DNAAssetExportTooltip", "Export DNA file from this asset");
										const FSlateIcon Icon{ FAppStyle::GetAppStyleSetName(), "Themes.Export" };
										const FUIAction UIAction = FUIAction(
											FExecuteAction::CreateStatic(&ExecuteDNAExport, Context)
										);
										Section.AddMenuEntry(TEXT("DNAAsssetExport"), Label, Tooltip, Icon, UIAction);
									}
								}
							}
						}
					)
				);
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(
		EDelayedRegisterRunPhase::EndOfEngineInit,
		[]
		{
			UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateStatic(&ExtendDNASkelMeshAssetActions));
		}
	);
}


FText UAssetDefinition_DNAAsset::GetAssetDisplayName() const
{
	return LOCTEXT("DNAAssetDisplayName", "DNA");
}

FLinearColor UAssetDefinition_DNAAsset::GetAssetColor() const
{
	return FColor::Green;
}

TSoftClassPtr<UObject> UAssetDefinition_DNAAsset::GetAssetClass() const
{
	return UDNA::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DNAAsset::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath{ LOCTEXT("MetaHumanAssetCategoryPath", "MetaHuman") } };
	return Categories;
}

// TODO: Have MH logo as thumbnail?
UThumbnailInfo* UAssetDefinition_DNAAsset::LoadThumbnailInfo(const FAssetData& InAssetData) const
{
	return UE::Editor::FindOrCreateThumbnailInfo(InAssetData.GetAsset(), USceneThumbnailInfo::StaticClass());
}

EAssetCommandResult UAssetDefinition_DNAAsset::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	// This should open up FSimpleAssetEditor, that should do it for now, but for future work we should probably have our own AssetEditor
	return Super::OpenAssets(InOpenArgs);
}

EAssetCommandResult UAssetDefinition_DNAAsset::GetSourceFiles(const FAssetSourceFilesArgs& InArgs, TFunctionRef<bool(const FAssetSourceFilesResult& InSourceFile)> SourceFileFunc) const
{
	for (const FAssetData& AssetData : InArgs.Assets)
	{
		if (const UDNA* DNAAsset = Cast<UDNA>(AssetData.GetAsset()))
		{
			if (DNAAsset->AssetImportData)
			{
				const FAssetImportInfo& ImportInfo = DNAAsset->AssetImportData->GetSourceData();

				for (const FAssetImportInfo::FSourceFile& SourceFile : ImportInfo.SourceFiles)
				{
					FAssetSourceFilesResult Result;

					FString ResolvedFilename = UAssetImportData::ResolveImportFilename(
						SourceFile.RelativeFilename,
						DNAAsset->GetPackage());

					if (InArgs.FilePathFormat == EPathUse::Display)
					{
						Result.FilePath = FPaths::GetCleanFilename(ResolvedFilename);
					}
					else
					{
						Result.FilePath = ResolvedFilename;
					}

					Result.DisplayLabel = SourceFile.DisplayLabelName.IsEmpty()
						? Result.FilePath
						: SourceFile.DisplayLabelName;
					Result.Timestamp = SourceFile.Timestamp;
					Result.FileHash = SourceFile.FileHash;

					if (!SourceFileFunc(Result))
					{
						break;
					}
				}

				return EAssetCommandResult::Handled;
			}
		}
	}

	return EAssetCommandResult::Unhandled;
}


#undef LOCTEXT_NAMESPACE