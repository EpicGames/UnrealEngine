// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_DNAAsset.generated.h"

UCLASS()
class UAssetDefinition_DNAAsset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~Begin UAssetDefinitionDefault interface
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& InOpenArgs) const override;
	virtual EAssetCommandResult GetSourceFiles(const FAssetSourceFilesArgs& InArgs, TFunctionRef<bool(const FAssetSourceFilesResult& InSourceFile)> SourceFileFunc) const override;
	virtual bool CanImport() const override { return true; }
	//~End UAssetDefinitionDefault interface
};