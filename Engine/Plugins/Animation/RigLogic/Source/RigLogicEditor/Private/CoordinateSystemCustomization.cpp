// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoordinateSystemCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "ScopedTransaction.h"
#include "tdm/TDM.h"

#define LOCTEXT_NAMESPACE "CoordinateSystemCustomization"

namespace
{

	const FString DirectionNames[] = {
		TEXT("Left"), TEXT("Right"), TEXT("Up"), TEXT("Down"), TEXT("Front"), TEXT("Back")
	};
	constexpr int32 NumDirections = UE_ARRAY_COUNT(DirectionNames);

	/** Which spatial dimension does this direction belong to? (0=Left/Right, 1=Up/Down, 2=Front/Back) */
	int32 GetDimensionIndex(EDirection Dir)
	{
		switch (Dir)
		{
		case EDirection::Left:  case EDirection::Right: return 0;
		case EDirection::Up:    case EDirection::Down:  return 1;
		case EDirection::Front: case EDirection::Back:  return 2;
		default: return -1;
		}
	}

	EDirection DirectionFromString(const FString& Str)
	{
		for (int32 i = 0; i < NumDirections; ++i)
		{
			if (Str == DirectionNames[i])
			{
				return static_cast<EDirection>(i);
			}
		}
		return EDirection::Left;
	}

	const FString& DirectionToString(EDirection Dir)
	{
		return DirectionNames[static_cast<int32>(Dir)];
	}

	/** Reads a single axis from a child handle. Returns FPropertyAccess::Result so callers can detect MultipleValues. */
	FPropertyAccess::Result ReadDirectionFromHandle(const TSharedPtr<IPropertyHandle>& Handle, EDirection& OutDir)
	{
		uint8 Value = 0;
		const FPropertyAccess::Result Result = Handle->GetValue(Value);
		if (Result == FPropertyAccess::Success)
		{
			OutDir = static_cast<EDirection>(Value);
		}
		return Result;
	}

}  // namespace

TSharedRef<IPropertyTypeCustomization> FCoordinateSystemCustomization::MakeInstance()
{
	return MakeShareable(new FCoordinateSystemCustomization);
}

void FCoordinateSystemCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& Utils)
{
	StructPropertyHandle = PropertyHandle;
	DetailFont = IDetailLayoutBuilder::GetDetailFont();

	// Get child handles (used for reading committed state)
	AxisHandles[0] = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCoordinateSystem, XAxis));
	AxisHandles[1] = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCoordinateSystem, YAxis));
	AxisHandles[2] = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCoordinateSystem, ZAxis));

	// Build direction options list
	for (int32 i = 0; i < NumDirections; ++i)
	{
		DirectionOptions.Add(MakeShareable(new FString(DirectionNames[i])));
	}

	// Read current committed values into staged state
	ReadFromProperty();

	// Re-sync staged state whenever the property is reset to defaults, undone, or changed externally.
	// Note: ReadFromProperty also pushes the new staged values into the combo boxes via RefreshComboSelections.
	StructPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCoordinateSystemCustomization::ReadFromProperty));

	static const FText AxisLabels[] = {
		LOCTEXT("XAxis", "X"),
		LOCTEXT("YAxis", "Y"),
		LOCTEXT("ZAxis", "Z")
	};

	// Helper to build one axis: [Label] [ComboBox]
	auto MakeAxisSlot = [this](int32 Index) -> TSharedRef<SWidget>
		{
			TSharedRef<SComboBox<TSharedPtr<FString>>> Combo = SNew(SComboBox<TSharedPtr<FString>>)
				.IsEnabled(this, &FCoordinateSystemCustomization::IsValueEditable)
				.OptionsSource(&DirectionOptions)
				.InitiallySelectedItem(FindOption(StagedAxes[Index]))
				.OnSelectionChanged_Lambda([this, Index](TSharedPtr<FString> NewVal, ESelectInfo::Type SelectInfo)
					{
						OnAxisSelectionChanged(NewVal, SelectInfo, Index);
					})
				.OnGenerateWidget(this, &FCoordinateSystemCustomization::GenerateDirectionWidget)
				.Content()
				[
					SNew(STextBlock)
						.Text(this, &FCoordinateSystemCustomization::GetCurrentText, Index)
						.Font(DetailFont)
				];

			AxisCombos[Index] = Combo;

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 4.f, 0.f)
				[
					SNew(STextBlock)
						.Text(AxisLabels[Index])
						.Font(DetailFont)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					Combo
				];
		};

	// Configure a single, well-defined reset-to-default predicate for the struct-level header row.
	// Without this, the engine falls back to comparing raw struct memory to the CDO, which produces
	// inconsistent arrow visibility across undo, multi-select, and staged-but-not-applied states.
	HeaderRow.OverrideResetToDefault(FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateSP(this, &FCoordinateSystemCustomization::IsResetToDefaultVisible),
		FResetToDefaultHandler::CreateSP(this, &FCoordinateSystemCustomization::OnResetToDefault)));

	HeaderRow
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(400.f)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					MakeAxisSlot(0)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					MakeAxisSlot(1)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					MakeAxisSlot(2)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.f, 0.f, 0.f, 0.f)
				[
					SNew(SButton)
						.OnClicked(this, &FCoordinateSystemCustomization::OnApplyClicked)
						.Visibility_Lambda([this]()
							{
								return IsValueEditable() ? EVisibility::Visible : EVisibility::Collapsed;
							})
						.IsEnabled_Lambda([this]() { return IsValid() && HasPendingChanges(); })
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ToolTipText_Lambda([this]()
							{
								if (!IsValid())
								{
									return LOCTEXT("InvalidTooltip", "Each axis must use a different spatial dimension (Left/Right, Up/Down, Front/Back).");
								}
								if (!HasPendingChanges())
								{
									return LOCTEXT("NoChangesTooltip", "No pending changes");
								}
								return LOCTEXT("ApplyTooltip", "Apply coordinate system changes");
							})
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Apply", "Apply"))
								.Font(DetailFont)
						]
				]
		];
}

void FCoordinateSystemCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& Utils)
{
	// Everything is in the header row - no child rows needed
}

void FCoordinateSystemCustomization::ReadFromProperty()
{
	for (int32 i = 0; i < 3; ++i)
	{
		bIndeterminate[i] = false;
		if (AxisHandles[i].IsValid())
		{
			EDirection ReadValue = StagedAxes[i];
			const FPropertyAccess::Result Result = ReadDirectionFromHandle(AxisHandles[i], ReadValue);
			if (Result == FPropertyAccess::Success)
			{
				StagedAxes[i] = ReadValue;
			}
			else if (Result == FPropertyAccess::MultipleValues)
			{
				// Leave StagedAxes[i] at its previous value; mark indeterminate so UI/Apply behave correctly.
				bIndeterminate[i] = true;
			}
		}
	}

	RefreshComboSelections();
}

void FCoordinateSystemCustomization::RefreshComboSelections()
{
	for (int32 i = 0; i < 3; ++i)
	{
		if (AxisCombos[i].IsValid())
		{
			if (bIndeterminate[i])
			{
				AxisCombos[i]->ClearSelection();
			}
			else
			{
				AxisCombos[i]->SetSelectedItem(FindOption(StagedAxes[i]));
			}
		}
	}
}

TSharedPtr<FString> FCoordinateSystemCustomization::FindOption(EDirection Dir) const
{
	const int32 Idx = static_cast<int32>(Dir);
	if (DirectionOptions.IsValidIndex(Idx))
	{
		return DirectionOptions[Idx];
	}
	return nullptr;
}

bool FCoordinateSystemCustomization::HasPendingChanges() const
{
	for (int32 i = 0; i < 3; ++i)
	{
		if (bIndeterminate[i])
		{
			// Until the user disambiguates with a real selection, don't treat this as a pending change.
			continue;
		}
		if (AxisHandles[i].IsValid())
		{
			EDirection Committed = StagedAxes[i];
			const FPropertyAccess::Result Result = ReadDirectionFromHandle(AxisHandles[i], Committed);
			if (Result == FPropertyAccess::Success && StagedAxes[i] != Committed)
			{
				return true;
			}
		}
	}
	return false;
}

bool FCoordinateSystemCustomization::IsValid() const
{
	// Indeterminate axes can't be validated until the user picks a value.
	if (bIndeterminate[0] || bIndeterminate[1] || bIndeterminate[2])
	{
		return false;
	}
	const int32 DimX = GetDimensionIndex(StagedAxes[0]);
	const int32 DimY = GetDimensionIndex(StagedAxes[1]);
	const int32 DimZ = GetDimensionIndex(StagedAxes[2]);
	return (DimX != DimY) && (DimX != DimZ) && (DimY != DimZ);
}

bool FCoordinateSystemCustomization::IsValueEditable() const
{
	return StructPropertyHandle.IsValid() && !StructPropertyHandle->IsEditConst();
}

FReply FCoordinateSystemCustomization::OnApplyClicked()
{
	FScopedTransaction Transaction(LOCTEXT("SetCoordinateSystem", "Set Coordinate System"));

	// Single pre-change notification
	StructPropertyHandle->NotifyPreChange();

	// Write all three axes at once via raw memory - no per-property notifications
	TArray<void*> RawData;
	StructPropertyHandle->AccessRawData(RawData);
	for (void* Data : RawData)
	{
		if (FCoordinateSystem* CS = static_cast<FCoordinateSystem*>(Data))
		{
			CS->XAxis = StagedAxes[0];
			CS->YAxis = StagedAxes[1];
			CS->ZAxis = StagedAxes[2];
		}
	}

	// Single post-change notification - one event to UDNA
	StructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);

	// Re-sync from the (now committed) property and push selections back into the combos.
	ReadFromProperty();

	return FReply::Handled();
}

bool FCoordinateSystemCustomization::IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> /*Handle*/) const
{
	// Don't trust DiffersFromDefault() on the struct handle - for native USTRUCTs without
	// a registered struct-on-scope diff path, the result flickers across multi-select,
	// archetype propagation, and transient outers. Ask each child (enum) handle instead;
	// those answer reliably.
	for (int32 i = 0; i < 3; ++i)
	{
		if (AxisHandles[i].IsValid() && AxisHandles[i]->DiffersFromDefault())
		{
			return true;
		}
	}

	// Also show the arrow when the user has staged but not yet applied changes,
	// so they can discard them without pressing Apply first.
	return HasPendingChanges();
}

void FCoordinateSystemCustomization::OnResetToDefault(TSharedPtr<IPropertyHandle> /*Handle*/)
{
	if (!StructPropertyHandle.IsValid())
	{
		return;
	}

	// Reset the *struct* handle (not its children). Internally this routes through
	// FPropertyValueImpl::ImportText on the FStructProperty, which does:
	//   - one NotifyPreChange,
	//   - ImportText_Direct of the entire struct's default value into each instance,
	//   - one NotifyPostChange (with EPropertyChangeType::ValueSet | ResetToDefault).
	//
	// So the downstream UDNA conversion still fires exactly once per object, matching
	// the one-shot invariant that OnApplyClicked relies on. (Crucially: do NOT loop and
	// reset each child AxisHandle - that would fire three separate conversions.)
	//
	// Bonus: this handles archetype/CDO sourcing, nested USTRUCT properties (e.g. the
	// FCoordinateSystem inside FDNAConfig), multi-select, and array/map containers
	// correctly, which we couldn't do reliably from a manual ContainerPtrToValuePtr walk.
	StructPropertyHandle->ResetToDefault();

	// Pick up the new committed values and refresh the combos.
	ReadFromProperty();
}

void FCoordinateSystemCustomization::OnAxisSelectionChanged(
	TSharedPtr<FString> NewValue,
	ESelectInfo::Type SelectInfo,
	int32 AxisIndex)
{
	// Ignore programmatic selection changes (from SetSelectedItem in RefreshComboSelections);
	// only react to genuine user input from the dropdown.
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	if (NewValue.IsValid() && AxisIndex >= 0 && AxisIndex < 3)
	{
		StagedAxes[AxisIndex] = DirectionFromString(*NewValue);
		// User has now disambiguated this axis; clear any indeterminate flag.
		bIndeterminate[AxisIndex] = false;
	}
}

TSharedRef<SWidget> FCoordinateSystemCustomization::GenerateDirectionWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(InItem.IsValid() ? FText::FromString(*InItem) : FText::GetEmpty())
		.Font(DetailFont);
}

FText FCoordinateSystemCustomization::GetCurrentText(int32 AxisIndex) const
{
	if (AxisIndex >= 0 && AxisIndex < 3)
	{
		if (bIndeterminate[AxisIndex])
		{
			return LOCTEXT("MultipleValues", "Multiple Values");
		}
		return FText::FromString(DirectionToString(StagedAxes[AxisIndex]));
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
