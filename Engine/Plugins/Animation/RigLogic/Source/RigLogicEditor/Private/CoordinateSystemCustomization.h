// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "DNACommon.h"

template <typename OptionType> class SComboBox;

class FCoordinateSystemCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& Utils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& Utils) override;

private:
	/** Read the current committed values from the property handles */
	void ReadFromProperty();

	/** Push the current staged values into the combo boxes' selected-item state */
	void RefreshComboSelections();

	/** Look up the shared options pointer for a given direction (combo box compares by pointer) */
	TSharedPtr<FString> FindOption(EDirection Dir) const;

	/** Is the staged state different from the committed property? */
	bool HasPendingChanges() const;

	/** Is the staged coordinate system valid? (3 distinct spatial dimensions, no indeterminate axes) */
	bool IsValid() const;

	/**
	 * Whether the value widget should accept edits. Mirrors the host
	 * UPROPERTY's edit-const state (e.g. a VisibleAnywhere member is
	 * read-only) so the combo boxes and Apply button don't bypass it.
	 */
	bool IsValueEditable() const;

	FReply OnApplyClicked();

	/** Reset-to-default override hooks */
	bool IsResetToDefaultVisible(TSharedPtr<IPropertyHandle> Handle) const;
	void OnResetToDefault(TSharedPtr<IPropertyHandle> Handle);

	TSharedRef<SWidget> GenerateDirectionWidget(TSharedPtr<FString> InItem);
	FText GetCurrentText(int32 AxisIndex) const;
	void OnAxisSelectionChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo, int32 AxisIndex);

	/** The struct-level property handle */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Child handles for XAxis, YAxis, ZAxis */
	TSharedPtr<IPropertyHandle> AxisHandles[3];

	/** Combo box widgets for XAxis, YAxis, ZAxis (kept so we can sync their selected item with StagedAxes) */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> AxisCombos[3];

	/** Staged values (local edits before Apply) */
	EDirection StagedAxes[3] = { EDirection::Left, EDirection::Front, EDirection::Up };

	/** Per-axis indeterminate flag (set when GetValue returns MultipleValues across a multi-select) */
	bool bIndeterminate[3] = { false, false, false };

	/** Options for the combo boxes */
	TArray<TSharedPtr<FString>> DirectionOptions;

	/** Cached detail panel font */
	FSlateFontInfo DetailFont;
};
