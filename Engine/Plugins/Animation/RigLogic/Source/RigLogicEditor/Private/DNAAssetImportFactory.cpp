// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAAssetImportFactory.h"
#include "DNAAssetUserData.h"
#include "DNA.h"
#include "DNAAutoConfig.h"
#include "DNAReader.h"
#include "DNAUtils.h"

#include "Engine/SkeletalMesh.h"
#include "UObject/ConstructorHelpers.h"
#include "Editor.h"

#include "ProfilingDebugging/ScopedTimers.h"

#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "AssetImportTask.h"
#include "EditorFramework/AssetImportData.h"
#include "Misc/FileHelper.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/Package.h"

#include "DNAImporter.h"
#include "DNAImportSettings.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DNAAssetImportFactory)

DEFINE_LOG_CATEGORY(LogDNAImportFactory);
#define LOCTEXT_NAMESPACE "DNAAssetImportFactory"

namespace UE::RigLogicEditor::Private
{
	enum class EDNAImportCoordinateSystemChoice : uint8
	{
		Default,
		MetaHumanAssemblyCompatible,
	};

	/** Format an FCoordinateSystem as "(X, Y, Z)" using the EDirection display names. */
	static FText FormatCoordinateSystem(const FCoordinateSystem& CoordinateSystem)
	{
		const UEnum* DirectionEnum = StaticEnum<EDirection>();
		const FText XText = DirectionEnum->GetDisplayNameTextByValue(static_cast<int64>(CoordinateSystem.XAxis));
		const FText YText = DirectionEnum->GetDisplayNameTextByValue(static_cast<int64>(CoordinateSystem.YAxis));
		const FText ZText = DirectionEnum->GetDisplayNameTextByValue(static_cast<int64>(CoordinateSystem.ZAxis));
		return FText::Format(LOCTEXT("DNAImportOptions_CoordinateSystemFormat", "({0}, {1}, {2})"), XText, YText, ZText);
	}

	/**
	 * Small modal prompt shown when a .dna file is dropped / interactively imported.
	 * Returns true if the user pressed Import; false if they cancelled (closed the window).
	 * InOutChoice is seeded with the initial selection and overwritten with the user's pick.
	 */
	static bool ShowDNAImportOptionsDialog(const FString& Filename, EDNAImportCoordinateSystemChoice& InOutChoice)
	{
		TSharedRef<EDNAImportCoordinateSystemChoice> SelectedChoice = MakeShared<EDNAImportCoordinateSystemChoice>(InOutChoice);

		TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("DNAImportOptions_Title", "DNA Import Options"))
			.SizingRule(ESizingRule::Autosized)
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		auto MakeRadioButton = [&SelectedChoice](EDNAImportCoordinateSystemChoice ThisChoice, const FText& Label, const FText& Tooltip)
		{
			return SNew(SCheckBox)
				.Style(FAppStyle::Get(), "RadioButton")
				.ToolTipText(Tooltip)
				.IsChecked_Lambda([SelectedChoice, ThisChoice]()
				{
					return (*SelectedChoice == ThisChoice) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([SelectedChoice, ThisChoice](ECheckBoxState NewState)
				{
					if (NewState == ECheckBoxState::Checked)
					{
						*SelectedChoice = ThisChoice;
					}
				})
				[
					SNew(STextBlock)
					.Text(Label)
					.Margin(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				];
		};

		bool bConfirmed = false;

		Window->SetContent(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(10.0f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.0f, 0.0f, 0.0f, 8.0f))
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("DNAImportOptions_ImportFile", "Import \"{0}\""),
						FText::FromString(FPaths::GetCleanFilename(Filename))))
					.ToolTipText(FText::FromString(Filename))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0.0f, 4.0f, 0.0f, 4.0f))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DNAImportOptions_CoordinateSystem", "Coordinate System:"))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(12.0f, 2.0f, 0.0f, 2.0f))
				[
					MakeRadioButton(
						EDNAImportCoordinateSystemChoice::Default,
						FText::Format(LOCTEXT("DNAImportOptions_Default", "Default  {0}"),
							FormatCoordinateSystem(GetDefault<UDNAImportSettings>()->DefaultDNAConfig.CoordinateSystem)),
						LOCTEXT("DNAImportOptions_Default_Tooltip",
							"Import the DNA using the project's default coordinate system, configured in "
							"Project Settings \u2192 Plugins \u2192 RigLogic."))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(12.0f, 2.0f, 0.0f, 2.0f))
				[
					MakeRadioButton(
						EDNAImportCoordinateSystemChoice::MetaHumanAssemblyCompatible,
						FText::Format(LOCTEXT("DNAImportOptions_MetaHumanAssembly", "MetaHuman Assembly Compatible  {0}"),
							FormatCoordinateSystem(FDNAConfig::Legacy().CoordinateSystem)),
						LOCTEXT("DNAImportOptions_MetaHumanAssembly_Tooltip",
							"Import the DNA using the legacy MetaHuman coordinate system (FDNAConfig::Legacy: "
							"Left/Down/Front, rotation XYZ, face winding CW). Required for compatibility with "
							"existing MetaHuman skeletal meshes."))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.Padding(FMargin(0.0f, 12.0f, 0.0f, 0.0f))
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FMargin(4.0f, 0.0f))
					+ SUniformGridPanel::Slot(0, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("DNAImportOptions_Import", "Import"))
						.OnClicked_Lambda([&bConfirmed, WindowPtr = TWeakPtr<SWindow>(Window)]()
						{
							bConfirmed = true;
							if (TSharedPtr<SWindow> Pinned = WindowPtr.Pin())
							{
								Pinned->RequestDestroyWindow();
							}
							return FReply::Handled();
						})
					]
					+ SUniformGridPanel::Slot(1, 0)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("DNAImportOptions_Cancel", "Cancel"))
						.OnClicked_Lambda([WindowPtr = TWeakPtr<SWindow>(Window)]()
						{
							if (TSharedPtr<SWindow> Pinned = WindowPtr.Pin())
							{
								Pinned->RequestDestroyWindow();
							}
							return FReply::Handled();
						})
					]
				]
			]
		);

		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow = MainFrame.GetParentWindow();
		}

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow);

		InOutChoice = *SelectedChoice;
		return bConfirmed;
	}

	static FDNAConfig BuildMetaHumanAssemblyCompatibleDNAConfig()
	{
		FDNAConfig DNAConfig = GetDefault<UDNAImportSettings>()->DefaultDNAConfig;
		const FDNAConfig LegacyConfig = FDNAConfig::Source();
		DNAConfig.CoordinateSystem = LegacyConfig.CoordinateSystem;
		DNAConfig.RotationSign = LegacyConfig.RotationSign;
		DNAConfig.RotationSequence = LegacyConfig.RotationSequence;
		DNAConfig.FaceWindingOrder = LegacyConfig.FaceWindingOrder;
		DNAConfig.CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Transform;
		return DNAConfig;
	}

	/** Build a DNAConfig from the project-level default DNA settings. */
	static FDNAConfig BuildDefaultDNAConfig()
	{
		FDNAConfig DNAConfig = GetDefault<UDNAImportSettings>()->DefaultDNAConfig;
		DNAConfig.CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Transform;
		return DNAConfig;
	}

	/**
	 * Peek a .dna file by loading only the (small) Descriptor layer and check whether
	 * its coordinate-system metadata matches FDNAConfig::Legacy() (Left/Down/Front,
	 * RotationSign Positive/Positive/Positive, RotationSequence XYZ, FaceWindingOrder CW).
	 * Used to pre-tick the "MHC Compatible" checkbox in the import dialog.
	 * Returns false on any failure (treat as "not legacy").
	 */
	static bool DoesDNAFileMatchLegacyCoordinateSystem(const FString& Path)
	{
		FDNAConfig ProbeConfig;
		ProbeConfig.Layers = static_cast<int32>(EDNADataLayer::Descriptor);

		TSharedPtr<IDNAReader> DNAReader = LoadDNAFromFile(Path, ProbeConfig);
		if (!DNAReader.IsValid())
		{
			return false;
		}

		const FDNAConfig LegacyConfig = FDNAConfig::Legacy();
		return DNAReader->GetCoordinateSystem() == LegacyConfig.CoordinateSystem
			&& DNAReader->GetRotationSign() == LegacyConfig.RotationSign
			&& DNAReader->GetRotationSequence() == LegacyConfig.RotationSequence
			&& DNAReader->GetFaceWindingOrder() == LegacyConfig.FaceWindingOrder;
	}
}

UDNAAssetImportFactory::UDNAAssetImportFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	bEditAfterNew = true;
	SupportedClass = UDNA::StaticClass();

	Formats.Add(TEXT("dna;Character DNA file"));
}

void UDNAAssetImportFactory::PostInitProperties()
{
	Super::PostInitProperties();
	bEditorImport = true;
	bText = false;
}

bool UDNAAssetImportFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UDNA* DNAAsset = Cast<UDNA>(Obj);
	if (DNAAsset && DNAAsset->AssetImportData)
	{
		FString Filename = DNAAsset->AssetImportData->GetFirstFilename();
		OutFilenames.Add(Filename);

		return !Filename.IsEmpty();
	}

	return false;
}

void UDNAAssetImportFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UDNA* DNAAsset = Cast<UDNA>(Obj);
	if (DNAAsset && ensure(NewReimportPaths.Num() == 1) && DNAAsset->AssetImportData)
	{
		DNAAsset->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

EReimportResult::Type UDNAAssetImportFactory::Reimport(UObject* Obj)
{
	UDNA* DNAAsset = Cast<UDNA>(Obj);
	bool Result = false;

	if(DNAAsset && DNAAsset->AssetImportData->GetFirstFilename().Len() > 0)
	{
		Result = DNAAsset->Init(DNAAsset->AssetImportData->GetFirstFilename(), DNAAsset->DNAConfig);
	} 

	if (Result)
	{
		const FDNAConfig LegacyConfig = FDNAConfig::Legacy();
		if (DNAAsset->DNAConfig.CoordinateSystem == LegacyConfig.CoordinateSystem &&
			DNAAsset->DNAConfig.RotationSign == LegacyConfig.RotationSign &&
			DNAAsset->DNAConfig.RotationSequence == LegacyConfig.RotationSequence &&
			DNAAsset->DNAConfig.FaceWindingOrder == LegacyConfig.FaceWindingOrder)
		{
			DNAAsset->RestoreLegacyUEMHCCompatibility();
		}
		DNAAsset->MarkPackageDirty();
		return EReimportResult::Succeeded;
	}

	return EReimportResult::Failed;
}

UObject* UDNAAssetImportFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	FDNAConfig DNAConfig;
	bool bDNAConfigResolved = false;
	if (UDNAConfigHolder* DNAConfigHolder = AssetImportTask ? Cast<UDNAConfigHolder>(AssetImportTask->Options) : nullptr)
	{
		DNAConfig = DNAConfigHolder->Config;
		bDNAConfigResolved = true;
	}

	using namespace UE::RigLogicEditor::Private;
	// Interactive imports (e.g. drag-and-drop into the Content Browser) get a small prompt
	// asking which coordinate system to use. Automated/scripted imports skip the prompt and
	// fall through to AutoDetectDNAConfig exactly as before.
	EDNAImportCoordinateSystemChoice Choice = EDNAImportCoordinateSystemChoice::Default;
	if (!bDNAConfigResolved && !IsAutomatedImport())
	{
		// Pre-select the MetaHuman option if the source file's coordinate system already
		// matches Legacy, otherwise default to the Unreal Engine option.
		Choice = DoesDNAFileMatchLegacyCoordinateSystem(Filename)
			? EDNAImportCoordinateSystemChoice::MetaHumanAssemblyCompatible
			: EDNAImportCoordinateSystemChoice::Default;

		const bool bConfirmed = ShowDNAImportOptionsDialog(Filename, Choice);
		if (!bConfirmed)
		{
			bOutOperationCanceled = true;
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}

		switch (Choice)
		{
		case EDNAImportCoordinateSystemChoice::MetaHumanAssemblyCompatible:
			DNAConfig = BuildMetaHumanAssemblyCompatibleDNAConfig();
			break;
		case EDNAImportCoordinateSystemChoice::Default:
		default:
			DNAConfig = BuildDefaultDNAConfig();
			break;
		}
		bDNAConfigResolved = true;
	}

	if (!bDNAConfigResolved)
	{
		DNAConfig = AutoDetectDNAConfig(Filename);
	}

	UDNA* DNAAsset = NewObject<UDNA>(InParent, InClass, InName, InFlags | RF_Public | RF_Standalone | RF_Transactional);

	// Seed RigLogicConfiguration from project default settings so per-platform calculation type,
	// floating point type, etc. are pre-filled on new assets (same pattern as DefaultDNAConfig above).
	// Must be set before Init() since InitializeRigRuntimeContext() reads RigLogicConfiguration directly.
	DNAAsset->RigLogicConfiguration = GetDefault<UDNAImportSettings>()->DefaultRigLogicConfiguration;

	if (DNAAsset->Init(*Filename, DNAConfig))
	{
		if (!DNAAsset->AssetImportData)
		{
			DNAAsset->AssetImportData = NewObject<UAssetImportData>(DNAAsset);
		}
				
		DNAAsset->AssetImportData->UpdateFilenameOnly(Filename);

		if (Choice == EDNAImportCoordinateSystemChoice::MetaHumanAssemblyCompatible)
		{
			DNAAsset->RestoreLegacyUEMHCCompatibility();
		}
		
		FNotificationInfo Info(LOCTEXT("DNA_ImportSuccessMessage", "DNA file successfully imported!"));
		Info.ExpireDuration = 3.0f;
		Info.bFireAndForget = true;
		Info.bUseLargeFont = false;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Success);
		}

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, DNAAsset);
		return DNAAsset;
	}
	else
	{
		// TODO: Use FMessageLog instead
		const FText Message = LOCTEXT("DNAImportFailed", "DNA asset failed to initialize from given file:");
		FMessageDialog::Open(EAppMsgType::Ok, Message);
		UE_LOGFMT(LogDNAImportFactory, Warning, "{} {}", *Message.ToString(), Filename);
	}

	// Failed to load file or create DNA stream. Clean and return nullptr.
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
	return nullptr;
}

bool UDNAAssetImportFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(Filename);

	if (Extension == TEXT("dna"))
	{
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

