// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAAssetImportWindow.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "IDocumentation.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"

#define LOCTEXT_NAMESPACE "DNAAssetImportWindow"

void SDNAAssetImportWindow::Construct(const FArguments& InArgs)
{
	WidgetWindow = InArgs._WidgetWindow;

	TSharedPtr<SBox> ImportTypeDisplay;
	TSharedPtr<SHorizontalBox> DNAHeaderButtons;
	TSharedPtr<SBox> InspectorBox;
	this->ChildSlot
		[
			SNew(SBox)
			.MaxDesiredHeight(InArgs._MaxWindowHeight)
		.MaxDesiredWidth(InArgs._MaxWindowWidth)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(ImportTypeDisplay, SBox)
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("CurveEd.LabelFont"))
		.Text(LOCTEXT("Import_CurrentFileTitle", "Current Asset: "))
		]
	+ SHorizontalBox::Slot()
		.Padding(5, 0, 0, 0)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
		.Text(InArgs._FullPath)
		.ToolTipText(InArgs._FullPath)
		]
		]
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2)
		[
			SAssignNew(InspectorBox, SBox)
			.MaxDesiredHeight(650.0f)
		.WidthOverride(400.0f)
		]
	+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
		+ SUniformGridPanel::Slot(0, 0)
		[
			IDocumentation::Get()->CreateAnchor(FString("fbx-import-options-reference-in-unreal-engine"))
		]
	+ SUniformGridPanel::Slot(1, 0)
		[
			SAssignNew(ImportButton, SButton)
			.HAlign(HAlign_Center)
		.Text(LOCTEXT("DNAAssetImportWindow_Import", "Import"))
		.IsEnabled(this, &SDNAAssetImportWindow::CanImport)
		.OnClicked(this, &SDNAAssetImportWindow::OnImport)
		]
	+ SUniformGridPanel::Slot(2, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
		.Text(LOCTEXT("DNAAssetImportWindow_Cancel", "Cancel"))
		.ToolTipText(LOCTEXT("DNAAssetImportWindow_Cancel_ToolTip", "Cancels importing this DNA file"))
		.OnClicked(this, &SDNAAssetImportWindow::OnCancel)
		]
		]
		]
		];

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InspectorBox->SetContent(DetailsView->AsShared());

	ImportTypeDisplay->SetContent(
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SDNAAssetImportWindow::GetImportTypeDisplayText)
		]
	+ SHorizontalBox::Slot()
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
		[
			SAssignNew(DNAHeaderButtons, SHorizontalBox)
			+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f))
		[
			SNew(SButton)
			.Text(LOCTEXT("DNAAssetImportWindow_ResetOptions", "Reset to Default"))
		.OnClicked(this, &SDNAAssetImportWindow::OnResetToDefaultClick)
		]
		]
		]
		]
	);
}

FReply SDNAAssetImportWindow::OnResetToDefaultClick() const
{
	return FReply::Handled();
}

FText SDNAAssetImportWindow::GetImportTypeDisplayText() const
{
	return LOCTEXT("DNAAssetImportWindow_ImportTypeAnim", "Import DNA");
}

bool SDNAAssetImportWindow::CanImport()  const
{
	// do test to see if we are ready to import
	return true;
}

#undef LOCTEXT_NAMESPACE
