// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DNACommon.h"

FDNAConfig AutoDetectDNAConfig(const FString& Path);
