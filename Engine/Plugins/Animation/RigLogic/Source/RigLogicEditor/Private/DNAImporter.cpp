// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Main implementation of FDNAImporter : import DNA data to Skeletal Mesh
=============================================================================*/

#include "DNAImporter.h"
#include "AssetToolsModule.h"
#include "AssetImportTask.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DNAAssetImportFactory.h"
#include "EditorReimportHandler.h" 
#include "EditorFramework/AssetImportData.h"
#include "DNA.h"
#include "DNAUtils.h"
#include "DNAAutoConfig.h"
#include "UObject/UObjectGlobals.h"
#include "DNAAssetUserData.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "HAL/PlatformApplicationMisc.h"
#include "DesktopPlatformModule.h"
#include "EditorDirectories.h"

DEFINE_LOG_CATEGORY_STATIC(LogDNAImporter, Log, All);

#define LOCTEXT_NAMESPACE "DNAImport"

UDNA* UDNAImporter::ImportDNAWithPrompt(TArray<USkeletalMesh*> SkeletalMeshes)
{
	const FString Filename =  PromptForDNAImportFile();

	if(SkeletalMeshes.IsEmpty())
	{
		const FText Message = LOCTEXT("DNA_ImportSkeletalMeshMissing", "Skeletal Mesh was not provided for Import function.");
		UE_LOGFMT(LogDNAImporter, Error, "{}", *Message.ToString());
		return nullptr;
	}
	UDNA* ImportedDNA = ImportDNA(Filename, SkeletalMeshes[0]);

	if(ImportedDNA)
	{
		for (USkeletalMesh* SkelMesh : SkeletalMeshes)
		{
			// If there is a legacy DNA Asset User data, we should remove it
			if (UDNAAsset* LegacyDNA = SkelMesh->GetAssetUserData<UDNAAsset>())
			{
				SkelMesh->RemoveUserDataOfClass(UDNAAsset::StaticClass());
			}

			UDNAAssetUserData* DNAAssetUserData = NewObject<UDNAAssetUserData>(SkelMesh);
			DNAAssetUserData->DNAAsset = ImportedDNA;
			SkelMesh->AddAssetUserData(DNAAssetUserData);
			SkelMesh->MarkPackageDirty();
		}
		return ImportedDNA;
	}
	else
	{
		return nullptr;
	}
}

bool UDNAImporter::ImportDNAAutomated(const FString FileName, USkeletalMesh* SkeletalMesh, bool bReplaceExisting /* = false*/)
{
	UDNA* ImportedDNA = ImportDNA(FileName, SkeletalMesh, bReplaceExisting);
	if (ImportedDNA)
	{
		SkeletalMesh->RemoveUserDataOfClass(UDNAAsset::StaticClass());
		
		UDNAAssetUserData* DNAUserData = NewObject<UDNAAssetUserData>(SkeletalMesh);
		DNAUserData->DNAAsset = ImportedDNA;
		SkeletalMesh->AddAssetUserData(DNAUserData);
		SkeletalMesh->MarkPackageDirty();
		
		return true;
	}

	return false;
}

UDNA* UDNAImporter::ImportDNA(const FString Filename, USkeletalMesh* SkeletalMesh, bool bReplaceExisting /* = false*/)
{
	UDNAAssetImportFactory* DNAFactory = NewObject<UDNAAssetImportFactory>(UDNAAssetImportFactory::StaticClass());
	// We get the package of the first of selected SkeletalMeshes
	FString ImportPackagePath = FPackageName::GetLongPackagePath(SkeletalMesh->GetOutermost()->GetName());
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	FString AssetName = FPaths::GetBaseFilename(Filename);

	// Check for an existing object
	FString PathToCheck = FString::Printf(TEXT("%s/%s.%s"), *ImportPackagePath, *AssetName, *AssetName);
	FAssetData ExistingData = FAssetRegistryModule::GetRegistry().GetAssetByObjectPath(PathToCheck);
	
	// If we have an existing asset with same name (Skeletal Mesh for example) we should add DNA suffix or create unique name if we don't want to replace existing DNA
	if (ExistingData.IsValid())
	{
		// Check if the existing asset is already a DNA Asset
		if (ExistingData.GetClass()->IsChildOf(UDNA::StaticClass()))
		{
			if (!bReplaceExisting)
			{
				FString BasePackageName = ImportPackagePath / AssetName;
				FString PackageName;
				
				AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), PackageName, AssetName);
			}
		}
		else
		{
			// Asset exists but is not a DNA
			FString DNAAssetName = AssetName + TEXT("_DNA");
			FString DNAPathToCheck = FString::Printf(TEXT("%s/%s.%s"), *ImportPackagePath, *DNAAssetName, *DNAAssetName);
			FAssetData ExistingDNAData = FAssetRegistryModule::GetRegistry().GetAssetByObjectPath(DNAPathToCheck);

			if (ExistingDNAData.IsValid() && bReplaceExisting)
			{
				AssetName = DNAAssetName;
			}
			else
			{
				FString BasePackageName = ImportPackagePath / DNAAssetName;
				FString PackageName;
				AssetToolsModule.Get().CreateUniqueAssetName(BasePackageName, TEXT(""), PackageName, AssetName);
			}
		}
	}

	TStrongObjectPtr<UAssetImportTask> ImportTask(NewObject<UAssetImportTask>(this));
	ImportTask->Filename = Filename;
	ImportTask->DestinationPath = ImportPackagePath;
	ImportTask->DestinationName = AssetName;
	ImportTask->bAutomated = true;
	ImportTask->bReplaceExisting = bReplaceExisting;
	ImportTask->Factory = DNAFactory;
	// Forward existing DNAConfig if importing on existing asset
	if (UDNAAssetUserData* UserData = SkeletalMesh->GetAssetUserData<UDNAAssetUserData>())
	{
		if (UDNA* OldDNA = UserData->DNAAsset)
		{
			UDNAConfigHolder* DNAConfigHolder = NewObject<UDNAConfigHolder>();
			DNAConfigHolder->Config = OldDNA->DNAConfig;
			ImportTask->Options = DNAConfigHolder;
		}
	}
	else if (UDNAAsset* OldDNA = SkeletalMesh->GetAssetUserData<UDNAAsset>())
	{
		UDNAConfigHolder* DNAConfigHolder = NewObject<UDNAConfigHolder>();
		DNAConfigHolder->Config = FDNAConfig::Legacy(ECoordinateSystemTransformPolicy::Transform);
		ImportTask->Options = DNAConfigHolder;
	}
	else
	{
		UDNAConfigHolder* DNAConfigHolder = NewObject<UDNAConfigHolder>();
		DNAConfigHolder->Config = AutoDetectDNAConfig(Filename);
		ImportTask->Options = DNAConfigHolder;
	}

	FAssetToolsModule::GetModule().Get().ImportAssetTasks({ ImportTask.Get()});

	if (ImportTask->ImportedObjectPaths.Num() == 1)
	{
		UDNA* ImportedDNA = Cast<UDNA>(ImportTask->GetObjects()[0]);
		return ImportedDNA;
	}

	return nullptr;
}

void UDNAImporter::ExportDNA(UDNA* DNAToExport, const FString FolderForExport)
{
	if(!DNAToExport)
	{
		UE_LOGF(LogDNAImporter, Error, "DNA asset provided for export isn't valid.");
		return;
	}

	FString FileName = FPaths::MakeValidFileName(DNAToExport->GetName());
	const FString Extension = TEXT(".dna");
	const FString FullExportPath = FPaths::Combine(FolderForExport, FileName + Extension);

	FText Message;
	FText DNAName = FText::FromString(DNAToExport->GetName());
	bool bWritingDNAToFileDone = false;

	if (!FolderForExport.IsEmpty() && IFileManager::Get().DirectoryExists(*FolderForExport))
	{
		// UEMHC Legacy case, see UDNA::RestoreLegacyUEMHCCompatibility where DNA data != DNAConfig
		TSharedPtr<IDNAReader> DNAReader = DNAToExport->GetDNAReader();
		const bool IsSameCoordinateSystem = (DNAReader->GetCoordinateSystem() == DNAToExport->DNAConfig.CoordinateSystem &&
											DNAReader->GetRotationSequence() == DNAToExport->DNAConfig.RotationSequence &&
											DNAReader->GetRotationSign() == DNAToExport->DNAConfig.RotationSign &&
											DNAReader->GetFaceWindingOrder() == DNAToExport->DNAConfig.FaceWindingOrder);
		if (!IsSameCoordinateSystem)
		{
			TArray<uint8> DNABuffer;
			SaveDNAToBuffer(DNAReader.Get(), EDNADataLayer::All, DNABuffer);
			DNAReader = LoadDNAFromBuffer(&DNABuffer, DNAToExport->DNAConfig);
		}

		WriteDNAToFile(DNAReader.Get(), EDNADataLayer::All, FullExportPath);

		Message = FText::Format(LOCTEXT("DNA_ExportMessageSuccess", "DNA Asset {0} successfuly exported!"), DNAName);
		bWritingDNAToFileDone = true;
	}
	else
	{
		Message = FText::Format(LOCTEXT("DNA_ExportMessageFailed", "DNA Asset {0} failed to export! Folder or DNA provided aren't valid."), DNAName);
	}

	FNotificationInfo Info(Message);
	Info.ExpireDuration = 3.0f;
	Info.bUseLargeFont = false;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if (Notification.IsValid())
	{
		Notification->SetCompletionState(bWritingDNAToFileDone ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
	}
}

bool UDNAImporter::ConvertFromLegacyAssetUserData(USkeletalMesh* SkeletalMesh)
{
	if(UDNAAsset* LegacyDNA = SkeletalMesh->GetAssetUserData<UDNAAsset>())
	{
		UPackage* SkeletalMeshPackage = SkeletalMesh->GetOutermost();
		FString SkeletalMeshPackagePath = SkeletalMeshPackage->GetName();

		const FString DNAName = FString::Printf(TEXT("%s_DNA"), *SkeletalMesh->GetName());
		FString DNAPackagePath = FPackageName::GetLongPackagePath(SkeletalMeshPackagePath);
		DNAPackagePath /= DNAName;

		UPackage* DNAPackage = CreatePackage(*DNAPackagePath);

		UDNA* NewDNAAsset = NewObject<UDNA>(DNAPackage, UDNA::StaticClass());
		NewDNAAsset->SetDNAReader(LegacyDNA->GetDNAReader());
		NewDNAAsset->AssetImportData = DuplicateObject(LegacyDNA->AssetImportData.Get(), NewDNAAsset);

		NewDNAAsset->Rename(*DNAName, DNAPackage);
		NewDNAAsset->SetFlags(RF_Public | RF_Standalone | RF_Transactional);
		FAssetRegistryModule::AssetCreated(NewDNAAsset);
		NewDNAAsset->MarkPackageDirty();
		

		SkeletalMesh->RemoveUserDataOfClass(UDNAAsset::StaticClass());
		UDNAAssetUserData* DNAAssetUserData = NewObject<UDNAAssetUserData>(SkeletalMesh);
		DNAAssetUserData->DNAAsset = NewDNAAsset;
		SkeletalMesh->AddAssetUserData(DNAAssetUserData);
		SkeletalMesh->MarkPackageDirty();

		return true;
	}
	else
	{
		FText SkelMeshName = FText::FromString(SkeletalMesh->GetName());
		const FText Message = FText::Format(LOCTEXT("DNA_CreateErrorMessage", "{0} SkeletalMesh doesn't have Legacy DNA Asset User Data attached!"), SkelMeshName);
		FNotificationInfo Info(Message);
		Info.ExpireDuration = 3.0f;
		Info.bUseLargeFont = false;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}

		return false;
	}
}

void UDNAImporter::ReimportDNA(UDNA* ReimportDNA)
{
	if(!ReimportDNA)
	{
		const FText Message = LOCTEXT("DNA_ReimportFailedInvalidDNA", "Invalid DNA for reimport!");
		UE_LOGFMT(LogDNAImporter, Error, "{}", *Message.ToString());
		return;
	}

	if (ReimportDNA->AssetImportData && !ReimportDNA->AssetImportData->GetFirstFilename().IsEmpty())
	{
		FString Filename = ReimportDNA->AssetImportData->GetFirstFilename();
		UDNAAssetImportFactory* DNAFactory = NewObject<UDNAAssetImportFactory>(UDNAAssetImportFactory::StaticClass());
		
		bool bSuccess = FReimportManager::Instance()->Reimport(ReimportDNA, false, true, Filename, DNAFactory);
			
		if (!bSuccess)
		{
			const FText Message = LOCTEXT("DNA_ReimportFailedMessage", "Reimporting of DNA failed");
			UE_LOGFMT(LogDNAImporter, Error, "{}", *Message.ToString());
		}
	}
	else
	{
		FText DNAName = FText::FromString(ReimportDNA->GetName());
		const FText Message = FText::Format(LOCTEXT("DNA_ReimportErrorMessage", "DNA Asset or AssetImportData is missing for Reimport {0} "), DNAName);
		FNotificationInfo Info(Message);
		Info.ExpireDuration = 3.0f;
		Info.bUseLargeFont = false;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}

		UE_LOGFMT(LogDNAImporter, Error, "{}", *Message.ToString());
	}
}

void UDNAImporter::ExportDNAWithPrompt(UDNA* DNAToExport)
{
	if(!DNAToExport)
	{
		const FText Message = LOCTEXT("DNA_ExportFailedInvalidDNA", "Invalid DNA for export!");
		UE_LOGFMT(LogDNAImporter, Error, "{}", *Message.ToString());
		return;
	}

	IDesktopPlatform* DesktopPlatform =
		FDesktopPlatformModule::Get();

	if (DesktopPlatform)
	{
		FString ChosenFolder;
		const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
		const bool bPicked = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			TEXT("Choose Folder"),
			TEXT(""),
			ChosenFolder
		);

		if (bPicked)
		{
			ExportDNA(DNAToExport, ChosenFolder);
		}
	}
}

FString UDNAImporter::PromptForDNAImportFile()
{
	const FText PromptTitle = LOCTEXT("DNAPromptTitle", "Choose a file to import for DNA");

	FString ChosenFilename("");

	FString ExtensionStr;
	ExtensionStr += TEXT("All model files|*.dna|");
	ExtensionStr += TEXT("DNA files|*.dna|");
	ExtensionStr += TEXT("All files|*.*");

	// First, display the file open dialog for selecting the file.
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpen = false;
	if (DesktopPlatform)
	{
		bOpen = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			PromptTitle.ToString(),
			*FEditorDirectories::Get().GetLastDirectory(ELastDirectory::UNR),
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::None,
			OpenFilenames
		);
	}

	if (bOpen)
	{
		if (OpenFilenames.Num() == 1)
		{
			ChosenFilename = OpenFilenames[0];
			FEditorDirectories::Get().SetLastDirectory(ELastDirectory::UNR, FPaths::GetPath(ChosenFilename));
		}
	}

	return ChosenFilename;
}

#undef LOCTEXT_NAMESPACE
