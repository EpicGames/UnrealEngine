// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAImporterLibrary.h"
#include "DNAAssetImportFactory.h"
#include "DNAImporter.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DNAImporterLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogDNAImporterLibrary, Log, All);

#define LOCTEXT_NAMESPACE "RigLogicEditor"

void UDNAImporterLibrary::ImportSkeletalMeshDNA(const FString FileName, UObject* Mesh)
{
	// Clean the path if needed
	FString CleanedFileName = FileName.TrimStartAndEnd();
	FPaths::MakeStandardFilename(CleanedFileName);
	CleanedFileName = FPaths::ConvertRelativePathToFull(CleanedFileName);
          
	if(!IFileManager::Get().FileExists(*CleanedFileName))
	{
		UE_LOGF(LogDNAImporterLibrary, Error, "DNA File doesn't exist.");
		return;
	}

	TStrongObjectPtr<UDNAImporter> DNAImporter(NewObject<UDNAImporter>());

	if (!DNAImporter->ImportDNAAutomated(CleanedFileName, Cast<USkeletalMesh>(Mesh)))
	{
		UE_LOGF(LogDNAImporterLibrary, Error, "Importing of DNA failed.");
	}

}

void UDNAImporterLibrary::ImportAndAttachDNA(const FString FileName, USkeletalMesh* Mesh, bool bReplaceExisting)
{
	// Clean the path if needed
	FString CleanedFileName = FileName.TrimStartAndEnd();
	FPaths::MakeStandardFilename(CleanedFileName);
	CleanedFileName = FPaths::ConvertRelativePathToFull(CleanedFileName);

	if (!IFileManager::Get().FileExists(*CleanedFileName))
	{
		UE_LOGF(LogDNAImporterLibrary, Error, "DNA File doesn't exist.");
		return;
	}

	TStrongObjectPtr<UDNAImporter> DNAImporter(NewObject<UDNAImporter>());

	if (!DNAImporter->ImportDNAAutomated(CleanedFileName, Mesh, bReplaceExisting))
	{
		UE_LOGF(LogDNAImporterLibrary, Error, "Importing of DNA failed.");
	}
}

#undef LOCTEXT_NAMESPACE
