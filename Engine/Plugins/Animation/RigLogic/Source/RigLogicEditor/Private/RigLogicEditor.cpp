// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicEditor.h"
#include "Logging/LogMacros.h"
#include "RigUnit_RigLogic.h"
#include "EditorFramework/AssetImportData.h"
#include "IAssetTypeActions.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ISettingsModule.h"
#include "DNAAssetUserData.h"
#include "DNA.h"
#include "DNAAsset.h"
#include "DNAReader.h"
#include "CoordinateSystemCustomization.h"
#include "RigLogicPerPlatformCustomization.h"
#include "PropertyEditorModule.h"
#include "Misc/Paths.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/AssetRegistryTagsContext.h"

IMPLEMENT_MODULE(FRigLogicEditor, RigLogicEditor)

DEFINE_LOG_CATEGORY_STATIC(LogRigLogicEditor, Log, All);

#define LOCTEXT_NAMESPACE "RigLogicEditor"

void FRigLogicEditor::StartupModule()
{
	UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddStatic(&GetAssetRegistryTagsForDNA);

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(
		TEXT("CoordinateSystem"),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FCoordinateSystemCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomPropertyTypeLayout(
		TEXT("PerPlatformERigLogicCalculationType"),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerPlatformEnumCustomization::MakeInstanceCalculationType)
	);

	PropertyModule.RegisterCustomPropertyTypeLayout(
		TEXT("PerPlatformERigLogicFloatingPointType"),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPerPlatformEnumCustomization::MakeInstanceFloatingPointType)
	);
}

void FRigLogicEditor::GetAssetRegistryTagsForDNA(FAssetRegistryTagsContext Context)
{
	const USkeletalMesh* SkelMesh = Cast<USkeletalMesh>(Context.GetObject());
	if (SkelMesh)
	{
		FString DNAName;

		USkeletalMesh* MutableSkelMesh = const_cast<USkeletalMesh*>(SkelMesh);

		if (const UDNAAssetUserData* DNAAssetUserData = MutableSkelMesh->GetAssetUserData<UDNAAssetUserData>())
		{
			if (UDNA* DNA = DNAAssetUserData->DNAAsset)
			{
				DNAName = DNA->GetName();
			}
		}

		// Fallback to deprecated UDNAAsset if UDNA not found
		if (DNAName.IsEmpty())
		{
			if (UDNAAsset* DNAAsset = MutableSkelMesh->GetAssetUserData<UDNAAsset>())
			{
				TSharedPtr<IDNAReader> DNAReader = DNAAsset->GetDNAReader();
				if (DNAReader.IsValid())
				{
					DNAName = DNAReader->GetName();
				}
			}
		}

		if (DNAName.IsEmpty())
		{
			DNAName = TEXT("No DNA");
		}

		Context.AddTag(UObject::FAssetRegistryTag("DNA", DNAName, UObject::FAssetRegistryTag::TT_Alphabetical));
	}
}

void FRigLogicEditor::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("CoordinateSystem"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("PerPlatformERigLogicCalculationType"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("PerPlatformERigLogicFloatingPointType"));
	}
}


#undef LOCTEXT_NAMESPACE
