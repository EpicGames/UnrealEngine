// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicPerPlatformCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "PerPlatformPropertyCustomization.h"
#include "SEnumCombo.h"
#include "RigLogic.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "ScopedTransaction.h"
#include "Widgets/SNullWidget.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "RigLogicPerPlatformCustomization"

FPerPlatformEnumCustomization::FPerPlatformEnumCustomization(const UEnum* InEnum) : Enum(InEnum)
{
}

TSharedRef<IPropertyTypeCustomization> FPerPlatformEnumCustomization::MakeInstanceCalculationType()
{
	return MakeShareable(new FPerPlatformEnumCustomization(StaticEnum<ERigLogicCalculationType>()));
}

TSharedRef<IPropertyTypeCustomization> FPerPlatformEnumCustomization::MakeInstanceFloatingPointType()
{
	return MakeShareable(new FPerPlatformEnumCustomization(StaticEnum<ERigLogicFloatingPointType>()));
}

void FPerPlatformEnumCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

	TAttribute<TArray<FName>> PlatformOverrideNames = TAttribute<TArray<FName>>::Create(
		TAttribute<TArray<FName>>::FGetter::CreateSP(this, &FPerPlatformEnumCustomization::GetPlatformOverrideNames, StructPropertyHandle));

	FPerPlatformPropertyCustomNodeBuilderArgs Args;
	Args.FilterText = StructPropertyHandle->GetPropertyDisplayName();
	Args.OnGenerateNameWidget = FOnGetContent::CreateLambda([StructPropertyHandle]()
	{
		return StructPropertyHandle->CreatePropertyNameWidget();
	});
	Args.PlatformOverrideNames = PlatformOverrideNames;
	Args.OnAddPlatformOverride = FOnPlatformOverrideAction::CreateSP(this,
		&FPerPlatformEnumCustomization::AddPlatformOverride, StructPropertyHandle);
	Args.OnRemovePlatformOverride = FOnPlatformOverrideAction::CreateSP(this,
		&FPerPlatformEnumCustomization::RemovePlatformOverride, StructPropertyHandle);
	Args.OnGenerateWidgetForPlatformRow = FOnGenerateWidget::CreateLambda(
		[this, StructPropertyHandle](FName PlatformGroupName)
		{
			return MakeEnumWidget(PlatformGroupName, StructPropertyHandle);
		});
	Args.IsEnabled = TAttribute<bool>::CreateLambda([StructPropertyHandle]()
	{
		return StructPropertyHandle->IsEditable();
	});

	StructBuilder.AddCustomBuilder(MakeShared<FPerPlatformPropertyCustomNodeBuilder>(MoveTemp(Args)));
}

TSharedRef<SWidget> FPerPlatformEnumCustomization::MakeEnumWidget(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle) const
{
	// Find the right child property handle — Default or a specific PerPlatform map entry
	TSharedPtr<IPropertyHandle> EditProperty;

	if (PlatformGroupName == NAME_None)
	{
		EditProperty = StructPropertyHandle->GetChildHandle(FName("Default"));
	}
	else
	{
		TSharedPtr<IPropertyHandle> MapProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
		if (MapProperty.IsValid())
		{
			uint32 NumChildren = 0;
			MapProperty->GetNumChildren(NumChildren);
			for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ChildIdx++)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = MapProperty->GetChildHandle(ChildIdx);
				if (ChildProperty.IsValid())
				{
					TSharedPtr<IPropertyHandle> KeyProperty = ChildProperty->GetKeyHandle();
					if (KeyProperty.IsValid())
					{
						FName KeyName;
						if (KeyProperty->GetValue(KeyName) == FPropertyAccess::Success && KeyName == PlatformGroupName)
						{
							EditProperty = ChildProperty;
							break;
						}
					}
				}
			}
		}
	}

	if (!EditProperty.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	// Create an enum combo box wired to the int32 property handle
	TSharedPtr<IPropertyHandle> SharedEditProperty = EditProperty;

	return SNew(SEnumComboBox, Enum)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.CurrentValue_Lambda([SharedEditProperty]() -> int32
			{
				int32 Value = 0;
				if (SharedEditProperty.IsValid())
				{
					SharedEditProperty->GetValue(Value);
				}
				return Value;
			})
		.OnEnumSelectionChanged_Lambda([SharedEditProperty](int32 NewValue, ESelectInfo::Type)
			{
				if (SharedEditProperty.IsValid())
				{
					SharedEditProperty->SetValue(NewValue);
				}
			});
}

TArray<FName> FPerPlatformEnumCustomization::GetPlatformOverrideNames(TSharedRef<IPropertyHandle> StructPropertyHandle) const
{
	TArray<FName> PlatformOverrideNames;

	TSharedPtr<IPropertyHandle> MapProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
	if (MapProperty.IsValid())
	{
		TArray<const void*> RawData;
		MapProperty->AccessRawData(RawData);
		for (const void* Data : RawData)
		{
			const TMap<FName, int32>* PerPlatformMap = static_cast<const TMap<FName, int32>*>(Data);
			check(PerPlatformMap);
			TArray<FName> KeyArray;
			PerPlatformMap->GenerateKeyArray(KeyArray);
			for (FName PlatformName : KeyArray)
			{
				PlatformOverrideNames.AddUnique(PlatformName);
			}
		}
	}
	return PlatformOverrideNames;
}

bool FPerPlatformEnumCustomization::AddPlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	FScopedTransaction Transaction(LOCTEXT("AddPlatformOverride", "Add Platform Override"));

	TSharedPtr<IPropertyHandle> PerPlatformProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
	TSharedPtr<IPropertyHandle> DefaultProperty = StructPropertyHandle->GetChildHandle(FName("Default"));

	if (PerPlatformProperty.IsValid() && DefaultProperty.IsValid())
	{
		TSharedPtr<IPropertyHandleMap> MapProperty = PerPlatformProperty->AsMap();
		if (MapProperty.IsValid())
		{
			MapProperty->AddItem();
			uint32 NumChildren = 0;
			PerPlatformProperty->GetNumChildren(NumChildren);
			for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ChildIdx++)
			{
				TSharedPtr<IPropertyHandle> ChildProperty = PerPlatformProperty->GetChildHandle(ChildIdx);
				if (ChildProperty.IsValid())
				{
					TSharedPtr<IPropertyHandle> KeyProperty = ChildProperty->GetKeyHandle();
					if (KeyProperty.IsValid())
					{
						FName KeyName;
						if (KeyProperty->GetValue(KeyName) == FPropertyAccess::Success && KeyName == NAME_None)
						{
							KeyProperty->SetValue(PlatformGroupName);

							FString PropertyValueString;
							DefaultProperty->GetValueAsFormattedString(PropertyValueString);
							ChildProperty->SetValueFromFormattedString(PropertyValueString);
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

bool FPerPlatformEnumCustomization::RemovePlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	FScopedTransaction Transaction(LOCTEXT("RemovePlatformOverride", "Remove Platform Override"));

	TSharedPtr<IPropertyHandle> MapProperty = StructPropertyHandle->GetChildHandle(FName("PerPlatform"));
	if (MapProperty.IsValid())
	{
		TArray<const void*> RawData;
		MapProperty->AccessRawData(RawData);
		for (const void* Data : RawData)
		{
			TMap<FName, int32>* PerPlatformMap = const_cast<TMap<FName, int32>*>(static_cast<const TMap<FName, int32>*>(Data));
			check(PerPlatformMap);
			TArray<FName> KeyArray;
			PerPlatformMap->GenerateKeyArray(KeyArray);
			for (FName PlatformName : KeyArray)
			{
				if (PlatformName == PlatformGroupName)
				{
					PerPlatformMap->Remove(PlatformName);
					return true;
				}
			}
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
