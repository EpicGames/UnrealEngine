// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "PerPlatformPropertyCustomization.h"

class IDetailChildrenBuilder;
class IPropertyHandle;
class UEnum;

/**
 * Property customization for FPerPlatformInt-derived structs that represent enums.
 * Shows enum dropdowns instead of numeric spinners in the per-platform widget.
 */
class FPerPlatformEnumCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstanceCalculationType();
	static TSharedRef<IPropertyTypeCustomization> MakeInstanceFloatingPointType();

	FPerPlatformEnumCustomization(const UEnum* InEnum);

	// IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override
	{
	}

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& StructBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	TSharedRef<SWidget> MakeEnumWidget(FName PlatformGroupName,
		TSharedRef<IPropertyHandle> StructPropertyHandle) const;

	TArray<FName> GetPlatformOverrideNames(TSharedRef<IPropertyHandle> StructPropertyHandle) const;
	bool AddPlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle);
	bool RemovePlatformOverride(FName PlatformGroupName, TSharedRef<IPropertyHandle> StructPropertyHandle);

	const UEnum* Enum;
	TWeakPtr<IPropertyUtilities> PropertyUtilities;
};
