// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Commandlets/Commandlet.h"

#include "ConvertLegacyDNAAssetsCommandlet.generated.h"

/**
 * Editor commandlet that converts every USkeletalMesh carrying a legacy
 * UDNAAsset (UAssetUserData) into a standalone UDNA asset, using the engine's
 * existing UDNAImporter::ConvertFromLegacyAssetUserData (or, when
 * -ReimportFromSource is requested, UDNAImporterLibrary::ImportDNAAutomated).
 *
 * Pair this with the stock -run=ResavePackages to bump the on-disk
 * FDNAAssetCustomVersion across all DNA-bearing packages once conversion is
 * done.
 *
 * Usage:
 *   UnrealEditor-Cmd.exe <Project>.uproject -run=ConvertLegacyDNAAssets \
 *       [-PathFilter=/Game/MetaHumans] [-DryRun] [-NoSCC] \
 *       [-ReimportFromSource] [-Verbose] [-help]
 */
UCLASS()
class UConvertLegacyDNAAssetsCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	UConvertLegacyDNAAssetsCommandlet();

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
