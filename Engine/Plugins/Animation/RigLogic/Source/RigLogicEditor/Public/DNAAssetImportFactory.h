// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Factories/Factory.h"
#include "Factories/ImportSettings.h"
#include "EditorReimportHandler.h"
#include "DNAAsset.h"
#include "Engine/SkeletalMesh.h"
#include "DNAAssetImportFactory.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDNAImportFactory, Log, All);

/** Factory responsible for importing DNA file and attaching DNA data into SkeletalMesh
*	Also extends ReimportHandler for importing DNA file with the same name as SkeletalMesh
 */
UCLASS(transient)
class UDNAAssetImportFactory: public UFactory, public FReimportHandler
{ 
	GENERATED_UCLASS_BODY()

public:

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Import UI is not needed since it adds unnecessary complexity and therefor it's deprecated")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Import UI is not needed since it adds unnecessary complexity and therefor it's deprecated"))
	TObjectPtr<class UDEPRECATED_DNAAssetImportUI> ImportUI_DEPRECATED;

	UE_DEPRECATED(5.8, "Import UI is not needed since it adds unnecessary complexity and therefor it's deprecated")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Import UI is not needed since it adds unnecessary complexity and therefor it's deprecated"))
	TObjectPtr<class UDEPRECATED_DNAAssetImportUI> OriginalImportUI_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	/** UObject properties */
	virtual void PostInitProperties() override;

	//~ Begin FReimportHandler Interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual void PostImportCleanUp() { CleanUp(); }
	//~ End FReimportHandler Interface

	/** UFactory interface */
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual void CleanUp() override {};
};
