// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * DNA Asset Importer UI options.
 */

#pragma once

#include "CoreMinimal.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Factories/ImportSettings.h"
#include "DNAAssetImportUI.generated.h"

UCLASS(Deprecated, meta = (DeprecationMessage = "This class is not needed and will be removed."))
class UDEPRECATED_DNAAssetImportUI : public UObject, public IImportSettingsParser
{
	GENERATED_BODY()

public:

	/** Skeletal mesh to use for imported DNA asset. When importing DNA, leaving this as "None" will generate new skeletal mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Mesh, meta = (ImportType = "DNAAsset"))
	TObjectPtr<class USkeletalMesh> SkeletalMesh;

	UFUNCTION(BlueprintCallable, Category = Miscellaneous)
	void ResetToDefault() {};

	/** IImportSettings Interface */
	virtual void ParseFromJson(TSharedRef<class FJsonObject> ImportSettingsJson) override {};

	/* Whether this UI is construct for a reimport */
	bool bIsReimport;

	/* When we are reimporting, we need the current object to preview skeletal mesh match issues. */
	UObject* ReimportMesh;


	//////////////////////////////////////////////////////////////////////////
		// DNA Asset file informations
		// Transient value that are set everytime we show the options dialog. These are information only and should be string.

	/* The DNA file name */
	UPROPERTY(VisibleAnywhere, Transient, Category = DNAFileInformation, meta = (ImportType = "DNAAsset", DisplayName = "File Name"))
	FString FileName;
};
