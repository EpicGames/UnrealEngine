// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "DNAAssetImportUI.h"

class SButton;
//UE_DEPRECATED(5.8, "SDNAAssetImportWindow is deprecated and not needed anymore") [[deprecated("deprecated, no replacement")]]
class UE_DEPRECATED(5.8, "SDNAAssetImportWindow is deprecated and not needed anymore") SDNAAssetImportWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDNAAssetImportWindow)
		: _WidgetWindow()
		, _FullPath()
		, _MaxWindowHeight(0.0f)
		, _MaxWindowWidth(0.0f)
		{}

		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(FText, FullPath)
		SLATE_ARGUMENT(float, MaxWindowHeight)
		SLATE_ARGUMENT(float, MaxWindowWidth)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnImport()
	{
		bShouldImport = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnImportAll()
	{
		return OnImport();
	}

	FReply OnCancel()
	{
		bShouldImport = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldImport;
	}

	SDNAAssetImportWindow()
		: bShouldImport(false)
	{}

private:

	bool CanImport() const;
	FReply OnResetToDefaultClick() const;
	FText GetImportTypeDisplayText() const;

private:
	TSharedPtr<class IDetailsView> DetailsView;
	TWeakPtr< SWindow > WidgetWindow;
	TSharedPtr< SButton > ImportButton;
	bool			bShouldImport;
};
