// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "DNACommon.h"
#include "RigLogic.h"
#include "DNAImportSettings.generated.h"

/**
 * Project-level settings for DNA asset import defaults.
 * Allows teams to configure default MaxLOD and MinLOD per-platform values
 * that will be applied to every newly imported DNA asset.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, defaultconfig, meta=(DisplayName="RigLogic"))
class UDNAImportSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	RIGLOGICEDITOR_API UDNAImportSettings();

	virtual FName GetContainerName() const override { return "Project"; }
	virtual FName GetCategoryName() const override { return "Plugins"; }
	virtual FName GetSectionName() const override { return "RigLogic"; }

	/**
	 * Default LOD configuration applied to every newly imported DNA asset.
	 * Set MaxLODPerPlatform / MinLODPerPlatform here to pre-fill those fields
	 * for each platform (e.g., Android, PS5, XSX) without touching existing assets.
	 *
	 * Reimporting existing assets will preserve their current LOD settings.
	 * Only affects new imports going forward.
	 */
	UPROPERTY(EditAnywhere, config, Category="Import Defaults")
	FDNAConfig DefaultDNAConfig;

	/**
	 * Default RigLogic configuration applied to every newly imported DNA asset.
	 * Controls the per-platform calculation backend (Scalar, SSE, AVX, NEON),
	 * floating point precision, multi-threaded ML compute, and which rig systems
	 * are loaded (joints, blend shapes, animated maps, etc.).
	 *
	 * Reimporting existing assets will preserve their current RigLogic settings.
	 * Only affects new imports going forward.
	 */
	UPROPERTY(EditAnywhere, config, Category="Import Defaults")
	FRigLogicConfiguration DefaultRigLogicConfiguration;
};
