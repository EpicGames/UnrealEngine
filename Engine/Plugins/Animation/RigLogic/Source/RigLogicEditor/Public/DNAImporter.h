// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "DNAImporter.generated.h"

class USkeletalMesh;
class UDNA;

UCLASS(BlueprintType)
class UDNAImporter : public UObject
{
	GENERATED_BODY()

public:
	
	/** 
	* Importing DNA asset with prompt for choosing a DNA file from disk 
	*/
	UDNA* ImportDNAWithPrompt(TArray<USkeletalMesh*> SkeletalMeshes);

	/**
	* Importing DNA asset automatically from sent file and SkeletalMesh provided
	*/
	bool ImportDNAAutomated(const FString FileName, USkeletalMesh* SkeletalMesh, bool bReplaceExisting = true);

	/**
	* Creating a DNA Asset from legacy DNA Asset User Data that should exist on Skeletal Mesh
	*/
	bool ConvertFromLegacyAssetUserData(USkeletalMesh* SkeletalMesh);

	/**
	* Reimporting DNA asset provided
	*/
	void ReimportDNA(UDNA* ReimportDNA);

	/**
	* Export DNA asset to choosen folder
	*/
	void ExportDNAWithPrompt(UDNA* DNAToExport);

	void ExportDNA(UDNA* DNAToExport, const FString FolderToExportTo);

private:

	FString PromptForDNAImportFile();
	UDNA* ImportDNA(const FString Filename, USkeletalMesh* SkeletalMesh, bool bReplaceExisting = true);
};