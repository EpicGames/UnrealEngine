// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "DNAImporterLibrary.generated.h"

#define UE_API RIGLOGICEDITOR_API

/**
 * 
 */
UCLASS(MinimalAPI)
class UDNAImporterLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
	/** Import a DNA file onto the specified mesh asset.*/
	UE_DEPRECATED(5.8, "This function is deprecated, please use ImportAndAttachDNA.")
	UFUNCTION(CallInEditor, BlueprintCallable, meta = (DisplayName = "Import Skeletal Mesh DNA File", Keywords = "DNA Importer Skeletal Mesh"), Category = "DNA")
	static UE_API void ImportSkeletalMeshDNA(const FString FileName, UObject* Mesh);

	/** Import a DNA file onto the specified mesh asset.*/
	UFUNCTION(CallInEditor, BlueprintCallable, meta = (DisplayName = "Import Skeletal Mesh DNA File", Keywords = "DNA Importer Skeletal Mesh"), Category = "DNA")
	static UE_API void ImportAndAttachDNA(const FString FileName, USkeletalMesh* Mesh, bool bReplaceExisting);

	
};

#undef UE_API
