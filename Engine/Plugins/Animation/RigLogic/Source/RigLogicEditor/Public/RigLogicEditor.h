// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Modules/ModuleInterface.h"

#define UE_API RIGLOGICEDITOR_API

class FExtender;
class FMenuBuilder;
struct FAssetData;

class FRigLogicEditor : public IModuleInterface 
{
public:
	UE_API void StartupModule() override;
	UE_API void ShutdownModule() override;

private:
	TArray<TSharedRef<class IAssetTypeActions>> AssetTypeActions;

	static UE_API TSharedRef<FExtender> OnExtendSkelMeshWithDNASelectionMenu(const TArray<FAssetData>& SelectedAssets);
	static UE_API void CreateDnaActionsSubMenu(FMenuBuilder& MenuBuilder, const TArray<FAssetData> SelectedAssets);
	static UE_API void GetDNAMenu(FMenuBuilder& MenuBuilder, const TArray<FAssetData> SelectedAssets);

	static UE_API void ExecuteDNAImport(UObject* Mesh);
	static UE_API void ExecuteDNAReimport(UObject* Mesh);

	static UE_API void GetAssetRegistryTagsForDNA(FAssetRegistryTagsContext Context);
};

#undef UE_API
