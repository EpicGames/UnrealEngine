// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RigLogicEditor : ModuleRules
	{
		public RigLogicEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"DeveloperSettings",
					"CoreUObject",
					"Engine",
					"ControlRig",
					"UnrealEd",
					"EditorFramework",
					"MainFrame",
					"RigLogicModule",
					"RigLogicLib",
					"PropertyEditor",
					"SlateCore",
					"ApplicationCore",
					"ToolMenus",
					"ContentBrowser",
					"ContentBrowserData",
					"AssetDefinition",
					"Slate",
					"InputCore",
					"EditorWidgets",
					"DesktopPlatform",
					"AssetTools",
					"DetailCustomizations"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"SourceControl"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Settings",
					"UnrealEd"
				}
			);
		}
	}
}
