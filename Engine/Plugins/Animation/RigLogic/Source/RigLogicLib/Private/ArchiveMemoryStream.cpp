// Copyright Epic Games, Inc. All Rights Reserved.

#include "ArchiveMemoryStream.h"

#include "Math/NumericLimits.h"
#include "Serialization/Archive.h"

FArchiveMemoryStream::FArchiveMemoryStream(FArchive* Archive) :
	Archive{Archive},
	Origin{}
{
	check(Archive != nullptr);
	Origin = Archive->Tell();
}

void FArchiveMemoryStream::seek(std::uint64_t Position)
{
	ensure(Position <= static_cast<std::uint64_t>(TNumericLimits<int64>::Max()));
	Archive->Seek(Origin + static_cast<int64>(Position));
}

std::uint64_t FArchiveMemoryStream::tell()
{
	const int64 RelPosition = (Archive->Tell() - Origin);
	ensure(RelPosition >= 0);
	return static_cast<std::uint64_t>(RelPosition);
}

void FArchiveMemoryStream::open()
{
}

void FArchiveMemoryStream::close()
{
}

size_t FArchiveMemoryStream::read(char* ReadToBuffer, size_t Size)
{
	Archive->Serialize(ReadToBuffer, Size);
	return Size;
}

size_t FArchiveMemoryStream::read(Writable* Destination, size_t Size)
{
	constexpr size_t BufferSize = 4096ul;
	char Buffer[BufferSize];
	size_t SizeToRead = Size;
	while (SizeToRead > BufferSize) {
		Archive->Serialize(Buffer, BufferSize);
		Destination->write(Buffer, BufferSize);
		SizeToRead -= BufferSize;
	}
	Archive->Serialize(Buffer, SizeToRead);
	Destination->write(Buffer, SizeToRead);
	return Size;
}

size_t FArchiveMemoryStream::write(const char* WriteFromBuffer, size_t Size)
{
	Archive->Serialize(const_cast<char*>(WriteFromBuffer), Size);
	return Size;
}

size_t FArchiveMemoryStream::write(Readable* Source, size_t Size)
{
	constexpr size_t BufferSize = 4096ul;
	char Buffer[BufferSize];
	size_t SizeToWrite = Size;
	while (SizeToWrite > BufferSize) {
		Source->read(Buffer, BufferSize);
		Archive->Serialize(const_cast<char*>(Buffer), BufferSize);
		SizeToWrite -= BufferSize;
	}
	Source->read(Buffer, SizeToWrite);
	Archive->Serialize(const_cast<char*>(Buffer), SizeToWrite);
	return Size;
}

std::uint64_t FArchiveMemoryStream::size()
{
	const int64 TotalSize = Archive->GetArchiveState().TotalSize();
	ensure(TotalSize >= 0);
	const int64 StreamSize = TotalSize - Origin;
	ensure(StreamSize >= 0);
	return static_cast<std::uint64_t>(StreamSize);
}
