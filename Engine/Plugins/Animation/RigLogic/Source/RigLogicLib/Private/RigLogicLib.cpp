// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef RIGLOGIC_MODULE_DISCARD

#include "RigLogicLib.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FRigLogicLib"

DEFINE_LOG_CATEGORY(LogRigLogicLib);

void FRigLogicLib::StartupModule()
{
}

void FRigLogicLib::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRigLogicLib, RigLogicLib)

#endif  // RIGLOGIC_MODULE_DISCARD
