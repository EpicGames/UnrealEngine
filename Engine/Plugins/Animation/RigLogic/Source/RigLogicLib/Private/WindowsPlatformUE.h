// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Windows/WindowsHWrapper.h"
