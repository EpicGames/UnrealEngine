// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/DNA.h"
#include "dna/Helpers.h"
#include "dna/types/Aliases.h"

namespace dna {

class BaseImpl {
protected:
    explicit BaseImpl(MemoryResource* memRes_) :
        memRes{memRes_},
        dna{UnknownLayerPolicy::Preserve, UpgradeFormatPolicy::Allowed, memRes} {
    }

    BaseImpl(UnknownLayerPolicy unknownPolicy, UpgradeFormatPolicy upgradePolicy, MemoryResource* memRes_) :
        memRes{memRes_},
        dna{unknownPolicy, upgradePolicy, memRes} {
    }

    ~BaseImpl() = default;

    BaseImpl(const BaseImpl&) = delete;
    BaseImpl& operator=(const BaseImpl&) = delete;

    BaseImpl(BaseImpl&& rhs) = delete;
    BaseImpl& operator=(BaseImpl&&) = delete;

public:
    MemoryResource* getMemoryResource() {
        return memRes;
    }

    void rawCopyInto(DNA& destination, DataLayer layer, UnknownLayerPolicy policy, MemoryResource* memRes_) {
        copy(dna, destination, layer, policy, memRes_);
    }

protected:
    MemoryResource* memRes;
    mutable DNA dna;
};

}  // namespace dna
