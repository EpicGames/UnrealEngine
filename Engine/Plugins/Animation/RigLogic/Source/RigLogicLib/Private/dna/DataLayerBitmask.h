// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"
#include "dna/utils/ScopedEnumEx.h"

namespace dna {

enum class DataLayerBitmask {
    Descriptor = 1,
    Definition = 2,
    Behavior = 4,
    GeometryBlendShapesOnly = 8,
    GeometryRest = 16,
    MachineLearnedBehavior = 32,
    RBFBehavior = 64,
    JointBehaviorMetadata = 128,
    TwistSwingBehavior = 256
};

inline DataLayerBitmask computeDataLayerBitmask(DataLayer layer) {
    DataLayerBitmask result = DataLayerBitmask::Descriptor;
    if (layer == DataLayer::All) {
        result |= DataLayerBitmask::Definition;
        result |= DataLayerBitmask::Behavior;
        result |= DataLayerBitmask::GeometryBlendShapesOnly;
        result |= DataLayerBitmask::GeometryRest;
        result |= DataLayerBitmask::MachineLearnedBehavior;
        result |= DataLayerBitmask::RBFBehavior;
        result |= DataLayerBitmask::JointBehaviorMetadata;
        result |= DataLayerBitmask::TwistSwingBehavior;
        return result;
    }

    if (contains(layer, DataLayer::Definition)) {
        result |= DataLayerBitmask::Definition;
    }

    if (contains(layer, DataLayer::Behavior)) {
        result |= DataLayerBitmask::Behavior;
    }

    if (contains(layer, DataLayer::Geometry)) {
        result |= DataLayerBitmask::GeometryBlendShapesOnly;
        result |= DataLayerBitmask::GeometryRest;
    }

    if (contains(layer, DataLayer::GeometryWithoutBlendShapes)) {
        result |= DataLayerBitmask::GeometryRest;
    }

    if (contains(layer, DataLayer::MachineLearnedBehavior)) {
        result |= DataLayerBitmask::MachineLearnedBehavior;
    }

    if (contains(layer, DataLayer::RBFBehavior)) {
        result |= DataLayerBitmask::RBFBehavior;
    }

    if (contains(layer, DataLayer::JointBehaviorMetadata)) {
        result |= DataLayerBitmask::JointBehaviorMetadata;
    }

    if (contains(layer, DataLayer::TwistSwingBehavior)) {
        result |= DataLayerBitmask::TwistSwingBehavior;
    }

    return result;
}

}  // namespace dna
