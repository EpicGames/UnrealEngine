// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/DNA.h"
#include "dna/TypeDefs.h"
#include "dna/stream/FilteredBinaryInputArchive.h"

#include <terse/archives/binary/OutputArchive.h>

namespace dna {

inline void copy(DNA& source, DNA& destination, DataLayer layer, UnknownLayerPolicy policy, MemoryResource* memRes) {
    auto stream = makeScoped<trio::MemoryStream>(memRes);
    terse::BinaryOutputArchive<trio::MemoryStream> outArchive{stream.get()};
    // UpgradeFormatPolicy is used to prevent file format upgrades when a pure copy of the existing data is just needed.
    // The issue manifests itself when a hypothetical future DNA version is fed into the reader, containing some unkown layers,
    // but missing e.g. one of the layers known by the current DNA version (for some reason?).
    // As the file format version is higher than the file format known by the current DNA library, during the serialization
    // process of the copy operation (serialization into temporary memory stream), the serializer would add any layers that this
    // file might be missing (but seeing it's format version, it knows it's supposed to be able to handle them, since it's a
    // newer version). So inside the copy operation, some extra empty layers might creep into the data, although they were not
    // part of the source data, and the intention was to perform a copy operation only, without adding any new data.
    const auto oldUpgradePolicy = source.layers.upgradePolicy;
    source.layers.upgradePolicy = UpgradeFormatPolicy::Disallowed;
    outArchive << source;
    source.layers.upgradePolicy = oldUpgradePolicy;

    destination.layers.unknownPolicy = policy;
    stream->seek(0ul);
    const std::uint16_t maxLOD = static_cast<std::uint16_t>(0);
    const std::uint16_t minLOD = static_cast<std::uint16_t>(source.descriptor.lodCount > 0 ? source.descriptor.lodCount - 1 : 0);
    FilteredBinaryInputArchive inArchive{stream.get(), layer, LODConstraint(maxLOD, minLOD, memRes), memRes};
    inArchive >> destination;
}

}  // namespace dna
