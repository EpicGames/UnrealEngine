// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/LODConstraint.h"

#include "dna/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <cstdint>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dna {

LODConstraint::LODConstraint(std::uint16_t maxLOD, std::uint16_t minLOD, MemoryResource* memRes) :
    lods{memRes} {
    assert(maxLOD <= minLOD);
    lods.resize(static_cast<std::size_t>(minLOD - maxLOD) + 1ul);
    std::iota(lods.begin(), lods.end(), maxLOD);
}

LODConstraint::LODConstraint(ConstArrayView<std::uint16_t> lods_, MemoryResource* memRes) :
    lods{lods_.begin(), lods_.end(), memRes} {
    std::sort(lods.begin(), lods.end());
}

bool LODConstraint::hasImpactOn(std::uint16_t lodCount) const {
    std::uint16_t lod = {};
    for (auto it = lods.begin(); (it != lods.end()) && (lod < lodCount); ++it) {
        lod = static_cast<std::uint16_t>(lod + static_cast<std::uint16_t>(lod == *it));
    }
    return (lod != lodCount);
}

std::uint16_t LODConstraint::getMaxLOD() const {
    return (lods.empty() ? std::uint16_t{} : lods.front());
}

std::uint16_t LODConstraint::getMinLOD() const {
    return (lods.empty() ? std::uint16_t{} : lods.back());
}

std::uint16_t LODConstraint::getLODCount() const {
    return static_cast<std::uint16_t>(lods.size());
}

void LODConstraint::clampTo(std::uint16_t lodCount) {
    extd::filter(lods, [lodCount](std::uint16_t lod, std::size_t /*unused*/) { return lod < lodCount; });
}

void LODConstraint::applyTo(Vector<std::uint16_t>& unconstrainedLODs) const {
    extd::filter(unconstrainedLODs, extd::byPosition(lods));
}

void LODConstraint::applyTo(DynArray<std::uint16_t>& unconstrainedLODs) const {
    extd::filter(unconstrainedLODs, extd::byPosition(lods));
}

}  // namespace dna
