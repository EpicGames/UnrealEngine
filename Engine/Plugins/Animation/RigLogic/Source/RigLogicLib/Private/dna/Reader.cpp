// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/Reader.h"

namespace dna {

HeaderReader::~HeaderReader() = default;
DescriptorReader::~DescriptorReader() = default;
DefinitionReader::~DefinitionReader() = default;
BehaviorReader::~BehaviorReader() = default;
GeometryReader::~GeometryReader() = default;
MachineLearnedBehaviorReader::~MachineLearnedBehaviorReader() = default;
RBFBehaviorReader::~RBFBehaviorReader() = default;
TwistSwingBehaviorReader::~TwistSwingBehaviorReader() = default;
JointBehaviorMetadataReader::~JointBehaviorMetadataReader() = default;
MachineLearnedBehaviorExtReader::~MachineLearnedBehaviorExtReader() = default;
Reader::~Reader() = default;

}  // namespace dna
