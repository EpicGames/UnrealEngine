// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/Writer.h"

#include "dna/DataLayerBitmask.h"
#include "dna/Reader.h"
#include "dna/TypeDefs.h"
#include "dna/types/Vector3.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dna {

namespace {

using Vector3Vector = Vector<Vector3>;
using TextureCoordinateVector = Vector<TextureCoordinate>;
using VertexLayoutVector = Vector<VertexLayout>;

}  // namespace

HeaderWriter::~HeaderWriter() = default;
DescriptorWriter::~DescriptorWriter() = default;
DefinitionWriter::~DefinitionWriter() = default;
BehaviorWriter::~BehaviorWriter() = default;
GeometryWriter::~GeometryWriter() = default;
MachineLearnedBehaviorWriter::~MachineLearnedBehaviorWriter() = default;
RBFBehaviorWriter::~RBFBehaviorWriter() = default;
TwistSwingBehaviorWriter::~TwistSwingBehaviorWriter() = default;
JointBehaviorMetadataWriter::~JointBehaviorMetadataWriter() = default;
MachineLearnedBehaviorExtWriter::~MachineLearnedBehaviorExtWriter() = default;
Writer::~Writer() = default;

template<typename TVector, typename TGetter>
static TVector collect_n(std::size_t count, TGetter getter, MemoryResource* memRes) {
    TVector retval{memRes};
    retval.reserve(count);
    for (std::size_t i = 0ul; i < count; ++i) {
        retval.push_back(getter(i));
    }
    return retval;
}

static void copyHeader(const HeaderReader* source, HeaderWriter* destination, MemoryResource* /*unused*/) {
    destination->setFileFormatGeneration(source->getFileFormatGeneration());
    destination->setFileFormatVersion(source->getFileFormatVersion());
}

static void copyDescriptor(const DescriptorReader* source, DescriptorWriter* destination, MemoryResource* /*unused*/) {
    destination->setName(source->getName());
    destination->setArchetype(source->getArchetype());
    destination->setGender(source->getGender());
    destination->setAge(source->getAge());
    destination->clearMetaData();
    for (std::uint32_t i = {}; i < source->getMetaDataCount(); ++i) {
        const auto key = source->getMetaDataKey(i);
        const auto value = source->getMetaDataValue(key);
        destination->setMetaData(key, value);
    }
    destination->setTranslationUnit(source->getTranslationUnit());
    destination->setRotationUnit(source->getRotationUnit());
    destination->setCoordinateSystem(source->getCoordinateSystem());
    destination->setRotationSequence(source->getRotationSequence());
    destination->setRotationSign(source->getRotationSign());
    destination->setFaceWindingOrder(source->getFaceWindingOrder());
    destination->setLODCount(source->getLODCount());
    destination->setDBMaxLOD(source->getDBMaxLOD());
    destination->setDBComplexity(source->getDBComplexity());
    destination->setDBName(source->getDBName());
}

static std::pair<bool, std::uint16_t> findIndices(const Matrix<std::uint16_t>& source, ConstArrayView<std::uint16_t> indices) {
    // In the common scenario each LOD has it's unique set of indices
    for (std::size_t i = 0ul; i < source.size(); ++i) {
        ConstArrayView<std::uint16_t> candidate{source[i].data(), source[i].size()};
        if (indices == candidate) {
            // Unless the indices are the same between multiple LODs, in which case use the
            // already registered index
            return {true, static_cast<std::uint16_t>(i)};
        }
    }
    return {false, static_cast<std::uint16_t>(0)};
}

using IndicesGetter = std::function<ConstArrayView<std::uint16_t>(std::uint16_t)>;
using IndicesSetter = std::function<void(std::uint16_t, const std::uint16_t*, std::uint16_t)>;
using LODMappingSetter = std::function<void(std::uint16_t, std::uint16_t)>;

static void copyLODIndices(IndicesGetter getIndices,
                           IndicesSetter setIndices,
                           LODMappingSetter setLODMapping,
                           std::uint16_t lodCount,
                           MemoryResource* memRes) {
    Matrix<std::uint16_t> allIndices{memRes};
    std::uint16_t index = {};
    for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
        auto indices = getIndices(lod);
        // Check if these same indices were perhaps already used for previous LODs
        auto found = findIndices(allIndices, indices);
        if (!found.first) {
            setIndices(index, indices.data(), static_cast<std::uint16_t>(indices.size()));
            setLODMapping(lod, index);
            allIndices.emplace_back(indices.begin(), indices.end());
            ++index;
        } else {
            // Already used so do not replicate the same data twice
            setLODMapping(lod, found.second);
        }
    }
}

static void copyDefinition(const DefinitionReader* source, DefinitionWriter* destination, MemoryResource* memRes) {
    destination->clearGUIControlNames();
    destination->clearRawControlNames();
    destination->clearJointNames();
    destination->clearBlendShapeChannelNames();
    destination->clearAnimatedMapNames();
    destination->clearMeshNames();
    destination->clearJointIndices();
    destination->clearLODJointMappings();
    destination->clearBlendShapeChannelIndices();
    destination->clearLODBlendShapeChannelMappings();
    destination->clearAnimatedMapIndices();
    destination->clearLODAnimatedMapMappings();
    destination->clearMeshIndices();
    destination->clearLODMeshMappings();
    destination->clearMeshBlendShapeChannelMappings();

    const auto lodCount = source->getLODCount();

    for (std::uint16_t i = source->getGUIControlCount(); i > 0u; --i) {
        const auto idx = static_cast<std::uint16_t>(i - 1u);
        destination->setGUIControlName(idx, source->getGUIControlName(idx).data());
    }
    for (std::uint16_t i = source->getRawControlCount(); i > 0u; --i) {
        const auto idx = static_cast<std::uint16_t>(i - 1u);
        destination->setRawControlName(idx, source->getRawControlName(idx).data());
    }
    for (std::uint16_t i = source->getJointCount(); i > 0u; --i) {
        const auto idx = static_cast<std::uint16_t>(i - 1u);
        destination->setJointName(idx, source->getJointName(idx).data());
    }
    for (std::uint16_t i = source->getBlendShapeChannelCount(); i > 0u; --i) {
        const auto idx = static_cast<std::uint16_t>(i - 1u);
        destination->setBlendShapeChannelName(idx, source->getBlendShapeChannelName(idx).data());
    }
    for (std::uint16_t i = source->getAnimatedMapCount(); i > 0u; --i) {
        const auto idx = static_cast<std::uint16_t>(i - 1u);
        destination->setAnimatedMapName(idx, source->getAnimatedMapName(idx));
    }
    for (std::uint16_t i = source->getMeshCount(); i > 0u; --i) {
        const auto idx = static_cast<std::uint16_t>(i - 1u);
        destination->setMeshName(idx, source->getMeshName(idx).data());
    }

    using namespace std::placeholders;
    copyLODIndices(std::bind(&DefinitionReader::getJointIndicesForLOD, source, _1),
                   std::bind(&DefinitionWriter::setJointIndices, destination, _1, _2, _3),
                   std::bind(&DefinitionWriter::setLODJointMapping, destination, _1, _2),
                   lodCount,
                   memRes);
    copyLODIndices(std::bind(&DefinitionReader::getBlendShapeChannelIndicesForLOD, source, _1),
                   std::bind(&DefinitionWriter::setBlendShapeChannelIndices, destination, _1, _2, _3),
                   std::bind(&DefinitionWriter::setLODBlendShapeChannelMapping, destination, _1, _2),
                   lodCount,
                   memRes);
    copyLODIndices(std::bind(&DefinitionReader::getAnimatedMapIndicesForLOD, source, _1),
                   std::bind(&DefinitionWriter::setAnimatedMapIndices, destination, _1, _2, _3),
                   std::bind(&DefinitionWriter::setLODAnimatedMapMapping, destination, _1, _2),
                   lodCount,
                   memRes);
    copyLODIndices(std::bind(&DefinitionReader::getMeshIndicesForLOD, source, _1),
                   std::bind(&DefinitionWriter::setMeshIndices, destination, _1, _2, _3),
                   std::bind(&DefinitionWriter::setLODMeshMapping, destination, _1, _2),
                   lodCount,
                   memRes);

    Vector<std::uint16_t> jointHierarchy{memRes};
    jointHierarchy.reserve(source->getJointCount());
    for (std::uint16_t i = {}; i < source->getJointCount(); ++i) {
        jointHierarchy.push_back(source->getJointParentIndex(i));
    }
    destination->setJointHierarchy(jointHierarchy.data(), static_cast<std::uint16_t>(jointHierarchy.size()));

    for (std::uint16_t i = {}; i < source->getMeshBlendShapeChannelMappingCount(); ++i) {
        auto mapping = source->getMeshBlendShapeChannelMapping(i);
        destination->setMeshBlendShapeChannelMapping(i, mapping.meshIndex, mapping.blendShapeChannelIndex);
    }

    auto jointCount = source->getJointCount();

    auto translations = collect_n<Vector3Vector>(
        jointCount,
        [source](std::size_t index) { return source->getNeutralJointTranslation(static_cast<std::uint16_t>(index)); },
        memRes);
    destination->setNeutralJointTranslations(translations.data(), static_cast<std::uint16_t>(translations.size()));

    auto rotations = collect_n<Vector3Vector>(
        jointCount,
        [source](std::size_t index) { return source->getNeutralJointRotation(static_cast<std::uint16_t>(index)); },
        memRes);
    destination->setNeutralJointRotations(rotations.data(), static_cast<std::uint16_t>(rotations.size()));
}

static void copyBehavior(const BehaviorReader* source, BehaviorWriter* destination, MemoryResource* /*unused*/) {
    destination->clearJointGroups();

    auto guiToRawInputIndices = source->getGUIToRawInputIndices();
    destination->setGUIToRawInputIndices(guiToRawInputIndices.data(), static_cast<std::uint16_t>(guiToRawInputIndices.size()));

    auto guiToRawOutputIndices = source->getGUIToRawOutputIndices();
    destination->setGUIToRawOutputIndices(guiToRawOutputIndices.data(), static_cast<std::uint16_t>(guiToRawOutputIndices.size()));

    auto guiToRawFromValues = source->getGUIToRawFromValues();
    destination->setGUIToRawFromValues(guiToRawFromValues.data(), static_cast<std::uint16_t>(guiToRawFromValues.size()));

    auto guiToRawToValues = source->getGUIToRawToValues();
    destination->setGUIToRawToValues(guiToRawToValues.data(), static_cast<std::uint16_t>(guiToRawToValues.size()));

    auto guiToRawSlopeValues = source->getGUIToRawSlopeValues();
    destination->setGUIToRawSlopeValues(guiToRawSlopeValues.data(), static_cast<std::uint16_t>(guiToRawSlopeValues.size()));

    auto guiToRawCutValues = source->getGUIToRawCutValues();
    destination->setGUIToRawCutValues(guiToRawCutValues.data(), static_cast<std::uint16_t>(guiToRawCutValues.size()));

    destination->setPSDCount(source->getPSDCount());

    auto psdRowIndices = source->getPSDRowIndices();
    destination->setPSDRowIndices(psdRowIndices.data(), static_cast<std::uint16_t>(psdRowIndices.size()));

    auto psdColumnIndices = source->getPSDColumnIndices();
    destination->setPSDColumnIndices(psdColumnIndices.data(), static_cast<std::uint16_t>(psdColumnIndices.size()));

    auto psdValues = source->getPSDValues();
    destination->setPSDValues(psdValues.data(), static_cast<std::uint16_t>(psdValues.size()));

    destination->setJointRowCount(source->getJointRowCount());
    destination->setJointColumnCount(source->getJointColumnCount());

    for (std::uint16_t jointGroupIndexPlusOne = source->getJointGroupCount(); jointGroupIndexPlusOne > 0u;
         --jointGroupIndexPlusOne) {
        const auto jointGroupIndex = static_cast<std::uint16_t>(jointGroupIndexPlusOne - 1u);

        auto jointGroupLODs = source->getJointGroupLODs(jointGroupIndex);
        destination->setJointGroupLODs(jointGroupIndex, jointGroupLODs.data(), static_cast<std::uint16_t>(jointGroupLODs.size()));

        auto jointGroupInputIndices = source->getJointGroupInputIndices(jointGroupIndex);
        destination->setJointGroupInputIndices(jointGroupIndex,
                                               jointGroupInputIndices.data(),
                                               static_cast<std::uint16_t>(jointGroupInputIndices.size()));

        auto jointGroupOutputIndices = source->getJointGroupOutputIndices(jointGroupIndex);
        destination->setJointGroupOutputIndices(jointGroupIndex,
                                                jointGroupOutputIndices.data(),
                                                static_cast<std::uint16_t>(jointGroupOutputIndices.size()));

        auto jointGroupValues = source->getJointGroupValues(jointGroupIndex);
        destination->setJointGroupValues(jointGroupIndex,
                                         jointGroupValues.data(),
                                         static_cast<std::uint32_t>(jointGroupValues.size()));

        auto jointGroupJointIndices = source->getJointGroupJointIndices(jointGroupIndex);
        destination->setJointGroupJointIndices(jointGroupIndex,
                                               jointGroupJointIndices.data(),
                                               static_cast<std::uint16_t>(jointGroupJointIndices.size()));
    }

    auto blendShapeLODs = source->getBlendShapeChannelLODs();
    destination->setBlendShapeChannelLODs(blendShapeLODs.data(), static_cast<std::uint16_t>(blendShapeLODs.size()));

    auto blendShapeInputIndices = source->getBlendShapeChannelInputIndices();
    destination->setBlendShapeChannelInputIndices(blendShapeInputIndices.data(),
                                                  static_cast<std::uint16_t>(blendShapeInputIndices.size()));

    auto blendShapeOutputIndices = source->getBlendShapeChannelOutputIndices();
    destination->setBlendShapeChannelOutputIndices(blendShapeOutputIndices.data(),
                                                   static_cast<std::uint16_t>(blendShapeOutputIndices.size()));

    auto animatedMapLODs = source->getAnimatedMapLODs();
    destination->setAnimatedMapLODs(animatedMapLODs.data(), static_cast<std::uint16_t>(animatedMapLODs.size()));

    auto animatedMapInputIndices = source->getAnimatedMapInputIndices();
    destination->setAnimatedMapInputIndices(animatedMapInputIndices.data(),
                                            static_cast<std::uint16_t>(animatedMapInputIndices.size()));

    auto animatedMapOutputIndices = source->getAnimatedMapOutputIndices();
    destination->setAnimatedMapOutputIndices(animatedMapOutputIndices.data(),
                                             static_cast<std::uint16_t>(animatedMapOutputIndices.size()));

    auto animatedMapFromValues = source->getAnimatedMapFromValues();
    destination->setAnimatedMapFromValues(animatedMapFromValues.data(), static_cast<std::uint16_t>(animatedMapFromValues.size()));

    auto animatedMapToValues = source->getAnimatedMapToValues();
    destination->setAnimatedMapToValues(animatedMapToValues.data(), static_cast<std::uint16_t>(animatedMapToValues.size()));

    auto animatedMapSlopeValues = source->getAnimatedMapSlopeValues();
    destination->setAnimatedMapSlopeValues(animatedMapSlopeValues.data(),
                                           static_cast<std::uint16_t>(animatedMapSlopeValues.size()));

    auto animatedMapCutValues = source->getAnimatedMapCutValues();
    destination->setAnimatedMapCutValues(animatedMapCutValues.data(), static_cast<std::uint16_t>(animatedMapCutValues.size()));
}

static bool hasGeometry(const GeometryReader* source) {
    // Heuristic for determining whether source DNA actually has any geometry data, or mesh count is non-zero only
    // because of mesh names stored in definition layer
    std::uint32_t totalVertexCount = {};
    std::uint32_t totalVertexNormalCount = {};
    std::uint32_t totalTextureCoordCount = {};
    std::uint32_t totalVertexLayoutCount = {};
    std::uint32_t totalSkinWeightCount = {};
    for (std::uint16_t meshIndex = {}; meshIndex < source->getMeshCount(); ++meshIndex) {
        totalVertexCount += source->getVertexPositionCount(meshIndex);
        totalVertexNormalCount += source->getVertexNormalCount(meshIndex);
        totalTextureCoordCount += source->getVertexTextureCoordinateCount(meshIndex);
        totalVertexLayoutCount += source->getVertexLayoutCount(meshIndex);
        totalSkinWeightCount += source->getSkinWeightsCount(meshIndex);
    }
    return ((totalVertexCount != 0u) || (totalVertexNormalCount != 0u) || (totalTextureCoordCount != 0u) ||
            (totalVertexLayoutCount != 0u) || (totalSkinWeightCount != 0u));
}

static void copyGeometry(const GeometryReader* source, GeometryWriter* destination, MemoryResource* memRes) {
    destination->clearMeshes();

    if (!hasGeometry(source)) {
        // Source DNA was loaded without geometry layer
        return;
    }

    for (std::uint16_t meshIndexPlusOne = source->getMeshCount(); meshIndexPlusOne > 0u; --meshIndexPlusOne) {
        const auto meshIndex = static_cast<std::uint16_t>(meshIndexPlusOne - 1u);
        auto vertexCount = source->getVertexPositionCount(meshIndex);
        auto positions = collect_n<Vector3Vector>(
            vertexCount,
            [source, meshIndex](std::size_t index) {
                return source->getVertexPosition(meshIndex, static_cast<std::uint32_t>(index));
            },
            memRes);
        destination->setVertexPositions(meshIndex, positions.data(), static_cast<std::uint32_t>(positions.size()));

        auto textureCoordinateCount = source->getVertexTextureCoordinateCount(meshIndex);
        auto textureCoordinates = collect_n<TextureCoordinateVector>(
            textureCoordinateCount,
            [source, meshIndex](std::size_t index) {
                return source->getVertexTextureCoordinate(meshIndex, static_cast<std::uint32_t>(index));
            },
            memRes);
        destination->setVertexTextureCoordinates(meshIndex,
                                                 textureCoordinates.data(),
                                                 static_cast<std::uint32_t>(textureCoordinates.size()));

        auto normalCount = source->getVertexNormalCount(meshIndex);
        auto normals = collect_n<Vector3Vector>(
            normalCount,
            [source, meshIndex](std::size_t index) {
                return source->getVertexNormal(meshIndex, static_cast<std::uint32_t>(index));
            },
            memRes);
        destination->setVertexNormals(meshIndex, normals.data(), static_cast<std::uint32_t>(normals.size()));

        auto layoutCount = source->getVertexLayoutCount(meshIndex);
        auto layouts = collect_n<VertexLayoutVector>(
            layoutCount,
            [source, meshIndex](std::size_t index) {
                return source->getVertexLayout(meshIndex, static_cast<std::uint32_t>(index));
            },
            memRes);
        destination->setVertexLayouts(meshIndex, layouts.data(), static_cast<std::uint32_t>(layouts.size()));

        for (std::uint32_t faceIndexPlusOne = source->getFaceCount(meshIndex); faceIndexPlusOne > 0u; --faceIndexPlusOne) {
            const auto faceIndex = faceIndexPlusOne - 1u;
            auto faceVertices = source->getFaceVertexLayoutIndices(meshIndex, faceIndex);
            destination->setFaceVertexLayoutIndices(meshIndex,
                                                    faceIndex,
                                                    faceVertices.data(),
                                                    static_cast<std::uint32_t>(faceVertices.size()));
        }

        destination->setMaximumInfluencePerVertex(meshIndex, source->getMaximumInfluencePerVertex(meshIndex));

        const auto skinWeightsCount = source->getSkinWeightsCount(meshIndex);
        for (std::uint32_t skinWeightsIndexPlusOne = skinWeightsCount; skinWeightsIndexPlusOne > 0u; --skinWeightsIndexPlusOne) {
            const auto skinWeightsIndex = skinWeightsIndexPlusOne - 1u;
            auto skinWeights = source->getSkinWeightsValues(meshIndex, skinWeightsIndex);
            destination->setSkinWeightsValues(meshIndex,
                                              skinWeightsIndex,
                                              skinWeights.data(),
                                              static_cast<std::uint16_t>(skinWeights.size()));

            auto skinWeightsJoints = source->getSkinWeightsJointIndices(meshIndex, skinWeightsIndex);
            destination->setSkinWeightsJointIndices(meshIndex,
                                                    skinWeightsIndex,
                                                    skinWeightsJoints.data(),
                                                    static_cast<std::uint16_t>(skinWeightsJoints.size()));
        }
    }
}

static bool hasBlendShapeTargets(const GeometryReader* source) {
    // Heuristic for determining whether source DNA actually has any blend shape target data, or mesh count is non-zero only
    // because of mesh names stored in definition layer
    std::uint32_t totalBlendShapeTargetCount = {};
    for (std::uint16_t meshIndex = {}; meshIndex < source->getMeshCount(); ++meshIndex) {
        totalBlendShapeTargetCount += source->getBlendShapeTargetCount(meshIndex);
    }
    return (totalBlendShapeTargetCount != 0u);
}

static void copyBlendShapeTargets(const GeometryReader* source, GeometryWriter* destination, MemoryResource* memRes) {
    if (!hasBlendShapeTargets(source)) {
        // Source DNA was loaded without blend shape targets
        return;
    }

    for (std::uint16_t meshIndexPlusOne = source->getMeshCount(); meshIndexPlusOne > 0u; --meshIndexPlusOne) {
        const auto meshIndex = static_cast<std::uint16_t>(meshIndexPlusOne - 1u);
        for (std::uint16_t blendShapeTargetIndexPlusOne = source->getBlendShapeTargetCount(meshIndex);
             blendShapeTargetIndexPlusOne > 0u;
             --blendShapeTargetIndexPlusOne) {
            const auto blendShapeTargetIndex = static_cast<std::uint16_t>(blendShapeTargetIndexPlusOne - 1u);
            auto channelIndex = source->getBlendShapeChannelIndex(meshIndex, blendShapeTargetIndex);
            destination->setBlendShapeChannelIndex(meshIndex, blendShapeTargetIndex, channelIndex);
            auto deltaCount = source->getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex);
            auto deltas = collect_n<Vector3Vector>(
                deltaCount,
                [source, meshIndex, blendShapeTargetIndex](std::size_t index) {
                    return source->getBlendShapeTargetDelta(meshIndex, blendShapeTargetIndex, static_cast<std::uint32_t>(index));
                },
                memRes);
            destination->setBlendShapeTargetDeltas(meshIndex,
                                                   blendShapeTargetIndex,
                                                   deltas.data(),
                                                   static_cast<std::uint32_t>(deltas.size()));

            auto deltasVertices = source->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex);
            destination->setBlendShapeTargetVertexIndices(meshIndex,
                                                          blendShapeTargetIndex,
                                                          deltasVertices.data(),
                                                          static_cast<std::uint32_t>(deltasVertices.size()));
        }
    }
}

static bool hasMachineLearnedBehavior(const MachineLearnedBehaviorReader* source) {
    // Heuristic for determining whether source DNA actually has any machine learned behavior data
    return (source->getMLControlCount() != 0u) || (source->getNeuralNetworkCount() != 0u);
}

static void copyMachineLearnedBehavior(const MachineLearnedBehaviorReader* source,
                                       MachineLearnedBehaviorWriter* destination,
                                       MemoryResource* memRes) {
    destination->clearMLControlNames();
    destination->clearMeshRegionNames();
    destination->clearNeuralNetworks();
    destination->clearNeuralNetworkIndices();
    destination->clearNeuralNetworkIndicesPerMeshRegion();

    if (!hasMachineLearnedBehavior(source)) {
        // Source DNA was loaded without machine learned behavior layer
        return;
    }

    for (std::uint16_t i = source->getMLControlCount(); i > 0u; --i) {
        const auto idx = static_cast<std::uint16_t>(i - 1u);
        destination->setMLControlName(idx, source->getMLControlName(idx).data());
    }

    using namespace std::placeholders;
    copyLODIndices(std::bind(&MachineLearnedBehaviorReader::getNeuralNetworkIndicesForLOD, source, _1),
                   std::bind(&MachineLearnedBehaviorWriter::setNeuralNetworkIndices, destination, _1, _2, _3),
                   std::bind(&MachineLearnedBehaviorWriter::setLODNeuralNetworkMapping, destination, _1, _2),
                   source->getLODCount(),
                   memRes);

    for (std::uint16_t meshIndexPlusOne = source->getMeshCount(); meshIndexPlusOne > 0u; --meshIndexPlusOne) {
        const auto meshIndex = static_cast<std::uint16_t>(meshIndexPlusOne - 1u);
        for (std::uint16_t regionIndexPlusOne = source->getMeshRegionCount(meshIndex); regionIndexPlusOne > 0u;
             --regionIndexPlusOne) {
            const auto regionIndex = static_cast<std::uint16_t>(regionIndexPlusOne - 1u);
            destination->setMeshRegionName(meshIndex, regionIndex, source->getMeshRegionName(meshIndex, regionIndex));
            const auto netIndices = source->getNeuralNetworkIndicesForMeshRegion(meshIndex, regionIndex);
            destination->setNeuralNetworkIndicesForMeshRegion(meshIndex,
                                                              regionIndex,
                                                              netIndices.data(),
                                                              static_cast<std::uint16_t>(netIndices.size()));
        }
    }

    for (std::uint16_t netIndexPlusOne = source->getNeuralNetworkCount(); netIndexPlusOne > 0u; --netIndexPlusOne) {
        const auto netIndex = static_cast<std::uint16_t>(netIndexPlusOne - 1u);
        const auto inputIndices = source->getNeuralNetworkInputIndices(netIndex);
        destination->setNeuralNetworkInputIndices(netIndex, inputIndices.data(), static_cast<std::uint16_t>(inputIndices.size()));

        const auto outputIndices = source->getNeuralNetworkOutputIndices(netIndex);
        destination->setNeuralNetworkOutputIndices(netIndex,
                                                   outputIndices.data(),
                                                   static_cast<std::uint16_t>(outputIndices.size()));

        const auto layerCount = source->getNeuralNetworkLayerCount(netIndex);
        for (std::uint16_t layerIndexPlusOne = layerCount; layerIndexPlusOne > 0u; --layerIndexPlusOne) {
            const auto layerIndex = static_cast<std::uint16_t>(layerIndexPlusOne - 1u);

            const auto activationFunction = source->getNeuralNetworkLayerActivationFunction(netIndex, layerIndex);
            destination->setNeuralNetworkLayerActivationFunction(netIndex, layerIndex, activationFunction);

            const auto activationFunctionParams = source->getNeuralNetworkLayerActivationFunctionParameters(netIndex, layerIndex);
            destination->setNeuralNetworkLayerActivationFunctionParameters(
                netIndex,
                layerIndex,
                activationFunctionParams.data(),
                static_cast<std::uint16_t>(activationFunctionParams.size()));

            const auto biases = source->getNeuralNetworkLayerBiases(netIndex, layerIndex);
            destination->setNeuralNetworkLayerBiases(netIndex,
                                                     layerIndex,
                                                     biases.data(),
                                                     static_cast<std::uint32_t>(biases.size()));

            const auto weights = source->getNeuralNetworkLayerWeights(netIndex, layerIndex);
            destination->setNeuralNetworkLayerWeights(netIndex,
                                                      layerIndex,
                                                      weights.data(),
                                                      static_cast<std::uint32_t>(weights.size()));
        }
    }
}

static bool hasRBFBehavior(const RBFBehaviorReader* source) {
    // Heuristic for determining whether source DNA actually has any RBF behavior data
    return (source->getRBFPoseCount() != 0u) || (source->getRBFSolverCount() != 0u);
}

static void copyRBFBehavior(const RBFBehaviorReader* source, RBFBehaviorWriter* destination, MemoryResource* memRes) {
    destination->clearRBFPoses();
    destination->clearRBFSolverIndices();
    destination->clearRBFSolvers();

    if (!hasRBFBehavior(source)) {
        // Source DNA was loaded without RBF behavior layer
        return;
    }

    using namespace std::placeholders;
    copyLODIndices(std::bind(&RBFBehaviorReader::getRBFSolverIndicesForLOD, source, _1),
                   std::bind(&RBFBehaviorWriter::setRBFSolverIndices, destination, _1, _2, _3),
                   std::bind(&RBFBehaviorWriter::setLODRBFSolverMapping, destination, _1, _2),
                   source->getLODCount(),
                   memRes);

    for (std::uint16_t pciPlusOne = source->getRBFPoseControlCount(); pciPlusOne > 0u; --pciPlusOne) {
        const auto pci = static_cast<std::uint16_t>(pciPlusOne - 1u);
        destination->setRBFPoseControlName(pci, source->getRBFPoseControlName(pci).data());
    }

    for (std::uint16_t posePlusOne = source->getRBFPoseCount(); posePlusOne > 0u; --posePlusOne) {
        const auto pi = static_cast<std::uint16_t>(posePlusOne - 1u);
        destination->setRBFPoseName(pi, source->getRBFPoseName(pi).data());
        destination->setRBFPoseScale(pi, source->getRBFPoseScale(pi));
        const auto inputControlIndices = source->getRBFPoseInputControlIndices(pi);
        destination->setRBFPoseInputControlIndices(pi,
                                                   inputControlIndices.data(),
                                                   static_cast<std::uint16_t>(inputControlIndices.size()));
        const auto outputControlIndices = source->getRBFPoseOutputControlIndices(pi);
        destination->setRBFPoseOutputControlIndices(pi,
                                                    outputControlIndices.data(),
                                                    static_cast<std::uint16_t>(outputControlIndices.size()));
        const auto outputControlWeights = source->getRBFPoseOutputControlWeights(pi);
        destination->setRBFPoseOutputControlWeights(pi,
                                                    outputControlWeights.data(),
                                                    static_cast<std::uint16_t>(outputControlWeights.size()));
    }

    for (std::uint16_t solverPlusOne = source->getRBFSolverCount(); solverPlusOne > 0u; --solverPlusOne) {
        const auto si = static_cast<std::uint16_t>(solverPlusOne - 1u);
        destination->setRBFSolverName(si, source->getRBFSolverName(si).data());
        const auto rawControlIndices = source->getRBFSolverRawControlIndices(si);
        destination->setRBFSolverRawControlIndices(si,
                                                   rawControlIndices.data(),
                                                   static_cast<std::uint16_t>(rawControlIndices.size()));
        const auto poseIndices = source->getRBFSolverPoseIndices(si);
        destination->setRBFSolverPoseIndices(si, poseIndices.data(), static_cast<std::uint16_t>(poseIndices.size()));
        const auto rawControlValues = source->getRBFSolverRawControlValues(si);
        destination->setRBFSolverRawControlValues(si,
                                                  rawControlValues.data(),
                                                  static_cast<std::uint16_t>(rawControlValues.size()));
        destination->setRBFSolverType(si, source->getRBFSolverType(si));
        destination->setRBFSolverRadius(si, source->getRBFSolverRadius(si));
        destination->setRBFSolverAutomaticRadius(si, source->getRBFSolverAutomaticRadius(si));
        destination->setRBFSolverWeightThreshold(si, source->getRBFSolverWeightThreshold(si));
        destination->setRBFSolverDistanceMethod(si, source->getRBFSolverDistanceMethod(si));
        destination->setRBFSolverNormalizeMethod(si, source->getRBFSolverNormalizeMethod(si));
        destination->setRBFSolverFunctionType(si, source->getRBFSolverFunctionType(si));
        destination->setRBFSolverTwistAxis(si, source->getRBFSolverTwistAxis(si));
    }
}

static void copyJointBehaviorMetaData(const JointBehaviorMetadataReader* source,
                                      JointBehaviorMetadataWriter* destination,
                                      MemoryResource* /*unused*/) {
    destination->clearJointRepresentations();

    for (std::uint16_t jointPlusOne = source->getJointCount(); jointPlusOne > 0u; --jointPlusOne) {
        const auto ji = static_cast<std::uint16_t>(jointPlusOne - 1u);
        destination->setJointTranslationRepresentation(ji, source->getJointTranslationRepresentation(ji));
        destination->setJointRotationRepresentation(ji, source->getJointRotationRepresentation(ji));
        destination->setJointScaleRepresentation(ji, source->getJointScaleRepresentation(ji));
    }
}

static bool hasTwistSwingBehavior(const TwistSwingBehaviorReader* source) {
    // Heuristic for determining whether source DNA actually has any TwistSwing behavior data
    return (source->getTwistCount() != 0u) || (source->getSwingCount() != 0u);
}

static void copyTwistSwingBehavior(const TwistSwingBehaviorReader* source,
                                   TwistSwingBehaviorWriter* destination,
                                   MemoryResource* /*unused*/) {
    destination->clearTwists();
    destination->clearSwings();

    if (!hasTwistSwingBehavior(source)) {
        // Source DNA was loaded without twist and swig behavior layer
        return;
    }

    for (std::uint16_t twistIndexPlusOne = source->getTwistCount(); twistIndexPlusOne > 0u; --twistIndexPlusOne) {
        const auto ti = static_cast<std::uint16_t>(twistIndexPlusOne - 1u);
        destination->setTwistSetupTwistAxis(ti, source->getTwistSetupTwistAxis(ti));
        const auto twistInputControlIndices = source->getTwistInputControlIndices(ti);
        destination->setTwistInputControlIndices(ti,
                                                 twistInputControlIndices.data(),
                                                 static_cast<std::uint16_t>(twistInputControlIndices.size()));
        const auto twistOutputJointIndices = source->getTwistOutputJointIndices(ti);
        destination->setTwistOutputJointIndices(ti,
                                                twistOutputJointIndices.data(),
                                                static_cast<std::uint16_t>(twistOutputJointIndices.size()));
        const auto twistBlendWeights = source->getTwistBlendWeights(ti);
        destination->setTwistBlendWeights(ti, twistBlendWeights.data(), static_cast<std::uint16_t>(twistBlendWeights.size()));
    }

    for (std::uint16_t swingIndexPlusOne = source->getSwingCount(); swingIndexPlusOne > 0u; --swingIndexPlusOne) {
        const auto si = static_cast<std::uint16_t>(swingIndexPlusOne - 1u);
        destination->setSwingSetupTwistAxis(si, source->getSwingSetupTwistAxis(si));
        const auto swingInputControlIndices = source->getSwingInputControlIndices(si);
        destination->setSwingInputControlIndices(si,
                                                 swingInputControlIndices.data(),
                                                 static_cast<std::uint16_t>(swingInputControlIndices.size()));
        const auto swingOutputJointIndices = source->getSwingOutputJointIndices(si);
        destination->setSwingOutputJointIndices(si,
                                                swingOutputJointIndices.data(),
                                                static_cast<std::uint16_t>(swingOutputJointIndices.size()));
        const auto swingBlendWeights = source->getSwingBlendWeights(si);
        destination->setSwingBlendWeights(si, swingBlendWeights.data(), static_cast<std::uint16_t>(swingBlendWeights.size()));
    }
}

static bool hasMachineLearnedBehaviorExt(const MachineLearnedBehaviorExtReader* source) {
    // Heuristic for determining whether source DNA actually has any ML behavior ext data
    const auto mlTypeCount = source->getMLTypeCount();
    const auto neuralNetCount = source->getNeuralNetworkCount();

    // Check if this is the old ML layer accessed through the new API
    if (mlTypeCount == 1u) {
        const auto mlOperationSetCount = source->getMLOperationSetCount(0u);
        if (mlOperationSetCount == 3u) {
            const auto mlOperationCount1 = source->getMLOperationCount(0u, 0u);
            const auto mlOperationCount2 = source->getMLOperationCount(0u, 1u);
            const auto mlOperationCount3 = source->getMLOperationCount(0u, 2u);
            if ((mlOperationCount1 == neuralNetCount) && (mlOperationCount2 == neuralNetCount) &&
                (mlOperationCount3 == neuralNetCount)) {
                return false;
            }
        }
    }

    return (mlTypeCount > 0u);
}

static void copyMachineLearnedBehaviorExt(const MachineLearnedBehaviorExtReader* source,
                                          MachineLearnedBehaviorExtWriter* destination,
                                          MemoryResource* memRes) {
    destination->clearMLExtData();
    if (!hasMachineLearnedBehaviorExt(source)) {
        // Source DNA was loaded without machine learned behavior extension layer
        return;
    }

    using namespace std::placeholders;
    for (std::uint16_t mlTypeIndex = 0; mlTypeIndex < source->getMLTypeCount(); ++mlTypeIndex) {
        for (std::uint16_t i = source->getMLOperationSetCount(mlTypeIndex); i > 0u; --i) {
            const auto opSetIdx = static_cast<std::uint16_t>(i - 1u);

            copyLODIndices(
                std::bind(&MachineLearnedBehaviorExtReader::getMLOperationIndicesForLOD, source, mlTypeIndex, opSetIdx, _1),
                std::bind(&MachineLearnedBehaviorExtWriter::setMLOperationIndices,
                          destination,
                          mlTypeIndex,
                          opSetIdx,
                          _1,
                          _2,
                          _3),
                std::bind(&MachineLearnedBehaviorExtWriter::setLODMLOperationMapping, destination, mlTypeIndex, opSetIdx, _1, _2),
                source->getLODCount(),
                memRes);

            for (std::uint16_t j = source->getMLOperationCount(mlTypeIndex, opSetIdx); j > 0u; --j) {
                const auto opIdx = static_cast<std::uint16_t>(j - 1u);
                destination->setMLOperationType(mlTypeIndex,
                                                opSetIdx,
                                                opIdx,
                                                source->getMLOperationType(mlTypeIndex, opSetIdx, opIdx));

                const auto parameters = source->getMLOperationParameters(mlTypeIndex, opSetIdx, opIdx);
                destination->setMLOperationParameters(mlTypeIndex,
                                                      opSetIdx,
                                                      opIdx,
                                                      parameters.data(),
                                                      static_cast<std::uint16_t>(parameters.size()));

                const auto depOpSetIndices = source->getMLOperationDependencyOperationSetIndices(mlTypeIndex, opSetIdx, opIdx);
                destination->setMLOperationDependencyOperationSetIndices(mlTypeIndex,
                                                                         opSetIdx,
                                                                         opIdx,
                                                                         depOpSetIndices.data(),
                                                                         static_cast<std::uint16_t>(depOpSetIndices.size()));

                const auto depOpIndices = source->getMLOperationDependencyOperationIndices(mlTypeIndex, opSetIdx, opIdx);
                destination->setMLOperationDependencyOperationIndices(mlTypeIndex,
                                                                      opSetIdx,
                                                                      opIdx,
                                                                      depOpIndices.data(),
                                                                      static_cast<std::uint16_t>(depOpIndices.size()));
            }
        }
    }

    const auto parameterKeys = source->getMLJointsParameterKeys();
    destination->setMLJointsParameterKeys(parameterKeys.data(), static_cast<std::uint16_t>(parameterKeys.size()));

    const auto parameterValues = source->getMLJointsParameterValues();
    destination->setMLJointsParameterValues(parameterValues.data(), static_cast<std::uint16_t>(parameterValues.size()));

    const auto inputIndices = source->getMLJointsInputIndices();
    destination->setMLJointsInputIndices(inputIndices.data(), static_cast<std::uint16_t>(inputIndices.size()));

    const auto outputIndices = source->getMLJointsOutputIndices();
    destination->setMLJointsOutputIndices(outputIndices.data(), static_cast<std::uint16_t>(outputIndices.size()));
}

void Writer::setFrom(const Reader* source, DataLayer layer, UnknownLayerPolicy /*unused*/, MemoryResource* memRes) {
    // Unknown layers are inaccessible when copying through the public API
    if (source == nullptr) {
        return;
    }

    const auto bitmask = computeDataLayerBitmask(layer);
    copyHeader(source, this, memRes);
    copyDescriptor(source, this, memRes);
    if (contains(bitmask, DataLayerBitmask::Definition)) {
        copyDefinition(source, this, memRes);
    }
    if (contains(bitmask, DataLayerBitmask::Behavior)) {
        copyBehavior(source, this, memRes);
    }
    if (contains(bitmask, DataLayerBitmask::GeometryRest)) {
        copyGeometry(source, this, memRes);
    }
    if (contains(bitmask, DataLayerBitmask::GeometryBlendShapesOnly)) {
        copyBlendShapeTargets(source, this, memRes);
    }
    if (contains(bitmask, DataLayerBitmask::MachineLearnedBehavior)) {
        copyMachineLearnedBehavior(source, this, memRes);
        copyMachineLearnedBehaviorExt(source, this, memRes);
    }
    if (contains(bitmask, DataLayerBitmask::RBFBehavior)) {
        copyRBFBehavior(source, this, memRes);
    }
    if (contains(bitmask, DataLayerBitmask::JointBehaviorMetadata)) {
        copyJointBehaviorMetaData(source, this, memRes);
    }
    if (contains(bitmask, DataLayerBitmask::TwistSwingBehavior)) {
        copyTwistSwingBehavior(source, this, memRes);
    }
}

}  // namespace dna
