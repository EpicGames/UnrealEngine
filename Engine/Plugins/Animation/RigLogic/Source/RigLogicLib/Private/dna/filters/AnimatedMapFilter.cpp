// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/filters/AnimatedMapFilter.h"

#include "dna/DNA.h"
#include "dna/TypeDefs.h"
#include "dna/filters/Remap.h"
#include "dna/utils/Extd.h"

namespace dna {

AnimatedMapFilter::AnimatedMapFilter(MemoryResource* memRes_) :
    memRes{memRes_},
    passingIndices{memRes},
    remappedIndices{memRes} {
}

void AnimatedMapFilter::configure(std::uint16_t animatedMapCount, UnorderedSet<std::uint16_t> allowedAnimatedMapIndices) {
    passingIndices = std::move(allowedAnimatedMapIndices);
    // Fill the structure that maps indices prior to deletion to indices after deletion
    remappedIndices.clear();
    remap(animatedMapCount, passingIndices, remappedIndices);
}

void AnimatedMapFilter::apply(RawDefinition& dest) {
    // Fix indices so they match the same elements as earlier (but their
    // actual position changed with the deletion of the unneeded entries)
    dest.lodAnimatedMapMapping.mapIndices([this](std::uint16_t value) { return remappedIndices.at(value); });
    // Delete elements that are not referenced by the new subset of LODs
    extd::filter(dest.animatedMapNames, extd::byPosition(passingIndices));
}

bool AnimatedMapFilter::passes(std::uint16_t index) const {
    return extd::contains(passingIndices, index);
}

}  // namespace dna
