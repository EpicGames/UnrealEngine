// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/filters/BlendShapeFilter.h"

#include "dna/DNA.h"
#include "dna/TypeDefs.h"
#include "dna/filters/Remap.h"
#include "dna/utils/Extd.h"

namespace dna {

BlendShapeFilter::BlendShapeFilter(MemoryResource* memRes_) :
    memRes{memRes_},
    passingIndices{memRes},
    remappedIndices{memRes} {
}

void BlendShapeFilter::configure(std::uint16_t blendShapeCount, UnorderedSet<std::uint16_t> allowedBlendShapeIndices) {
    passingIndices = std::move(allowedBlendShapeIndices);
    // Fill the structure that maps indices prior to deletion to indices after deletion
    remappedIndices.clear();
    remap(blendShapeCount, passingIndices, remappedIndices);
}

void BlendShapeFilter::apply(RawDefinition& dest) {
    // Fix indices so they match the same elements as earlier (but their
    // actual position changed with the deletion of the unneeded entries)
    dest.lodBlendShapeMapping.mapIndices([this](std::uint16_t value) { return remappedIndices.at(value); });
    // Delete elements that are not referenced by the new subset of LODs
    extd::filter(dest.blendShapeChannelNames, extd::byPosition(passingIndices));
    // Delete entries from other mappings that reference any of the deleted elements
    auto ignoredByLODConstraint = [this](std::uint16_t /*unused*/, std::uint16_t blendShapeIndex) {
        return !extd::contains(passingIndices, blendShapeIndex);
    };
    dest.meshBlendShapeChannelMapping.removeIf(ignoredByLODConstraint);
    dest.meshBlendShapeChannelMapping.updateTo(remappedIndices);
}

void BlendShapeFilter::apply(RawMesh& dest) {
    // Remove blend shape targets of blend shapes to remove
    extd::filter(dest.blendShapeTargets, [this](const RawBlendShapeTarget& bsTarget, std::size_t /*unused*/) {
        return passes(bsTarget.blendShapeChannelIndex);
    });

    // Remap blend shape targets
    for (auto& bsTarget : dest.blendShapeTargets) {
        bsTarget.blendShapeChannelIndex = remapped(bsTarget.blendShapeChannelIndex);
    }
}

bool BlendShapeFilter::passes(std::uint16_t index) const {
    return extd::contains(passingIndices, index);
}

std::uint16_t BlendShapeFilter::remapped(std::uint16_t index) const {
    return remappedIndices.at(index);
}

}  // namespace dna
