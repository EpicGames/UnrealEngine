// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/TypeDefs.h"

#include <cstdint>

namespace dna {

struct RawBehavior;
struct RawDefinition;
struct RawJointBehaviorMetadata;
struct RawLODMapping;
struct RawTwistSwingBehavior;
struct RawVertexSkinWeights;
struct RawMachineLearnedBehaviorJoints;

class JointFilter {
public:
    enum class Option {
        All,
        AnimationOnly
    };

public:
    explicit JointFilter(MemoryResource* memRes_);
    void configure(std::uint16_t jointCount, UnorderedSet<std::uint16_t> allowedJointIndices, Option option_ = Option::All);
    void apply(RawDefinition& dest);
    void apply(RawBehavior& dest);
    void apply(RawVertexSkinWeights& dest);
    void apply(RawJointBehaviorMetadata& dest);
    void apply(RawTwistSwingBehavior& dest);
    void apply(RawMachineLearnedBehaviorJoints& dest);
    bool passes(std::uint16_t index) const;
    std::uint16_t remapped(std::uint16_t oldIndex) const;
    std::uint16_t filteredJointCount() const;

private:
    MemoryResource* memRes;
    UnorderedSet<std::uint16_t> passingIndices;
    UnorderedMap<std::uint16_t, std::uint16_t> remappedIndices;
    Option option;
    std::uint16_t rootJointIndex;
};

}  // namespace dna
