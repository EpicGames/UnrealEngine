// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/TypeDefs.h"
#include "dna/utils/Extd.h"

namespace dna {

template<typename T>
inline void remap(T originalCount, const UnorderedSet<T>& keptIndices, UnorderedMap<T, T>& mapping) {
    for (T oldIndex{}, newIndex{}; oldIndex < originalCount; ++oldIndex) {
        if (extd::contains(keptIndices, oldIndex)) {
            mapping.insert({oldIndex, newIndex});
            ++newIndex;
        }
    }
}

}  // namespace dna
