// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

#include "dna/stream/BinaryStreamReaderImpl.h"

#include "dna/TypeDefs.h"
#include "dna/types/CoordinateSystemConverter.h"
#include "dna/types/Limits.h"

#include <status/Provider.h>
#include <tdm/TDM.h>
#include <trio/utils/StreamScope.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <limits>
#include <tuple>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dna {

static LODConstraint createLODConstraintFromConfig(const Configuration& config, MemoryResource* memRes) {
    return (config.lods.size() != 0ul) ? LODConstraint(config.lods, memRes) : LODConstraint(config.maxLOD, config.minLOD, memRes);
}

BinaryStreamReader::~BinaryStreamReader() = default;

BinaryStreamReader* BinaryStreamReader::create(BoundedIOStream* stream,
                                               DataLayer layer,
                                               UnknownLayerPolicy policy,
                                               std::uint16_t maxLOD,
                                               MemoryResource* memRes) {
    Configuration config = {};
    config.layer = layer;
    config.unknownLayerPolicy = policy;
    config.maxLOD = maxLOD;
    return create(stream, config, memRes);
}

BinaryStreamReader* BinaryStreamReader::create(BoundedIOStream* stream,
                                               DataLayer layer,
                                               UnknownLayerPolicy policy,
                                               std::uint16_t maxLOD,
                                               std::uint16_t minLOD,
                                               MemoryResource* memRes) {
    Configuration config = {};
    config.layer = layer;
    config.unknownLayerPolicy = policy;
    config.maxLOD = maxLOD;
    config.minLOD = minLOD;
    return create(stream, config, memRes);
}

BinaryStreamReader* BinaryStreamReader::create(BoundedIOStream* stream,
                                               DataLayer layer,
                                               UnknownLayerPolicy policy,
                                               std::uint16_t* lods,
                                               std::uint16_t lodCount,
                                               MemoryResource* memRes) {
    Configuration config = {};
    config.layer = layer;
    config.unknownLayerPolicy = policy;
    config.lods = {lods, lodCount};
    return create(stream, config, memRes);
}

BinaryStreamReader* BinaryStreamReader::create(BoundedIOStream* stream, const Configuration& config, MemoryResource* memRes) {
    PolyAllocator<BinaryStreamReaderImpl> alloc{memRes};
    return alloc.newObject(stream, config, memRes);
}

void BinaryStreamReader::destroy(BinaryStreamReader* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto reader = static_cast<BinaryStreamReaderImpl*>(instance);
    PolyAllocator<BinaryStreamReaderImpl> alloc{reader->getMemoryResource()};
    alloc.deleteObject(reader);
}

BinaryStreamReaderImpl::BinaryStreamReaderImpl(BoundedIOStream* stream_, const Configuration& config_, MemoryResource* memRes_) :
    BaseImpl{config_.unknownLayerPolicy, UpgradeFormatPolicy::Disallowed, memRes_},
    ReaderImpl{memRes_},
    stream{stream_},
    archive{stream_, config_.layer, createLODConstraintFromConfig(config_, memRes), memRes_},
    config{config_},
    lodConstrained{(config.maxLOD != LODLimits::max()) || (config.minLOD != LODLimits::min()) || (config.lods.size() != 0ul)} {
}

bool BinaryStreamReaderImpl::isLODConstrained() const {
    return lodConstrained;
}

void BinaryStreamReaderImpl::unload(DataLayer layer) {
    if ((layer == DataLayer::All) || (layer == DataLayer::Descriptor)) {
        dna = DNA{dna.layers.unknownPolicy, dna.layers.upgradePolicy, memRes};
    } else if (layer == DataLayer::TwistSwingBehavior) {
        dna.unloadTwistSwingBehavior();
    } else if (layer == DataLayer::RBFBehavior) {
        dna.unloadRBFBehavior();
    } else if (layer == DataLayer::JointBehaviorMetadata) {
        dna.unloadJointBehaviorMetadata();
    } else if (layer == DataLayer::MachineLearnedBehavior) {
        dna.unloadMachineLearnedBehavior();
    } else if ((layer == DataLayer::Geometry) || (layer == DataLayer::GeometryWithoutBlendShapes)) {
        dna.unloadGeometry();
    } else if (layer == DataLayer::Behavior) {
        dna.unloadRBFBehavior();
        dna.unloadBehavior();
    } else if (layer == DataLayer::Definition) {
        dna.unloadJointBehaviorMetadata();
        dna.unloadTwistSwingBehavior();
        dna.unloadRBFBehavior();
        dna.unloadMachineLearnedBehavior();
        dna.unloadGeometry();
        dna.unloadBehavior();
        dna.unloadDefinition();
    }
}

void BinaryStreamReaderImpl::read() {
    // Due to possible usage of custom stream implementations, the status actually must be cleared at this point
    // as external streams do not have access to the status reset API
    status.reset();

    trio::StreamScope scope{stream};
    if (!sc::Status::isOk()) {
        return;
    }

    archive >> dna;
    if (!archive.isOk()) {
        status.set(InvalidDataError);
        return;
    }

    if (!dna.signature.matches()) {
        status.set(SignatureMismatchError, dna.signature.value.expected.data(), dna.signature.value.got.data());
        return;
    }
    if (!dna.version.supported()) {
        status.set(VersionMismatchError, dna.version.generation, dna.version.version);
        return;
    }

    const bool isSameCoordSys = (dna.descriptor.coordinateSystem == config.coordinateSystem) &&
                                (dna.descriptorExt.rotationSequence == config.rotationSequence) &&
                                (dna.descriptorExt.rotationSign == config.rotationSign) &&
                                (dna.descriptorExt.faceWindingOrder == config.faceWindingOrder);
    if ((config.coordinateSystemTransformPolicy == CoordinateSystemTransformPolicy::Transform) && !isSameCoordSys) {
        CoordinateSystemConverter csc{config.coordinateSystem,
                                      config.rotationSequence,
                                      config.rotationSign,
                                      config.faceWindingOrder,
                                      memRes};
        csc.convert(dna);
    }

    // Cache must be populated after coordinate system conversion since some data that is about to be cached needs conversion
    cache.populate(this);
}

}  // namespace dna
