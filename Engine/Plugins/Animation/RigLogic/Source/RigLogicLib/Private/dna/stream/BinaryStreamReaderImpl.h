// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/BinaryStreamReader.h"
#include "dna/ReaderImpl.h"
#include "dna/TypeDefs.h"
#include "dna/stream/FilteredBinaryInputArchive.h"
#include "dna/stream/StreamReaderStatus.h"

#include <status/Provider.h>

namespace dna {

class BinaryStreamReaderImpl : public ReaderImpl<BinaryStreamReader>, private StreamReaderStatus {
public:
    BinaryStreamReaderImpl(BoundedIOStream* stream_, const Configuration& config_, MemoryResource* memRes_);

    void unload(DataLayer layer) override;
    void read() override;
    bool isLODConstrained() const;

private:
    BoundedIOStream* stream;
    FilteredBinaryInputArchive archive;
    Configuration config;
    bool lodConstrained;
};

}  // namespace dna
