// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

#include "dna/stream/BinaryStreamWriterImpl.h"

#include "dna/TypeDefs.h"
#include "dna/stream/BinaryStreamReaderImpl.h"
#include "dna/stream/JSONStreamReaderImpl.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <cstring>
#include <tuple>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dna {

BinaryStreamWriter::~BinaryStreamWriter() = default;

BinaryStreamWriter* BinaryStreamWriter::create(BoundedIOStream* stream, MemoryResource* memRes) {
    PolyAllocator<BinaryStreamWriterImpl> alloc{memRes};
    return alloc.newObject(stream, memRes);
}

void BinaryStreamWriter::destroy(BinaryStreamWriter* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto writer = static_cast<BinaryStreamWriterImpl*>(instance);
    PolyAllocator<BinaryStreamWriterImpl> alloc{writer->getMemoryResource()};
    alloc.deleteObject(writer);
}

BinaryStreamWriterImpl::BinaryStreamWriterImpl(BoundedIOStream* stream_, MemoryResource* memRes_) :
    BaseImpl{UnknownLayerPolicy::Preserve, UpgradeFormatPolicy::Allowed, memRes_},
    WriterImpl{memRes_},
    stream{stream_},
    archive{stream_} {
}

void BinaryStreamWriterImpl::write() {
    stream->open();
    archive << dna;
    archive.sync();
    stream->close();
}

void BinaryStreamWriterImpl::setFrom(const BinaryStreamReader* source,
                                     DataLayer layer,
                                     UnknownLayerPolicy policy,
                                     MemoryResource* memRes_) {
    auto reader = const_cast<BinaryStreamReaderImpl*>(static_cast<const BinaryStreamReaderImpl*>(source));
    reader->rawCopyInto(dna, layer, policy, memRes_);
}

#ifdef DNA_BUILD_WITH_JSON_SUPPORT
void BinaryStreamWriterImpl::setFrom(const JSONStreamReader* source,
                                     DataLayer layer,
                                     UnknownLayerPolicy policy,
                                     MemoryResource* memRes_) {
    auto reader = const_cast<JSONStreamReaderImpl*>(static_cast<const JSONStreamReaderImpl*>(source));
    reader->rawCopyInto(dna, layer, policy, memRes_);
}

#endif  // DNA_BUILD_WITH_JSON_SUPPORT

}  // namespace dna
