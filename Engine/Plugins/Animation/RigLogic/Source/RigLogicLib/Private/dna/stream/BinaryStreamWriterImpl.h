// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/BinaryStreamWriter.h"
#include "dna/DNA.h"
#include "dna/WriterImpl.h"

#include <terse/archives/binary/OutputArchive.h>

namespace dna {

class BinaryStreamReader;
class JSONStreamReader;

class BinaryStreamWriterImpl : public WriterImpl<BinaryStreamWriter> {
public:
    BinaryStreamWriterImpl(BoundedIOStream* stream_, MemoryResource* memRes_);

    void write() override;

    using BinaryStreamWriter::setFrom;
    void setFrom(const BinaryStreamReader* source, DataLayer layer, UnknownLayerPolicy policy, MemoryResource* memRes_) override;
#ifdef DNA_BUILD_WITH_JSON_SUPPORT
    void setFrom(const JSONStreamReader* source, DataLayer layer, UnknownLayerPolicy policy, MemoryResource* memRes_) override;
#endif  // DNA_BUILD_WITH_JSON_SUPPORT

private:
    BoundedIOStream* stream;
    terse::BinaryOutputArchive<BoundedIOStream> archive;
};

}  // namespace dna
