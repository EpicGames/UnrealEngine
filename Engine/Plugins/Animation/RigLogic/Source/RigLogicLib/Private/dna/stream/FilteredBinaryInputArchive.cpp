// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/stream/FilteredBinaryInputArchive.h"

#include "dna/DNA.h"
#include "dna/TypeDefs.h"
#include "dna/filters/JointFilter.h"
#include "dna/filters/Remap.h"
#include "dna/layers/MachineLearnedBehaviorExt.h"
#include "dna/types/Limits.h"
#include "dna/utils/Extd.h"
#include "dna/utils/ScopedEnumEx.h"

#include <status/Provider.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dna {

static constexpr std::uint16_t jointAttributeCount = 9u;

template<typename TContainer>
static void removeByIndex(TContainer& container, std::size_t index) {
    assert(index < container.size());
    for (std::size_t i = index; i < (container.size() - 1ul); ++i) {
        container[i] = container[i + 1ul];
    }
    container.resize(container.size() - 1ul);
}

FilteredBinaryInputArchive::FilteredBinaryInputArchive(BoundedIOStream* stream_,
                                                       DataLayer layer_,
                                                       LODConstraint&& lodConstraint_,
                                                       MemoryResource* memRes_) :
    AnimatedMapFilter{memRes_},
    BlendShapeFilter{memRes_},
    JointFilter{memRes_},
    MeshFilter{memRes_},
    BaseArchive{this, stream_},
    stream{stream_},
    memRes{memRes_},
    layerBitmask{computeDataLayerBitmask(layer_)},
    lodConstraint{std::move(lodConstraint_)},
    loadedControls(memRes),
    unconstrainedLODCount{},
    malformed{false} {
}

bool FilteredBinaryInputArchive::isOk() {
    return !malformed && sc::Status::isOk();
}

std::size_t FilteredBinaryInputArchive::boundSize(std::size_t size) {
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
    if (static_cast<std::uint64_t>(size) > stream->size()) {
        malformed = true;
        return 0ul;
    }
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
    return size;
}

template<class TContainer>
void FilteredBinaryInputArchive::processSubset(TContainer& dest, std::size_t offset, std::size_t size) {
    using ElementType = typename TContainer::value_type;
    const auto availableSize = processSize();
    assert(offset + size <= availableSize);
    const auto startPosition = stream->tell();
    // Skip over first N elements
    stream->seek(startPosition + offset * sizeof(ElementType));
    // Read requested number of elements
    BaseArchive::processElements(dest, size);
    // Even if not all elements were read, seek to the end of the list
    stream->seek(startPosition + availableSize * sizeof(ElementType));
}

void FilteredBinaryInputArchive::process(RawDescriptor& dest) {
    BaseArchive::process(dest);
    malformed = malformed || (dest.lodCount > LODLimits::count());
    if (malformed) {
        return;
    }

    lodConstraint.clampTo(dest.lodCount);
    unconstrainedLODCount = dest.lodCount;
    dest.maxLOD = static_cast<std::uint16_t>(dest.maxLOD + lodConstraint.getMaxLOD());
    dest.lodCount = lodConstraint.getLODCount();
}

void FilteredBinaryInputArchive::process(RawDefinition& dest) {
    if (malformed) {
        return;
    }

    if (!contains(layerBitmask, DataLayerBitmask::Definition)) {
        return;
    }
    // Load all data
    BaseArchive::process(dest);

    loadedControls.resize(dest.rawControlNames.size(), true);

    // No filtering is done, unless LOD constraint may have some effect
    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        return;
    }

    // To find joints that are not in any LOD, find the joints that are not in LOD 0 (the current max LOD, at index 0), as it
    // contains joints from all lower LODs.
    Vector<std::uint16_t> jointsNotInLOD0{memRes};
    const auto jointIndicesForLOD0 = dest.lodJointMapping.getIndices(0);
    for (std::uint16_t idx = 0; idx < dest.jointNames.size(); ++idx) {
        if (std::find(jointIndicesForLOD0.begin(), jointIndicesForLOD0.end(), idx) == jointIndicesForLOD0.end()) {
            jointsNotInLOD0.push_back(idx);
        }
    }

    // Discard LOD data that is not relevant for the selected MaxLOD and MinLOD constraints
    dest.lodMeshMapping.discardLODs(lodConstraint);
    dest.lodJointMapping.discardLODs(lodConstraint);
    dest.lodBlendShapeMapping.discardLODs(lodConstraint);
    dest.lodAnimatedMapMapping.discardLODs(lodConstraint);
    MeshFilter::configure(static_cast<std::uint16_t>(dest.meshNames.size()),
                          dest.lodMeshMapping.getCombinedDistinctIndices(memRes));
    MeshFilter::apply(dest);
    auto allowedJointIndices = dest.lodJointMapping.getCombinedDistinctIndices(memRes);
    // In order to keep joints that are not in any LOD, add them all to the list of joints to keep when filtering.
    allowedJointIndices.insert(jointsNotInLOD0.begin(), jointsNotInLOD0.end());
    JointFilter::configure(static_cast<std::uint16_t>(dest.jointNames.size()), allowedJointIndices);
    JointFilter::apply(dest);
    BlendShapeFilter::configure(static_cast<std::uint16_t>(dest.blendShapeChannelNames.size()),
                                dest.lodBlendShapeMapping.getCombinedDistinctIndices(memRes));
    BlendShapeFilter::apply(dest);
    AnimatedMapFilter::configure(static_cast<std::uint16_t>(dest.animatedMapNames.size()),
                                 dest.lodAnimatedMapMapping.getCombinedDistinctIndices(memRes));
    AnimatedMapFilter::apply(dest);
}

void FilteredBinaryInputArchive::process(RawControls& dest) {
    BaseArchive::process(dest);
    loadedControls.resize(loadedControls.size() + dest.psdCount, true);
}

void FilteredBinaryInputArchive::process(RawJoints& dest) {
    process(dest.rowCount);
    process(dest.colCount);
    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        process(dest.jointGroups);
        return;
    }
    // Perform filtered load only if LOD constraints have been set
    const auto jointGroupCount = processSize();
    dest.jointGroups.reserve(jointGroupCount);
    for (std::size_t i = 0ul; i < jointGroupCount; ++i) {
        RawJointGroup jointGroup{memRes};
        process(jointGroup.lods);
        // Discard everything that falls outside the region bounded by LOD constraints
        lodConstraint.applyTo(jointGroup.lods);
        // Input indices are all loaded always (unless the whole joint group is empty)
        const auto jointGroupRowCount = (jointGroup.lods.empty() ? static_cast<std::uint16_t>(0) : jointGroup.lods[0]);
        if (jointGroupRowCount != 0u) {
            process(jointGroup.inputIndices);
        } else {
            processSubset(jointGroup.inputIndices, 0ul, 0ul);
        }
        const auto jointGroupColumnCount = jointGroup.inputIndices.size();

        processSubset(jointGroup.outputIndices, 0ul, jointGroupRowCount);
        // Remap joint attribute indices
        for (auto& attrIdx : jointGroup.outputIndices) {
            const auto jntIdx = static_cast<std::uint16_t>(attrIdx / jointAttributeCount);
            const auto relAttrIdx = attrIdx - (jntIdx * jointAttributeCount);
            attrIdx = static_cast<std::uint16_t>(JointFilter::remapped(jntIdx) * jointAttributeCount + relAttrIdx);
        }

        processSubset(jointGroup.values, 0ul, jointGroupRowCount * jointGroupColumnCount);
        // Load and remap joint indices (according to the remapping created while loading the Definition layer)
        process(jointGroup.jointIndices);
        extd::filter(jointGroup.jointIndices,
                     [this](std::uint16_t jntIdx, std::size_t /*unused*/) { return JointFilter::passes(jntIdx); });
        for (auto& jntIdx : jointGroup.jointIndices) {
            jntIdx = JointFilter::remapped(jntIdx);
        }

        dest.jointGroups.push_back(std::move(jointGroup));
    }
    dest.rowCount = static_cast<std::uint16_t>(JointFilter::filteredJointCount() * jointAttributeCount);
}

void FilteredBinaryInputArchive::process(RawBlendShapeChannels& dest) {
    process(dest.lods);
    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        process(dest.inputIndices);
        process(dest.outputIndices);
        return;
    }
    // Discard everything that falls outside the region bounded by LOD constraints
    lodConstraint.applyTo(dest.lods);
    const auto count = (dest.lods.empty() ? static_cast<std::uint16_t>(0) : dest.lods[0]);
    processSubset(dest.inputIndices, 0ul, count);
    processSubset(dest.outputIndices, 0ul, count);
}

void FilteredBinaryInputArchive::process(RawAnimatedMaps& dest) {
    process(dest.lods);
    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        process(dest.conditionals);
        return;
    }
    // Discard everything that falls outside the region bounded by LOD constraints
    lodConstraint.applyTo(dest.lods);
    const auto rowCount = (dest.lods.empty() ? static_cast<std::uint16_t>(0) : dest.lods[0]);
    processSubset(dest.conditionals.inputIndices, 0ul, rowCount);
    processSubset(dest.conditionals.outputIndices, 0ul, rowCount);
    processSubset(dest.conditionals.fromValues, 0ul, rowCount);
    processSubset(dest.conditionals.toValues, 0ul, rowCount);
    processSubset(dest.conditionals.slopeValues, 0ul, rowCount);
    processSubset(dest.conditionals.cutValues, 0ul, rowCount);
}

void FilteredBinaryInputArchive::process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v21>>&& dest) {
    process(dest);
}

void FilteredBinaryInputArchive::process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v21>>& dest) {
    if (malformed) {
        return;
    }

    if (!contains(layerBitmask, DataLayerBitmask::GeometryRest)) {
        // As mesh sizes are variable, iterate over each of them, reading only the mesh
        // offsets and jumping over the actual data of the meshes.
        // This will correctly position the underlying stream (end of geometry layer),
        // while still not reading the data.
        const auto meshCount = processSize();
        decltype(RawMesh::offset) meshOffset{};
        decltype(RawMesh::offsetMarker) meshMarker{meshOffset};
        for (std::uint16_t i = {}; i < meshCount; ++i) {
            process(meshOffset);
            process(meshMarker);
        }
        return;
    }

    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        process(dest.data.meshes);
        return;
    }

    // Perform filtered load only if a different maxLOD is set
    const auto meshCount = processSize();
    dest.data.meshes.reserve(meshCount);
    for (std::uint16_t i = {}; i < meshCount; ++i) {
        // Check if the mesh indices filtered for the current maxLOD permit loading this mesh
        if (MeshFilter::passes(i)) {
            RawMesh mesh{memRes};
            process(mesh);
            dest.data.meshes.push_back(std::move(mesh));
        } else {
            // Jump over the whole section of data related to this mesh
            decltype(RawMesh::offset) meshOffset{};
            decltype(RawMesh::offsetMarker) meshOffsetMarker{meshOffset};
            process(meshOffset);
            process(meshOffsetMarker);
        }
    }
}

void FilteredBinaryInputArchive::process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v22>>&& dest) {
    process(dest);
}

void FilteredBinaryInputArchive::process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v22>>& dest) {
    if (malformed) {
        return;
    }

    if (!contains(layerBitmask, DataLayerBitmask::GeometryRest)) {
        return;
    }

    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        process(dest.data.meshes);
        return;
    }
    // Perform filtered load only if a different maxLOD is set
    const auto meshCount = processSize();
    dest.data.meshes.reserve(meshCount);
    for (std::uint16_t i = {}; i < meshCount; ++i) {
        // Check if the mesh indices filtered for the current maxLOD permit loading this mesh
        if (MeshFilter::passes(i)) {
            RawMesh mesh{memRes};
            process(mesh);
            dest.data.meshes.push_back(std::move(mesh));
        } else {
            // Jump over the whole section of data related to this mesh
            decltype(RawMesh::size) meshSize{};
            decltype(RawMesh::baseMarker) meshBase{};
            decltype(RawMesh::sizeMarker) meshSizeMarker{meshSize, meshBase};
            process(meshSize);
            process(meshBase);
            process(meshSizeMarker);
        }
    }
}

void FilteredBinaryInputArchive::process(Vector<RawBlendShapeTarget>& dest) {
    if (contains(layerBitmask, DataLayerBitmask::GeometryBlendShapesOnly)) {
        BaseArchive::process(dest);
        if (lodConstraint.hasImpactOn(unconstrainedLODCount)) {
            extd::filter(dest, [this](const RawBlendShapeTarget& bst, std::size_t /*unused*/) {
                return BlendShapeFilter::passes(bst.blendShapeChannelIndex);
            });
        }
    }
}

void FilteredBinaryInputArchive::process(RawVertexSkinWeights& dest) {
    process(dest.weights);
    process(dest.jointIndices);

    if (lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        assert(dest.weights.size() == dest.jointIndices.size());
        JointFilter::apply(dest);
    }
}

void FilteredBinaryInputArchive::process(RawMachineLearnedBehavior& dest) {
    if (malformed) {
        return;
    }

    if (contains(layerBitmask, DataLayerBitmask::MachineLearnedBehavior)) {
        process(dest.mlControlNames);
        process(dest.lodNeuralNetworkMapping);
        process(dest.neuralNetworkToMeshRegion);

        loadedControls.resize(loadedControls.size() + dest.mlControlNames.size(), true);

        if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
            process(dest.neuralNetworks);
            return;
        }

        // Delete meshes and regions from neuralNetworkToMeshRegion mapping
        MeshFilter::apply(dest);

        // Perform filtered load only if a different maxLOD is set
        const auto neuralNetCount = static_cast<std::uint16_t>(processSize());

        dest.lodNeuralNetworkMapping.discardLODs(lodConstraint);
        const auto passingNetIndices = dest.lodNeuralNetworkMapping.getCombinedDistinctIndices(memRes);
        UnorderedMap<std::uint16_t, std::uint16_t> remappedIndicesLocal{memRes};
        remap(neuralNetCount, passingNetIndices, remappedIndicesLocal);
        dest.lodNeuralNetworkMapping.mapIndices(
            [&remappedIndicesLocal](std::uint16_t value) { return remappedIndicesLocal.at(value); });

        // Delete and remap neural network indices in neuralNetworkToMeshRegion mapping
        for (auto& mesh : dest.neuralNetworkToMeshRegion.indices) {
            for (auto& region : mesh) {
                extd::filter(region, extd::byValue(passingNetIndices));
                for (auto& netIndex : region) {
                    netIndex = remappedIndicesLocal.at(netIndex);
                }
            }
        }

        dest.neuralNetworks.reserve(passingNetIndices.size());
        for (std::uint16_t i = {}; i < neuralNetCount; ++i) {
            // Check if the neural net indices filtered for the current maxLOD permit loading
            // this neural network
            if (extd::contains(passingNetIndices, i)) {
                RawNeuralNetwork neuralNetwork{memRes};
                process(neuralNetwork);
                dest.neuralNetworks.push_back(std::move(neuralNetwork));
            } else {
                // Jump over neural network
                decltype(RawNeuralNetwork::size) netSize{};
                decltype(RawNeuralNetwork::baseMarker) netBase{};
                decltype(RawNeuralNetwork::sizeMarker) netSizeMarker{netSize, netBase};
                process(netSize);
                process(netBase);
                process(netSizeMarker);
            }
        }
    } else {
        const auto mlControlCount = processSize();
        loadedControls.resize(loadedControls.size() + mlControlCount, false);
    }
}

void FilteredBinaryInputArchive::process(RawMachineLearnedBehaviorExt& dest) {
    if (malformed) {
        return;
    }

    if (contains(layerBitmask, DataLayerBitmask::MachineLearnedBehavior)) {
        process(dest.mlbTypeData);
        process(dest.mlbJoints);

        if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
            return;
        }

        for (auto& mlbType : dest.mlbTypeData) {
            assert(mlbType.operations.size() == mlbType.lodMLOperationMappings.size());
            for (std::size_t mlOperationSetIndex = {}; mlOperationSetIndex < mlbType.operations.size(); ++mlOperationSetIndex) {
                auto& lodMapping = mlbType.lodMLOperationMappings[mlOperationSetIndex];
                auto& mlOperations = mlbType.operations[mlOperationSetIndex];

                const auto mlOperationCount = static_cast<std::uint16_t>(mlOperations.size());
                lodMapping.discardLODs(lodConstraint);
                const auto distinctIndices = lodMapping.getCombinedDistinctIndices(memRes);
                UnorderedMap<std::uint16_t, std::uint16_t> remappedIndicesLocal{memRes};
                remap(mlOperationCount, distinctIndices, remappedIndicesLocal);
                lodMapping.mapIndices([&remappedIndicesLocal](std::uint16_t value) { return remappedIndicesLocal.at(value); });

                extd::filter(mlOperations, [&distinctIndices](const RawMLOperation& operation, std::size_t index) {
                    if (!extd::contains(distinctIndices, index)) {
                        if (operation.operationType == static_cast<std::uint16_t>(MachineLearnedBehaviorOperationType::MLP)) {
                            assert(operation.parameters.size() == 1ul);
                        }
                        return false;
                    }
                    return true;
                });

                for (std::size_t mlNextOperationSetIndex = mlOperationSetIndex + 1;
                     mlNextOperationSetIndex < mlbType.operations.size();
                     ++mlNextOperationSetIndex) {
                    for (auto& operation : mlbType.operations[mlNextOperationSetIndex]) {
                        for (std::size_t i = {}; i < operation.mlOperationDependencyIndices.size(); ++i) {
                            if (operation.mlOperationSetDependencyIndices[i] == mlOperationSetIndex) {
                                auto it = remappedIndicesLocal.find(operation.mlOperationDependencyIndices[i]);
                                if (it != remappedIndicesLocal.end()) {
                                    operation.mlOperationDependencyIndices[i] = it->second;
                                }
                            }
                        }
                    }
                }
            }
        }

        JointFilter::apply(dest.mlbJoints);
    }
}

void FilteredBinaryInputArchive::process(DNA& dest) {
    BaseArchive::process(dest);
    // Don't run control-based post-load filtering for delta DNA files
    if ((dest.layers.unknownPolicy == UnknownLayerPolicy::Ignore) && (!loadedControls.empty())) {
        removeUnreferencedBlendShapes(dest);
    }
}

void FilteredBinaryInputArchive::removeUnreferencedBlendShapes(DNA& dest) {
    auto& bsc = dest.behavior.blendShapeChannels;

    const auto originalLODs = bsc.lods;
    Vector<std::uint16_t> unreferencedChannels{memRes};
    for (std::size_t iPlusOne = bsc.inputIndices.size(); iPlusOne > 0ul; --iPlusOne) {
        const auto i = iPlusOne - 1ul;
        const auto controlIndex = bsc.inputIndices[i];
        if ((controlIndex > loadedControls.size()) || (!loadedControls[controlIndex])) {
            unreferencedChannels.push_back(bsc.outputIndices[i]);
            // Remove behavior data
            removeByIndex(bsc.inputIndices, i);
            removeByIndex(bsc.outputIndices, i);
            for (std::uint16_t lod = {}; lod < bsc.lods.size(); ++lod) {
                if (i < originalLODs[lod]) {
                    --bsc.lods[lod];
                }
            }
        }
    }

    // Remove channel from definition
    dest.definition.lodBlendShapeMapping.filterIndices(
        [&unreferencedChannels](std::uint16_t index) { return !extd::contains(unreferencedChannels, index); });

    BlendShapeFilter::configure(static_cast<std::uint16_t>(dest.definition.blendShapeChannelNames.size()),
                                dest.definition.lodBlendShapeMapping.getCombinedDistinctIndices(memRes));
    BlendShapeFilter::apply(dest.definition);

    // Remap remaining output indices
    for (auto& outputIdx : bsc.outputIndices) {
        outputIdx = BlendShapeFilter::remapped(outputIdx);
    }

    // Remove associated blend shape targets from geometry
    for (auto& mesh : dest.geometry.meshes) {
        BlendShapeFilter::apply(mesh);
    }
}

void FilteredBinaryInputArchive::process(RawRBFBehavior& dest) {
    if (malformed) {
        return;
    }

    if (!contains(layerBitmask, DataLayerBitmask::RBFBehavior)) {
        return;
    }

    process(dest.lodSolverMapping);
    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        process(dest.solvers);
        process(dest.poses);
        return;
    }

    // Perform filtered load only if a different maxLOD is set
    const auto solverCount = static_cast<std::uint16_t>(processSize());
    dest.lodSolverMapping.discardLODs(lodConstraint);

    const auto passingSolverIndices = dest.lodSolverMapping.getCombinedDistinctIndices(memRes);
    UnorderedMap<std::uint16_t, std::uint16_t> remappedIndicesLocal{memRes};
    remap(solverCount, passingSolverIndices, remappedIndicesLocal);
    dest.lodSolverMapping.mapIndices([&remappedIndicesLocal](std::uint16_t value) { return remappedIndicesLocal.at(value); });

    dest.solvers.reserve(passingSolverIndices.size());
    for (std::uint16_t i = {}; i < solverCount; ++i) {
        // Check if the solver indices filtered for the current maxLOD permit loading
        // this solver
        if (extd::contains(passingSolverIndices, i)) {
            RawRBFSolver solver{memRes};
            process(solver);
            dest.solvers.push_back(std::move(solver));
        } else {
            // Jump over solver
            decltype(RawRBFSolver::size) solverSize{};
            decltype(RawRBFSolver::baseMarker) solverBase{};
            decltype(RawRBFSolver::sizeMarker) solverSizeMarker{solverSize, solverBase};
            process(solverSize);
            process(solverBase);
            process(solverSizeMarker);
        }
    }
    process(dest.poses);
}

void FilteredBinaryInputArchive::process(RawRBFBehaviorExt& dest) {
    if (malformed) {
        return;
    }

    if (contains(layerBitmask, DataLayerBitmask::RBFBehavior)) {
        process(dest.poseControlNames);
        process(dest.poses);
        loadedControls.resize(loadedControls.size() + dest.poseControlNames.size(), true);
    } else {
        const auto poseControlCount = processSize();
        loadedControls.resize(loadedControls.size() + poseControlCount, false);
    }
}

void FilteredBinaryInputArchive::process(RawJointBehaviorMetadata& dest) {
    if (malformed) {
        return;
    }
    if (!contains(layerBitmask, DataLayerBitmask::JointBehaviorMetadata)) {
        return;
    }

    BaseArchive::process(dest);
    // No filtering is done, unless LOD constraint may have some effect
    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        return;
    }
    JointFilter::apply(dest);
}

void FilteredBinaryInputArchive::process(RawTwistSwingBehavior& dest) {
    if (malformed) {
        return;
    }
    if (!contains(layerBitmask, DataLayerBitmask::TwistSwingBehavior)) {
        return;
    }

    BaseArchive::process(dest);
    // No filtering is done, unless LOD constraint may have some effect
    if (!lodConstraint.hasImpactOn(unconstrainedLODCount)) {
        return;
    }
    JointFilter::apply(dest);
}

}  // namespace dna
