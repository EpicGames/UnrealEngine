// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"
#include "dna/DNA.h"
#include "dna/DataLayerBitmask.h"
#include "dna/LODConstraint.h"
#include "dna/filters/AnimatedMapFilter.h"
#include "dna/filters/BlendShapeFilter.h"
#include "dna/filters/JointFilter.h"
#include "dna/filters/MeshFilter.h"
#include <terse/archives/binary/InputArchive.h>

#include <cstddef>
#include <cstdint>

namespace dna {

class FilteredBinaryInputArchive final : public AnimatedMapFilter,
                                         public BlendShapeFilter,
                                         public JointFilter,
                                         public MeshFilter,
                                         public terse::ExtendableBinaryInputArchive<FilteredBinaryInputArchive,
                                                                                    BoundedIOStream,
                                                                                    std::uint32_t,
                                                                                    std::uint32_t,
                                                                                    terse::Endianness::Network> {
private:
    using BaseArchive = terse::ExtendableBinaryInputArchive<FilteredBinaryInputArchive,
                                                            BoundedIOStream,
                                                            std::uint32_t,
                                                            std::uint32_t,
                                                            terse::Endianness::Network>;
    friend Archive<FilteredBinaryInputArchive>;
    friend BaseArchive;

public:
    FilteredBinaryInputArchive(BoundedIOStream* stream_,
                               DataLayer layer_,
                               LODConstraint&& lodConstraint,
                               MemoryResource* memRes_);

    bool isOk();

    std::size_t boundSize(std::size_t size);

private:
    void process(RawDescriptor& dest);
    void process(RawDefinition& dest);

    template<typename AnyVersion>
    void process(terse::Versioned<RawBehavior, AnyVersion>& dest) {
        if (malformed) {
            return;
        }

        if (contains(layerBitmask, DataLayerBitmask::Behavior)) {
            BaseArchive::process(std::move(dest));
        }
    }

    template<typename AnyVersion>
    void process(terse::Versioned<RawBehavior, AnyVersion>&& dest) {
        process(dest);
    }

    void process(RawControls& dest);
    void process(RawJoints& dest);
    void process(RawBlendShapeChannels& dest);
    void process(RawAnimatedMaps& dest);
    void process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v21>>& dest);
    void process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v21>>&& dest);
    void process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v22>>& dest);
    void process(terse::Versioned<RawGeometry, terse::Version<FileVersion::v22>>&& dest);

    template<typename AnyVersion>
    void process(terse::Versioned<RawGeometry, AnyVersion>& dest) {
        process(terse::versioned(dest.data, terse::Version<FileVersion::v22>{}));
    }

    template<typename AnyVersion>
    void process(terse::Versioned<RawGeometry, AnyVersion>&& dest) {
        process(dest);
    }

    void process(Vector<RawBlendShapeTarget>& dest);
    void process(RawVertexSkinWeights& dest);
    void process(RawMachineLearnedBehavior& dest);
    void process(RawRBFBehavior& dest);
    void process(RawRBFBehaviorExt& dest);
    void process(RawJointBehaviorMetadata& dest);
    void process(RawTwistSwingBehavior& dest);
    void process(RawMachineLearnedBehaviorExt& dest);

    void process(DNA& dest);
    void removeUnreferencedBlendShapes(DNA& dest);

    template<typename... Args>
    void process(Args&&... args) {
        BaseArchive::process(std::forward<Args>(args)...);
    }

    template<typename TContainer>
    void processSubset(TContainer& dest, std::size_t offset, std::size_t size);

private:
    BoundedIOStream* stream;
    MemoryResource* memRes;
    DataLayerBitmask layerBitmask;
    LODConstraint lodConstraint;
    Vector<bool> loadedControls;
    std::uint16_t unconstrainedLODCount;
    bool malformed;
};

}  // namespace dna
