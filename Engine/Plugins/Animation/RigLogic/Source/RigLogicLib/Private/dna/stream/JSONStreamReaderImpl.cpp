// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #ifdef _MSC_VER
        #pragma warning(disable : 4503)
    #endif

    #include "dna/stream/JSONStreamReaderImpl.h"

    #include "dna/TypeDefs.h"
    #include "dna/types/Limits.h"

    #include <status/Provider.h>
    #include <trio/utils/StreamScope.h>

    #ifdef _MSC_VER
        #pragma warning(push)
        #pragma warning(disable : 4365 4987)
    #endif
    #include <cstddef>
    #include <limits>
    #include <tuple>
    #ifdef _MSC_VER
        #pragma warning(pop)
    #endif

namespace dna {

JSONStreamReader::~JSONStreamReader() = default;

JSONStreamReader* JSONStreamReader::create(BoundedIOStream* stream, MemoryResource* memRes) {
    PolyAllocator<JSONStreamReaderImpl> alloc{memRes};
    return alloc.newObject(stream, memRes);
}

void JSONStreamReader::destroy(JSONStreamReader* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto reader = static_cast<JSONStreamReaderImpl*>(instance);
    PolyAllocator<JSONStreamReaderImpl> alloc{reader->getMemoryResource()};
    alloc.deleteObject(reader);
}

JSONStreamReaderImpl::JSONStreamReaderImpl(BoundedIOStream* stream_, MemoryResource* memRes_) :
    BaseImpl{UnknownLayerPolicy::Preserve, UpgradeFormatPolicy::Disallowed, memRes_},
    ReaderImpl{memRes_},
    stream{stream_},
    archive{stream_} {
}

void JSONStreamReaderImpl::unload(DataLayer layer) {
    if ((layer == DataLayer::All) || (layer == DataLayer::Descriptor)) {
        dna = DNA{dna.layers.unknownPolicy, dna.layers.upgradePolicy, memRes};
    } else if (layer == DataLayer::TwistSwingBehavior) {
        dna.unloadTwistSwingBehavior();
    } else if (layer == DataLayer::RBFBehavior) {
        dna.unloadRBFBehavior();
    } else if (layer == DataLayer::JointBehaviorMetadata) {
        dna.unloadJointBehaviorMetadata();
    } else if (layer == DataLayer::MachineLearnedBehavior) {
        dna.unloadMachineLearnedBehavior();
    } else if ((layer == DataLayer::Geometry) || (layer == DataLayer::GeometryWithoutBlendShapes)) {
        dna.unloadGeometry();
    } else if (layer == DataLayer::Behavior) {
        dna.unloadRBFBehavior();
        dna.unloadBehavior();
    } else if (layer == DataLayer::Definition) {
        dna.unloadJointBehaviorMetadata();
        dna.unloadTwistSwingBehavior();
        dna.unloadRBFBehavior();
        dna.unloadMachineLearnedBehavior();
        dna.unloadGeometry();
        dna.unloadBehavior();
        dna.unloadDefinition();
    }
}

void JSONStreamReaderImpl::read() {
    // Due to possible usage of custom stream implementations, the status actually must be cleared at this point
    // as external streams do not have access to the status reset API
    status.reset();

    trio::StreamScope scope{stream};
    if (!sc::Status::isOk()) {
        return;
    }

    archive >> dna;
    if (!archive.isOk()) {
        status.set(InvalidDataError);
        return;
    }

    if (!dna.signature.matches()) {
        status.set(SignatureMismatchError, dna.signature.value.expected.data(), dna.signature.value.got.data());
        return;
    }
    if (!dna.version.supported()) {
        status.set(VersionMismatchError, dna.version.generation, dna.version.version);
        return;
    }
}

}  // namespace dna

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
