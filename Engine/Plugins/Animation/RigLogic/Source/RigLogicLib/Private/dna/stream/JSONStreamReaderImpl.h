// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #include "dna/JSONStreamReader.h"
    #include "dna/ReaderImpl.h"
    #include "dna/TypeDefs.h"
    #include "dna/stream/StreamReaderStatus.h"

    #include <status/Provider.h>
    #include <terse/archives/json/InputArchive.h>

namespace dna {

class JSONStreamReaderImpl : public ReaderImpl<JSONStreamReader>, private StreamReaderStatus {
public:
    JSONStreamReaderImpl(BoundedIOStream* stream_, MemoryResource* memRes_);

    void unload(DataLayer layer) override;
    void read() override;

private:
    BoundedIOStream* stream;
    terse::JSONInputArchive<BoundedIOStream> archive;
};

}  // namespace dna

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
// *INDENT-ON*
