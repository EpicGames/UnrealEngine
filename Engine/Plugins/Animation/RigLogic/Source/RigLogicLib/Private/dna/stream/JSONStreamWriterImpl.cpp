// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #ifdef _MSC_VER
        #pragma warning(disable : 4503)
    #endif

    #include "dna/stream/JSONStreamWriterImpl.h"

    #include "dna/TypeDefs.h"
    #include "dna/stream/BinaryStreamReaderImpl.h"
    #include "dna/stream/JSONStreamReaderImpl.h"

    #ifdef _MSC_VER
        #pragma warning(push)
        #pragma warning(disable : 4365 4987)
    #endif
    #include <cassert>
    #include <cstddef>
    #include <cstring>
    #include <tuple>
    #include <utility>
    #ifdef _MSC_VER
        #pragma warning(pop)
    #endif

namespace dna {

JSONStreamWriter::~JSONStreamWriter() = default;

JSONStreamWriter* JSONStreamWriter::create(BoundedIOStream* stream, std::uint32_t indentWidth, MemoryResource* memRes) {
    PolyAllocator<JSONStreamWriterImpl> alloc{memRes};
    return alloc.newObject(stream, indentWidth, memRes);
}

void JSONStreamWriter::destroy(JSONStreamWriter* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto writer = static_cast<JSONStreamWriterImpl*>(instance);
    PolyAllocator<JSONStreamWriterImpl> alloc{writer->getMemoryResource()};
    alloc.deleteObject(writer);
}

JSONStreamWriterImpl::JSONStreamWriterImpl(BoundedIOStream* stream_, std::uint32_t indentWidth, MemoryResource* memRes_) :
    BaseImpl{UnknownLayerPolicy::Preserve, UpgradeFormatPolicy::Allowed, memRes_},
    WriterImpl{memRes_},
    stream{stream_},
    archive{stream_, indentWidth} {
}

void JSONStreamWriterImpl::write() {
    stream->open();
    archive << dna;
    archive.sync();
    stream->close();
}

void JSONStreamWriterImpl::setFrom(const BinaryStreamReader* source,
                                   DataLayer layer,
                                   UnknownLayerPolicy policy,
                                   MemoryResource* memRes_) {
    auto reader = const_cast<BinaryStreamReaderImpl*>(static_cast<const BinaryStreamReaderImpl*>(source));
    reader->rawCopyInto(dna, layer, policy, memRes_);
}

void JSONStreamWriterImpl::setFrom(const JSONStreamReader* source,
                                   DataLayer layer,
                                   UnknownLayerPolicy policy,
                                   MemoryResource* memRes_) {
    auto reader = const_cast<JSONStreamReaderImpl*>(static_cast<const JSONStreamReaderImpl*>(source));
    reader->rawCopyInto(dna, layer, policy, memRes_);
}

}  // namespace dna

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
