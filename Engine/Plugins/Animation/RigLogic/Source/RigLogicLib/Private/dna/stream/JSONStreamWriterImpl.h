// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #include "dna/DNA.h"
    #include "dna/JSONStreamWriter.h"
    #include "dna/WriterImpl.h"

    #include <terse/archives/json/OutputArchive.h>

namespace dna {

class BinaryStreamReader;
class JSONStreamReader;

class JSONStreamWriterImpl : public WriterImpl<JSONStreamWriter> {
public:
    JSONStreamWriterImpl(BoundedIOStream* stream_, std::uint32_t indentWidth, MemoryResource* memRes_);

    void write() override;

    using JSONStreamWriter::setFrom;
    void setFrom(const BinaryStreamReader* source, DataLayer layer, UnknownLayerPolicy policy, MemoryResource* memRes_) override;
    void setFrom(const JSONStreamReader* source, DataLayer layer, UnknownLayerPolicy policy, MemoryResource* memRes_) override;

private:
    BoundedIOStream* stream;
    terse::JSONOutputArchive<BoundedIOStream> archive;
};

}  // namespace dna

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
// *INDENT-ON*
