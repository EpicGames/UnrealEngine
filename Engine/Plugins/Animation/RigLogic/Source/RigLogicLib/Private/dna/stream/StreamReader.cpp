// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/StreamReader.h"

#include "dna/stream/StreamReaderStatus.h"

#include <status/Provider.h>

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
#endif

namespace dna {

StreamReader::~StreamReader() = default;

const sc::StatusCode StreamReader::SignatureMismatchError{200, "DNA signature mismatched, expected %.3s, got %.3s"};
const sc::StatusCode StreamReader::VersionMismatchError{201, "DNA version mismatched, got %hu.%hu"};
const sc::StatusCode StreamReader::InvalidDataError{202, "Invalid data in DNA"};

// Defined here so StreamReaderStatus.obj is no longer a separate link unit.
// StreamReader.obj is always pulled in by BinaryStreamReaderImpl.obj and
// JSONStreamReaderImpl.obj through their references to SignatureMismatchError
// et al., so status registration is guaranteed for any binary that uses a reader.
sc::StatusProvider StreamReaderStatus::status{StreamReader::SignatureMismatchError,
                                              StreamReader::VersionMismatchError,
                                              StreamReader::InvalidDataError};

}  // namespace dna

#ifdef __clang__
    #pragma clang diagnostic pop
#endif
