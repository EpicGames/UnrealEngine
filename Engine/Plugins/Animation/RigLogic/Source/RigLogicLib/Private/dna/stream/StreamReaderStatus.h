// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <status/Provider.h>

namespace dna {

struct StreamReaderStatus {
    static sc::StatusProvider status;
};

}  // namespace dna
