// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/StreamWriter.h"

namespace dna {

StreamWriter::~StreamWriter() = default;

}  // namespace dna
