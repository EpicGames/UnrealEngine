// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/types/CoordinateSystemConverter.h"

#include "dna/Configuration.h"
#include "dna/DNA.h"
#include "dna/layers/Twist.h"
#include "dna/utils/Extd.h"
#include "dna/utils/Macros.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cmath>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace dna {

static tdm::frad fromAngles(float angle, RotationUnit rotationUnit) {
    return (rotationUnit == RotationUnit::degrees) ? tdm::frad{tdm::fdeg{angle}} : tdm::frad{angle};
}

static float toAngles(tdm::frad angle, dna::RotationUnit rotationUnit) {
    return (rotationUnit == RotationUnit::degrees) ? tdm::fdeg{angle}.value : angle.value;
}

static void defragmentJointGroup(AlignedDynArray<float>& values,
                                 DynArray<std::uint16_t>& inputIndices,
                                 DynArray<std::uint16_t>& outputIndices,
                                 DynArray<std::uint16_t>& lods) {
    auto hasNonZeroDeltasInColumnUntilRow = [&](std::uint32_t col, std::uint32_t colCount, std::uint32_t rowCount) {
        for (std::uint32_t ri = {}; ri < rowCount; ++ri) {
            if (values[ri * colCount + col] != 0.0f) {
                return true;
            }
        }
        return false;
    };

    auto hasNonZeroDeltasInRowUntilColumn = [&](std::uint32_t row, std::uint32_t /*unused*/, std::uint32_t colCount) {
        for (std::uint32_t ci = {}; ci < colCount; ++ci) {
            if (values[row * colCount + ci] != 0.0f) {
                return true;
            }
        }
        return false;
    };

    auto deleteColumn = [&values, &inputIndices](std::uint32_t column) {
        const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        const auto rowCount = static_cast<std::uint32_t>(values.size() / colCount);
        std::uint32_t dstIndex = {};
        for (std::uint32_t ri = {}; ri < rowCount; ++ri) {
            for (std::uint32_t ci = {}; ci < colCount; ++ci) {
                if (ci != column) {
                    const auto i = ri * colCount + ci;
                    values[dstIndex++] = values[i];
                }
            }
        }
        values.resize(values.size() - rowCount);
        extd::filter(inputIndices, [column](std::uint16_t /*unused*/, std::size_t index) { return index != column; });
    };

    auto deleteRow = [&values, &inputIndices, &outputIndices](std::uint32_t row) {
        const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        for (std::uint32_t i = (row * colCount); i < (values.size() - colCount); ++i) {
            values[i] = values[i + colCount];
        }
        values.resize(values.size() - colCount);
        extd::filter(outputIndices, [row](std::uint16_t /*unused*/, std::size_t index) { return index != row; });
    };

    auto purgeEmptyColumns = [&]() {
        auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        const auto rowCount = static_cast<std::uint32_t>(values.size() / std::max(colCount, 1u));
        for (std::uint32_t col = (colCount - 1u); col != static_cast<std::uint32_t>(-1); --col) {
            if (!hasNonZeroDeltasInColumnUntilRow(col, colCount, rowCount)) {
                deleteColumn(col);
                --colCount;
            }
        }
    };

    auto purgeEmptyRows = [&]() {
        const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        auto rowCount = static_cast<std::uint32_t>(values.size() / std::max(colCount, 1u));
        for (std::uint32_t row = (rowCount - 1u); row != static_cast<std::uint32_t>(-1); --row) {
            if (!hasNonZeroDeltasInRowUntilColumn(row, rowCount, colCount)) {
                deleteRow(row);
                --rowCount;
                for (std::uint16_t lod = {}; lod < static_cast<std::uint16_t>(lods.size()); ++lod) {
                    if (row < lods[lod]) {
                        --lods[lod];
                    }
                }
            }
        }
    };

    purgeEmptyRows();
    purgeEmptyColumns();
}

CoordinateSystemConverter::CoordinateSystemConverter(const CoordinateSystem& dstCoordinateSystem_,
                                                     RotationSequence dstRotationSequence_,
                                                     const RotationSign& dstRotationSigns_,
                                                     FaceWindingOrder dstFaceWindingOrder_,
                                                     MemoryResource* memRes_) :
    dstCoordinateSystem{dstCoordinateSystem_},
    dstRotationSequence{dstRotationSequence_},
    dstRotationSigns{dstRotationSigns_},
    dstFaceWindingOrder{dstFaceWindingOrder_},
    memRes{memRes_} {
}

void CoordinateSystemConverter::convertNeutralJointTranslations(DNA& dna,
                                                                const CoordinateSystem& srcCoordinateSystem,
                                                                RotationSequence srcRotationSequence,
                                                                const RotationSign& srcRotationSigns,
                                                                FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcRotationSequence);
    UNUSED(srcRotationSigns);
    UNUSED(srcFaceWindingOrder);
    auto& xs = dna.definition.neutralJointTranslations.xs;
    auto& ys = dna.definition.neutralJointTranslations.ys;
    auto& zs = dna.definition.neutralJointTranslations.zs;
    const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);
    for (std::size_t jointIndex = {}; jointIndex < xs.size(); ++jointIndex) {
        const tdm::fvec3 srcTranslation{xs[jointIndex], ys[jointIndex], zs[jointIndex]};
        const tdm::fvec3 dstTranslation = tdm::convert_position(srcTranslation, changeOfBasis);
        xs[jointIndex] = dstTranslation[0];
        ys[jointIndex] = dstTranslation[1];
        zs[jointIndex] = dstTranslation[2];
    }
}

void CoordinateSystemConverter::convertNeutralJointRotations(DNA& dna,
                                                             const CoordinateSystem& srcCoordinateSystem,
                                                             RotationSequence srcRotationSequence,
                                                             const RotationSign& srcRotationSigns,
                                                             FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcFaceWindingOrder);
    const auto rotationUnit = static_cast<dna::RotationUnit>(dna.descriptor.rotationUnit);
    auto& xs = dna.definition.neutralJointRotations.xs;
    auto& ys = dna.definition.neutralJointRotations.ys;
    auto& zs = dna.definition.neutralJointRotations.zs;
    const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);
    for (std::size_t jointIndex = {}; jointIndex < xs.size(); ++jointIndex) {
        const tdm::frad3 src = {fromAngles(xs[jointIndex], rotationUnit),
                                fromAngles(ys[jointIndex], rotationUnit),
                                fromAngles(zs[jointIndex], rotationUnit)};
        const tdm::frad3 dst = tdm::convert_rotation(src,
                                                     changeOfBasis,
                                                     srcRotationSequence,
                                                     srcRotationSigns,
                                                     dstRotationSequence,
                                                     dstRotationSigns);
        xs[jointIndex] = toAngles(dst[0], rotationUnit);
        ys[jointIndex] = toAngles(dst[1], rotationUnit);
        zs[jointIndex] = toAngles(dst[2], rotationUnit);
    }
}

void CoordinateSystemConverter::convertJointDeltas(DNA& dna,
                                                   const CoordinateSystem& srcCoordinateSystem,
                                                   RotationSequence srcRotationSequence,
                                                   const RotationSign& srcRotationSigns,
                                                   FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcFaceWindingOrder);
    static constexpr auto jointAttrCount = 9ul;
    const auto jointGroupCount = static_cast<std::uint16_t>(dna.behavior.joints.jointGroups.size());
    const auto rotationUnit = static_cast<dna::RotationUnit>(dna.descriptor.rotationUnit);

    for (std::uint16_t jgIndex = {}; jgIndex < jointGroupCount; ++jgIndex) {
        const auto& originalInputIndices = dna.behavior.joints.jointGroups[jgIndex].inputIndices;
        const auto& originalOutputIndices = dna.behavior.joints.jointGroups[jgIndex].outputIndices;
        const auto& originalLODs = dna.behavior.joints.jointGroups[jgIndex].lods;
        const auto& originalValues = dna.behavior.joints.jointGroups[jgIndex].values;
        const auto colCount = originalInputIndices.size();
        const auto rowCount = originalOutputIndices.size();
        const auto jointCount = [&]() {
            UnorderedSet<std::uint16_t> jointIndices{memRes};
            for (std::size_t row = {}; row < rowCount; ++row) {
                jointIndices.insert(static_cast<std::uint16_t>(originalOutputIndices[row] / jointAttrCount));
            }
            return jointIndices.size();
        }();

        dna::DynArray<std::uint16_t> newLODs{originalLODs.size(), {}, memRes};
        dna::DynArray<std::uint16_t> newInputIndices{originalInputIndices.size(), {}, memRes};
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
        dna::DynArray<std::uint16_t> newOutputIndices{static_cast<std::size_t>(jointCount * jointAttrCount), {}, memRes};
        dna::AlignedDynArray<float> newValues{static_cast<std::size_t>(jointCount * jointAttrCount * colCount), {}, memRes};
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        newInputIndices.assign(originalInputIndices.begin(), originalInputIndices.end());

        const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);
        for (std::size_t col = {}; col < colCount; ++col) {
            std::size_t newRowIndex = {};
            for (std::size_t row = {}; row < rowCount && newRowIndex < newOutputIndices.size();) {
                tdm::fvec3 srcTranslation;
                tdm::frad3 srcRotation;
                tdm::fvec3 srcScale;
                const std::size_t jointIndex = originalOutputIndices[row] / jointAttrCount;
                while ((row < rowCount) && (jointIndex == (originalOutputIndices[row] / jointAttrCount))) {
                    const auto relAttrIndex = static_cast<std::uint16_t>(originalOutputIndices[row] % jointAttrCount);
                    const auto componentIndex = static_cast<std::uint16_t>(relAttrIndex % 3u);
                    if (relAttrIndex < 3) {
                        srcTranslation[componentIndex] = originalValues[row * colCount + col];
                    } else if (relAttrIndex < 6) {
                        srcRotation[componentIndex] = fromAngles(originalValues[row * colCount + col], rotationUnit);
                    } else {
                        srcScale[componentIndex] = originalValues[row * colCount + col];
                    }
                    ++row;
                }

                const tdm::fvec3 dstTranslation = tdm::convert_position(srcTranslation, changeOfBasis);
                const tdm::frad3 dstRotation = tdm::convert_rotation(srcRotation,
                                                                     changeOfBasis,
                                                                     srcRotationSequence,
                                                                     srcRotationSigns,
                                                                     dstRotationSequence,
                                                                     dstRotationSigns);
                const tdm::fvec3 dstScale = tdm::convert_scale(srcScale, changeOfBasis);

                newValues[(newRowIndex + 0) * colCount + col] = dstTranslation[0];
                newValues[(newRowIndex + 1) * colCount + col] = dstTranslation[1];
                newValues[(newRowIndex + 2) * colCount + col] = dstTranslation[2];

                newValues[(newRowIndex + 3) * colCount + col] = toAngles(dstRotation[0], rotationUnit);
                newValues[(newRowIndex + 4) * colCount + col] = toAngles(dstRotation[1], rotationUnit);
                newValues[(newRowIndex + 5) * colCount + col] = toAngles(dstRotation[2], rotationUnit);

                newValues[(newRowIndex + 6) * colCount + col] = dstScale[0];
                newValues[(newRowIndex + 7) * colCount + col] = dstScale[1];
                newValues[(newRowIndex + 8) * colCount + col] = dstScale[2];

                // TODO: This is wasteful to do for each column as it's the same
                for (std::uint16_t ri = {}; ri < jointAttrCount; ++ri) {
                    newOutputIndices[newRowIndex + ri] = static_cast<std::uint16_t>(jointIndex * jointAttrCount + ri);
                }

                newRowIndex += jointAttrCount;
            }
        }

        newLODs[0] = static_cast<std::uint16_t>(newOutputIndices.size());
        // This logic is dependent on the joint group containing output indices in joint order, i.e. no joints are split in half
        // within a joint group.
        for (std::uint16_t li = 1; li < newLODs.size(); ++li) {
            const std::uint16_t originalRowCount = originalLODs[li];
            if (originalRowCount > 0) {
                const auto jointIndex = originalOutputIndices[static_cast<std::uint16_t>(originalRowCount - 1)] / jointAttrCount;
                // Find position of last attribute
                const auto lastAttrPos = std::distance(newOutputIndices.begin(),
                                                       std::find(newOutputIndices.begin(),
                                                                 newOutputIndices.end(),
                                                                 jointIndex * jointAttrCount + (jointAttrCount - 1)));
                newLODs[li] = static_cast<std::uint16_t>(lastAttrPos + 1);
            }
        }

        defragmentJointGroup(newValues, newInputIndices, newOutputIndices, newLODs);
        dna.behavior.joints.jointGroups[jgIndex].values = std::move(newValues);
        dna.behavior.joints.jointGroups[jgIndex].inputIndices = std::move(newInputIndices);
        dna.behavior.joints.jointGroups[jgIndex].outputIndices = std::move(newOutputIndices);
        dna.behavior.joints.jointGroups[jgIndex].lods = std::move(newLODs);
    }
}

void CoordinateSystemConverter::convertVertexPositions(DNA& dna,
                                                       const CoordinateSystem& srcCoordinateSystem,
                                                       RotationSequence srcRotationSequence,
                                                       const RotationSign& srcRotationSigns,
                                                       FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcRotationSequence);
    UNUSED(srcRotationSigns);
    UNUSED(srcFaceWindingOrder);
    const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);
    for (std::size_t meshIndex = {}; meshIndex < dna.geometry.meshes.size(); ++meshIndex) {
        auto& vertices = dna.geometry.meshes[meshIndex].positions;
        for (std::size_t vertexIndex = {}; vertexIndex < vertices.xs.size(); ++vertexIndex) {
            const tdm::fvec3 srcPosition{vertices.xs[vertexIndex], vertices.ys[vertexIndex], vertices.zs[vertexIndex]};
            const tdm::fvec3 dstPosition = tdm::convert_position(srcPosition, changeOfBasis);
            vertices.xs[vertexIndex] = dstPosition[0];
            vertices.ys[vertexIndex] = dstPosition[1];
            vertices.zs[vertexIndex] = dstPosition[2];
        }
    }
}

void CoordinateSystemConverter::convertVertexNormals(DNA& dna,
                                                     const CoordinateSystem& srcCoordinateSystem,
                                                     RotationSequence srcRotationSequence,
                                                     const RotationSign& srcRotationSigns,
                                                     FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcRotationSequence);
    UNUSED(srcRotationSigns);
    UNUSED(srcFaceWindingOrder);
    const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);
    for (std::size_t meshIndex = {}; meshIndex < dna.geometry.meshes.size(); ++meshIndex) {
        auto& normals = dna.geometry.meshes[meshIndex].normals;
        for (std::size_t normalIndex = {}; normalIndex < normals.xs.size(); ++normalIndex) {
            const tdm::fvec3 srcNormal{normals.xs[normalIndex], normals.ys[normalIndex], normals.zs[normalIndex]};
            const tdm::fvec3 dstNormal = tdm::convert_direction(srcNormal, changeOfBasis, false);
            normals.xs[normalIndex] = dstNormal[0];
            normals.ys[normalIndex] = dstNormal[1];
            normals.zs[normalIndex] = dstNormal[2];
        }
    }
}

void CoordinateSystemConverter::convertBlendShapeDeltas(DNA& dna,
                                                        const CoordinateSystem& srcCoordinateSystem,
                                                        RotationSequence srcRotationSequence,
                                                        const RotationSign& srcRotationSigns,
                                                        FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcRotationSequence);
    UNUSED(srcRotationSigns);
    UNUSED(srcFaceWindingOrder);
    const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);
    for (std::size_t meshIndex = {}; meshIndex < dna.geometry.meshes.size(); ++meshIndex) {
        auto& mesh = dna.geometry.meshes[meshIndex];
        for (std::size_t bsTargetIndex = {}; bsTargetIndex < mesh.blendShapeTargets.size(); ++bsTargetIndex) {
            auto& deltas = mesh.blendShapeTargets[bsTargetIndex].deltas;
            for (std::size_t deltaIndex = {}; deltaIndex < deltas.xs.size(); ++deltaIndex) {
                const tdm::fvec3 srcDelta{deltas.xs[deltaIndex], deltas.ys[deltaIndex], deltas.zs[deltaIndex]};
                const tdm::fvec3 dstDelta = tdm::convert_position(srcDelta, changeOfBasis);
                deltas.xs[deltaIndex] = dstDelta[0];
                deltas.ys[deltaIndex] = dstDelta[1];
                deltas.zs[deltaIndex] = dstDelta[2];
            }
        }
    }
}

void CoordinateSystemConverter::convertRBFSolverRawControlValues(DNA& dna,
                                                                 const CoordinateSystem& srcCoordinateSystem,
                                                                 RotationSequence srcRotationSequence,
                                                                 const RotationSign& srcRotationSigns,
                                                                 FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcRotationSequence);
    UNUSED(srcRotationSigns);
    UNUSED(srcFaceWindingOrder);
    const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);
    // Conjugate the pose quaternions directly: q' = (det(C) * (v * C), w).
    // Quaternions carry no rotation sequence or sign convention, so no euler roundtrip is needed or allowed:
    // extracting euler angles destroys the q vs -q distinction that "half rotation" RBF solvers rely on to track driver rotations
    // beyond 180 degrees.
    const float chirality = tdm::determinant(changeOfBasis);
    for (std::size_t solverIndex = {}; solverIndex < dna.rbfBehavior.solvers.size(); ++solverIndex) {
        auto& rawControlValues = dna.rbfBehavior.solvers[solverIndex].rawControlValues;
        if (rawControlValues.size() % 4ul == 0ul) {
            for (std::size_t offset = {}; offset < rawControlValues.size(); offset += 4ul) {
                const tdm::fvec3 srcImaginary{rawControlValues[offset + 0ul],
                                              rawControlValues[offset + 1ul],
                                              rawControlValues[offset + 2ul]};
                const tdm::fvec3 dstImaginary = tdm::convert_direction(srcImaginary, changeOfBasis, false) * chirality;
                rawControlValues[offset + 0] = dstImaginary[0];
                rawControlValues[offset + 1] = dstImaginary[1];
                rawControlValues[offset + 2] = dstImaginary[2];
            }
        }
    }
}

void CoordinateSystemConverter::convertTwistAxes(DNA& dna,
                                                 const CoordinateSystem& srcCoordinateSystem,
                                                 RotationSequence srcRotationSequence,
                                                 const RotationSign& srcRotationSigns,
                                                 FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcRotationSequence);
    UNUSED(srcRotationSigns);
    UNUSED(srcFaceWindingOrder);

    const tdm::fmat3 changeOfBasis = tdm::change_of_basis<float>(srcCoordinateSystem, dstCoordinateSystem);

    auto remapTwistAxis = [&changeOfBasis](TwistAxis srcTwistAxis) {
        if (srcTwistAxis > TwistAxis::Z) {
            return srcTwistAxis;
        }
        tdm::fvec3 srcAxis{};
        srcAxis[static_cast<tdm::dim_t>(srcTwistAxis)] = 1.0f;
        const tdm::fvec3 dstAxis = tdm::convert_direction(srcAxis, changeOfBasis, false);
        tdm::dim_t dominant = {};
        for (tdm::dim_t ci = 1u; ci < 3u; ++ci) {
            if (std::fabs(dstAxis[ci]) > std::fabs(dstAxis[dominant])) {
                dominant = ci;
            }
        }
        return static_cast<TwistAxis>(dominant);
    };

    for (auto& solver : dna.rbfBehavior.solvers) {
        solver.twistAxis = static_cast<std::uint16_t>(remapTwistAxis(static_cast<TwistAxis>(solver.twistAxis)));
    }

    for (auto& twist : dna.twistSwingBehavior.twists) {
        twist.twistAxis = static_cast<std::uint16_t>(remapTwistAxis(static_cast<TwistAxis>(twist.twistAxis)));
    }

    for (auto& swing : dna.twistSwingBehavior.swings) {
        swing.twistAxis = static_cast<std::uint16_t>(remapTwistAxis(static_cast<TwistAxis>(swing.twistAxis)));
    }
}

void CoordinateSystemConverter::convertFaceWinding(DNA& dna,
                                                   const CoordinateSystem& srcCoordinateSystem,
                                                   RotationSequence srcRotationSequence,
                                                   const RotationSign& srcRotationSigns,
                                                   FaceWindingOrder srcFaceWindingOrder) {
    UNUSED(srcRotationSequence);
    UNUSED(srcRotationSigns);
    // Flip face winding if the source winding (reinterpreted in destination space) doesn't match
    // what is requested by dstFaceWindingOrder.
    // The source DNA carries faceWindingOrder metadata describing its winding convention as
    // CCW/CW viewed from the outward normal in src space. After the change-of-basis is applied,
    // an orientation-reversing basis (different handedness between src and dst) flips the
    // operational meaning of "outward normal" relative to face vertex order: CCW-from-outward
    // in src becomes CW-from-outward in dst, and vice versa. Pure axis permutations that keep
    // handedness preserve the meaning. We compute what the source winding "looks like" in dst
    // space and compare against the consumer's requested winding.
    const auto invertWinding = [](FaceWindingOrder fwo) {
        return (fwo == FaceWindingOrder::ccw) ? FaceWindingOrder::cw : FaceWindingOrder::ccw;
    };
    const bool basisPreservesOrientation = (srcCoordinateSystem.handedness<float>() == dstCoordinateSystem.handedness<float>());
    const FaceWindingOrder srcWindingInDst = basisPreservesOrientation ? srcFaceWindingOrder : invertWinding(srcFaceWindingOrder);
    if (srcWindingInDst != dstFaceWindingOrder) {
        for (auto& mesh : dna.geometry.meshes) {
            for (auto& face : mesh.faces) {
                std::reverse(face.layoutIndices.begin(), face.layoutIndices.end());
            }
        }
    }
}

void CoordinateSystemConverter::convert(DNA& dna) {
    const auto srcX = dna.descriptor.coordinateSystem.xAxis;
    const auto srcY = dna.descriptor.coordinateSystem.yAxis;
    const auto srcZ = dna.descriptor.coordinateSystem.zAxis;
    const tdm::coord_sys& srcCoordinateSystem = {static_cast<tdm::axis_dir>(srcX),
                                                 static_cast<tdm::axis_dir>(srcY),
                                                 static_cast<tdm::axis_dir>(srcZ)};
    const tdm::rot_seq srcRotationSequence = dna.descriptorExt.rotationSequence;
    const tdm::rot_sign srcRotationSigns = dna.descriptorExt.rotationSign;
    const FaceWindingOrder srcFaceWindingOrder = dna.descriptorExt.faceWindingOrder;

    convertNeutralJointTranslations(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertNeutralJointRotations(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertJointDeltas(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertVertexPositions(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertVertexNormals(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertBlendShapeDeltas(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertRBFSolverRawControlValues(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertTwistAxes(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);
    convertFaceWinding(dna, srcCoordinateSystem, srcRotationSequence, srcRotationSigns, srcFaceWindingOrder);

    dna.descriptor.coordinateSystem.xAxis = static_cast<std::uint16_t>(dstCoordinateSystem.x);
    dna.descriptor.coordinateSystem.yAxis = static_cast<std::uint16_t>(dstCoordinateSystem.y);
    dna.descriptor.coordinateSystem.zAxis = static_cast<std::uint16_t>(dstCoordinateSystem.z);
    dna.descriptorExt.rotationSequence = dstRotationSequence;
    dna.descriptorExt.rotationSign = dstRotationSigns;
    dna.descriptorExt.faceWindingOrder = dstFaceWindingOrder;

    // The v2.1 monolithic format carries no layer index, but always contains these four
    // sections. As the version upgrade below switches the DNA to the layered format, which
    // writes only indexed layers when format upgrades are disallowed (e.g. during raw-copy),
    // index them explicitly so they don't get silently dropped.
    if (dna.version.matches(FileVersion::v21)) {
        dna.ensureLayerIndexed(dna.descriptor.layerId(), dna.descriptor.layerVersion());
        dna.ensureLayerIndexed(dna.definition.layerId(), dna.definition.layerVersion());
        dna.ensureLayerIndexed(dna.behavior.layerId(), dna.behavior.layerVersion());
        dna.ensureLayerIndexed(dna.geometry.layerId(), dna.geometry.layerVersion());
    }

    // The faceWindingOrder field was added in v28; any DNA written by this converter must be
    // at least v28 so that the updated winding convention is serialized.
    constexpr std::uint16_t minDNAGeneration = high16(static_cast<std::uint32_t>(FileVersion::v28));
    constexpr std::uint16_t minDNAVersion = low16(static_cast<std::uint32_t>(FileVersion::v28));
    if ((dna.version.generation == minDNAGeneration) && (dna.version.version < minDNAVersion)) {
        // File format must be upgraded to 2.8 at least, since that version introduces the
        // faceWindingOrder field on DescriptorExt.
        dna.version.version = minDNAVersion;
    }

    // The converted rotation and winding conventions are stored in the DescriptorExt layer,
    // which sources older than v2.7 don't have in their layer index. Index it explicitly, so
    // serialization with UpgradeFormatPolicy::Disallowed (e.g. during raw-copy) preserves it.
    dna.ensureLayerIndexed(dna.descriptorExt.layerId(), dna.descriptorExt.layerVersion());
}

}  // namespace dna
