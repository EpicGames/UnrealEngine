// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"

namespace dna {

struct DNA;

class CoordinateSystemConverter {
public:
    CoordinateSystemConverter(const CoordinateSystem& dstCoordinateSystem_,
                              RotationSequence dstRotationSequence_,
                              const RotationSign& dstRotationSigns_,
                              FaceWindingOrder dstFaceWindingOrder_,
                              MemoryResource* memRes_);
    void convert(DNA& dna);

private:
    void convertNeutralJointTranslations(DNA& dna,
                                         const CoordinateSystem& srcCoordinateSystem,
                                         RotationSequence srcRotationSequence,
                                         const RotationSign& srcRotationSigns,
                                         FaceWindingOrder srcFaceWindingOrder);
    void convertNeutralJointRotations(DNA& dna,
                                      const CoordinateSystem& srcCoordinateSystem,
                                      RotationSequence srcRotationSequence,
                                      const RotationSign& srcRotationSigns,
                                      FaceWindingOrder srcFaceWindingOrder);
    void convertJointDeltas(DNA& dna,
                            const CoordinateSystem& srcCoordinateSystem,
                            RotationSequence srcRotationSequence,
                            const RotationSign& srcRotationSigns,
                            FaceWindingOrder srcFaceWindingOrder);
    void convertVertexPositions(DNA& dna,
                                const CoordinateSystem& srcCoordinateSystem,
                                RotationSequence srcRotationSequence,
                                const RotationSign& srcRotationSigns,
                                FaceWindingOrder srcFaceWindingOrder);
    void convertVertexNormals(DNA& dna,
                              const CoordinateSystem& srcCoordinateSystem,
                              RotationSequence srcRotationSequence,
                              const RotationSign& srcRotationSigns,
                              FaceWindingOrder srcFaceWindingOrder);
    void convertBlendShapeDeltas(DNA& dna,
                                 const CoordinateSystem& srcCoordinateSystem,
                                 RotationSequence srcRotationSequence,
                                 const RotationSign& srcRotationSigns,
                                 FaceWindingOrder srcFaceWindingOrder);
    void convertRBFSolverRawControlValues(DNA& dna,
                                          const CoordinateSystem& srcCoordinateSystem,
                                          RotationSequence srcRotationSequence,
                                          const RotationSign& srcRotationSigns,
                                          FaceWindingOrder srcFaceWindingOrder);
    void convertTwistAxes(DNA& dna,
                          const CoordinateSystem& srcCoordinateSystem,
                          RotationSequence srcRotationSequence,
                          const RotationSign& srcRotationSigns,
                          FaceWindingOrder srcFaceWindingOrder);
    void convertFaceWinding(DNA& dna,
                            const CoordinateSystem& srcCoordinateSystem,
                            RotationSequence srcRotationSequence,
                            const RotationSign& srcRotationSigns,
                            FaceWindingOrder srcFaceWindingOrder);

private:
    CoordinateSystem dstCoordinateSystem;
    RotationSequence dstRotationSequence;
    RotationSign dstRotationSigns;
    FaceWindingOrder dstFaceWindingOrder;
    MemoryResource* memRes;
};

}  // namespace dna
