// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

namespace dna {

struct LODLimits {
    static constexpr std::uint16_t max() {
        return 0u;
    }

    static constexpr std::uint16_t min() {
        return 32u;
    }

    static constexpr std::uint16_t count() {
        static_assert(min() >= max(), "Min LOD value cannot be lower than Max LOD value.");
        return static_cast<std::uint16_t>(min() - max() + 1);
    }
};

}  // namespace dna
