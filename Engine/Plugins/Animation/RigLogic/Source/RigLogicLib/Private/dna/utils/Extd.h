// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#ifndef EXTD_GUARD
    #define EXTD_GUARD

    #include <arrayview/ArrayView.h>

    #ifdef _MSC_VER
        #pragma warning(push)
        #pragma warning(disable : 4365 4987)
    #endif
    #include <algorithm>
    #include <cstddef>
    #include <functional>
    #include <iterator>
    #include <set>
    #if defined(_MSC_VER) && !defined(__clang__) && (_MSC_VER < 1938) && (_MSC_VER >= 1900) && (__cplusplus >= 202002L)
        #include <span>
    #endif
    #include <vector>
    #ifdef _MSC_VER
        #pragma warning(pop)
    #endif

namespace extd {

template<typename T>
inline T clamp(T value, T low, T high) {
    return std::min(std::max(value, low), high);
}

template<typename T>
inline T roundUp(T number, T multiple) {
    return static_cast<T>(((number + multiple - 1) / multiple) * multiple);
}

template<typename T>
inline T interpolate(T a, T b, T weight) {
    return a * (static_cast<T>(1) - weight) + b * weight;
}

template<class TInputIterator, class T>
inline bool contains(TInputIterator first, TInputIterator last, const T& value) {
    return std::find(first, last, value) != last;
}

template<class TContainer, class T>
inline bool contains(const TContainer& container, const T& value) {
    return contains(std::begin(container), std::end(container), value);
}

template<class T>
inline bool contains(const std::set<T>& container, const T& value) {
    return container.find(value) != container.end();
}

template<class T, class Predicate, typename... Args>
inline void filter(std::vector<T, Args...>& source, Predicate pred) {
    source.erase(std::remove_if(std::begin(source),
                                std::end(source),
                                [&source, &pred](const T& value) {
                                    const auto index = static_cast<std::size_t>(&value - &(source.front()));
                                    return !pred(value, index);
                                }),
                 source.end());
}

template<class TContainer, class Predicate, typename... Args>
inline void filter(TContainer& source, Predicate pred) {
    using value_type = typename TContainer::value_type;
    auto newEnd = std::remove_if(std::begin(source), std::end(source), [&source, &pred](const value_type& value) {
        const auto index = static_cast<std::size_t>(&value - source.data());
        return !pred(value, index);
    });
    const auto newSize = static_cast<std::size_t>(std::distance(source.begin(), newEnd));
    source.resize(newSize);
}

namespace impl {

enum class LUTStrategy {
    ByValue,
    ByPosition
};

template<typename TLUT, LUTStrategy Strategy>
class LUTFilter {
public:
    explicit LUTFilter(const TLUT& lut_) :
        lut{lut_} {
    }

    template<typename T, LUTStrategy ChosenStrategy = Strategy>
    typename std::enable_if<ChosenStrategy == LUTStrategy::ByValue, bool>::type operator()(const T& value,
                                                                                           std::size_t /*unused*/) {
        return contains(lut, value);
    }

    template<typename T, LUTStrategy ChosenStrategy = Strategy>
    typename std::enable_if<ChosenStrategy == LUTStrategy::ByPosition, bool>::type operator()(const T& /*unused*/,
                                                                                              std::size_t index) {
        return contains(lut, index);
    }

private:
    const TLUT& lut;
};

}  // namespace impl

template<typename TLookUpTable>
inline impl::LUTFilter<TLookUpTable, impl::LUTStrategy::ByValue> byValue(const TLookUpTable& lookUpTable) {
    return impl::LUTFilter<TLookUpTable, impl::LUTStrategy::ByValue>{lookUpTable};
}

template<typename TLookUpTable>
inline impl::LUTFilter<TLookUpTable, impl::LUTStrategy::ByPosition> byPosition(const TLookUpTable& lookUpTable) {
    return impl::LUTFilter<TLookUpTable, impl::LUTStrategy::ByPosition>{lookUpTable};
}

template<class TContainer, class TComparator>
inline typename TContainer::value_type minOf(const TContainer& container, TComparator&& comparator) {
    using ValueType = typename TContainer::value_type;
    const auto it = std::min_element(container.begin(), container.end(), comparator);
    return (it == container.end() ? ValueType{} : *it);
}

template<class TContainer>
inline typename TContainer::value_type minOf(const TContainer& container) {
    using ValueType = typename TContainer::value_type;
    const auto compare = [](const ValueType& lhs, const ValueType& rhs) { return lhs < rhs; };
    return minOf(container, compare);
}

template<class TContainer, class TComparator>
inline typename TContainer::value_type maxOf(const TContainer& container, TComparator&& comparator) {
    using ValueType = typename TContainer::value_type;
    const auto it = std::max_element(container.begin(), container.end(), comparator);
    return (it == container.end() ? ValueType{} : *it);
}

template<class TContainer>
inline typename TContainer::value_type maxOf(const TContainer& container) {
    using ValueType = typename TContainer::value_type;
    const auto compare = [](const ValueType& lhs, const ValueType& rhs) { return lhs < rhs; };
    return maxOf(container, compare);
}

template<class TSource, class TDestination>
inline void copy(const TSource& source, TDestination& destination) {
    std::copy(std::begin(source), std::end(source), std::back_inserter(destination));
}

template<class T, class U>
inline void copy(av::ArrayView<T> source, av::ArrayView<U> destination) {
    #if defined(_MSC_VER) && !defined(__clang__) && (_MSC_VER < 1938)
        #if (_MSC_VER >= 1900) && (__cplusplus >= 202002L)
    std::copy(source.begin(), source.end(), std::span{destination.data(), destination.size()}.begin());
        #else
    if ((source.data() != nullptr) || (destination.data() != nullptr)) {
        std::copy(source.begin(), source.end(), stdext::make_checked_array_iterator(destination.data(), destination.size()));
    }
        #endif
    #else
    std::copy(source.begin(), source.end(), destination.begin());
    #endif
}

template<class TIterator, typename TDistance>
TIterator advanced(TIterator source, TDistance distance) {
    std::advance(source, static_cast<typename std::iterator_traits<TIterator>::difference_type>(distance));
    return source;
}

template<class TIterator, class Predicate>
typename std::iterator_traits<TIterator>::difference_type advanceWhile(TIterator& it, const TIterator& end, Predicate pred) {
    const auto start = it;
    while (it != end && pred(*it)) {
        ++it;
    }
    return std::distance(start, it);
}

}  // namespace extd

#endif  // EXTD_GUARD
// *INDENT-ON*
