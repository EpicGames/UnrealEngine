// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(FORCE_INLINE)
    #if defined(_MSC_VER)
        #define FORCE_INLINE __forceinline
    #else
        #define FORCE_INLINE inline __attribute__((always_inline))
    #endif
#endif

#if !defined(ASSUME_TRUE)
    #if defined(_MSC_VER) && !defined(__clang__)
        #define ASSUME_TRUE(condition) __assume(condition)
    #else
        #define ASSUME_TRUE(condition)                                                                                           \
            do {                                                                                                                 \
                if (!(condition))                                                                                                \
                    __builtin_unreachable();                                                                                     \
            } while (0)
    #endif
#endif

#if !defined(UNUSED)
    #define UNUSED(x) static_cast<void>(x)
#endif
