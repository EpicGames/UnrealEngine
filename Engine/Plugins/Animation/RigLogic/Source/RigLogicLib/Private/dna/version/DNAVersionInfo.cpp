// Copyright Epic Games, Inc. All Rights Reserved.

#include "dna/version/VersionInfo.h"

#include "dna/version/Version.h"

#include <cstring>

namespace dna {

namespace {

constexpr int majorVersion = DNA_MAJOR_VERSION;
constexpr int minorVersion = DNA_MINOR_VERSION;
constexpr int patchVersion = DNA_PATCH_VERSION;
constexpr const char* versionString = DNA_VERSION_STRING;

}  // namespace

int VersionInfo::getMajorVersion() {
    return majorVersion;
}

int VersionInfo::getMinorVersion() {
    return minorVersion;
}

int VersionInfo::getPatchVersion() {
    return patchVersion;
}

StringView VersionInfo::getVersionString() {
    return {versionString, std::strlen(versionString)};
}

}  // namespace dna
