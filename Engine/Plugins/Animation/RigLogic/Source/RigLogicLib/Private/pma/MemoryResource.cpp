// Copyright Epic Games, Inc. All Rights Reserved.

#include "pma/MemoryResource.h"

namespace pma {

MemoryResource::~MemoryResource() = default;

}  // namespace pma
