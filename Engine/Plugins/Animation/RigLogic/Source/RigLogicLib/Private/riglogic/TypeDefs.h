// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <pma/PolyAllocator.h>
#include <pma/TypeDefs.h>
#include <pma/resources/AlignedMemoryResource.h>
#include <pma/resources/DefaultMemoryResource.h>
#include <pma/utils/ManagedInstance.h>
#include <tdm/TDM.h>
#include <terse/archives/binary/InputArchive.h>
#include <terse/archives/binary/OutputArchive.h>
#include <terse/utils/VirtualSerializerProxy.h>

#include <cstddef>

namespace tdm {

#ifndef TDM_FMAT3_SERIALIZER_DEFINED
    #define TDM_FMAT3_SERIALIZER_DEFINED

    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wattributes"
    #endif
template<typename Archive>
void load(Archive& archive, tdm::fmat3& value) {
    archive.label("value");
    value.apply([archive](tdm::fmat3::row_type& row, tdm::dim_t ri) mutable {
        row.apply([archive, ri](tdm::fmat3::value_type& v, tdm::dim_t ci) mutable {
            RL_UNUSED(ri);
            RL_UNUSED(ci);
            archive(v);
        });
    });
}

template<typename Archive>
void save(Archive& archive, tdm::fmat3& value) {
    archive.label("value");
    value.apply([archive](tdm::fmat3::row_type& row, tdm::dim_t ri) mutable {
        row.apply([archive, ri](tdm::fmat3::value_type& v, tdm::dim_t ci) mutable {
            RL_UNUSED(ri);
            RL_UNUSED(ci);
            archive(v);
        });
    });
}

    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif

#endif  // TDM_FMAT3_SERIALIZER_DEFINED

#ifndef TDM_COORD_SYS_SERIALIZER_DEFINED
    #define TDM_COORD_SYS_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::coord_sys& value) {
    std::uint32_t tmp = {};

    archive.label("x");
    archive(tmp);
    value.x = static_cast<tdm::axis_dir>(tmp);

    archive.label("y");
    archive(tmp);
    value.y = static_cast<tdm::axis_dir>(tmp);

    archive.label("z");
    archive(tmp);
    value.z = static_cast<tdm::axis_dir>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::coord_sys& value) {
    archive.label("x");
    archive(static_cast<std::uint32_t>(value.x));

    archive.label("y");
    archive(static_cast<std::uint32_t>(value.y));

    archive.label("z");
    archive(static_cast<std::uint32_t>(value.z));
}

#endif  // TDM_COORD_SYS_SERIALIZER_DEFINED

#ifndef TDM_ROT_SEQ_SERIALIZER_DEFINED
    #define TDM_ROT_SEQ_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::rot_seq& value) {
    archive.label("value");
    std::uint32_t tmp = {};
    archive(tmp);
    value = static_cast<tdm::rot_seq>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::rot_seq& value) {
    archive.label("value");
    archive(static_cast<std::uint32_t>(value));
}

#endif  // TDM_ROT_SEQ_SERIALIZER_DEFINED

#ifndef TDM_ROT_SIGN_SERIALIZER_DEFINED
    #define TDM_ROT_SIGN_SERIALIZER_DEFINED

template<typename Archive>
void load(Archive& archive, tdm::rot_sign& value) {
    std::uint32_t tmp = {};

    archive.label("x");
    archive(tmp);
    value.x = static_cast<tdm::rot_dir>(tmp);

    archive.label("y");
    archive(tmp);
    value.y = static_cast<tdm::rot_dir>(tmp);

    archive.label("z");
    archive(tmp);
    value.z = static_cast<tdm::rot_dir>(tmp);
}

template<typename Archive>
void save(Archive& archive, tdm::rot_sign& value) {
    archive.label("x");
    archive(static_cast<std::uint32_t>(value.x));

    archive.label("y");
    archive(static_cast<std::uint32_t>(value.y));

    archive.label("z");
    archive(static_cast<std::uint32_t>(value.z));
}

#endif  // TDM_ROT_SIGN_SERIALIZER_DEFINED
}  // namespace tdm

namespace rl4 {

using namespace pma;

static constexpr std::size_t cacheLineAlignment = 64ul;

template<typename T>
using AlignedAllocator = PolyAllocator<T, cacheLineAlignment, AlignedMemoryResource>;

template<typename T>
using AlignedVector = Vector<T, AlignedAllocator<T>>;

template<typename T>
using AlignedMatrix = Vector<AlignedVector<T>>;

}  // namespace rl4
