// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class AnimatedMaps {
public:
    using Pointer = UniqueInstance<AnimatedMaps>::PointerType;

protected:
    virtual ~AnimatedMaps();

public:
    virtual AnimatedMapsOutputInstance::Pointer createInstance(MemoryResource* memRes) const = 0;
    virtual ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t /*unused*/) const = 0;
    virtual void calculate(const ControlsInputInstance* inputs, AnimatedMapsOutputInstance* outputs, std::uint16_t lod) const = 0;
    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive) = 0;
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) = 0;
};

}  // namespace rl4
