// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/animatedmaps/AnimatedMapsFactory.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsImpl.h"
#include "riglogic/animatedmaps/AnimatedMapsImplOutputInstance.h"
#include "riglogic/animatedmaps/AnimatedMapsNull.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"
#include "riglogic/conditionaltable/ConditionalTable.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/utils/Extd.h"

#include <cstddef>
#include <cstdint>

namespace rl4 {

static AnimatedMapsOutputInstance::Factory createAnimatedMapsOutputInstanceFactory(const Configuration& /*unused*/,
                                                                                   std::uint16_t animatedMapCount) {
    return [=](MemoryResource* memRes) {
        return UniqueInstance<AnimatedMapsImplOutputInstance, AnimatedMapsOutputInstance>::with(memRes).create(animatedMapCount,
                                                                                                               memRes);
    };
}

AnimatedMaps::Pointer AnimatedMapsFactory::create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes) {
    const EvaluatorType type = meta->popFrontEvaluator();

    if (type == EvaluatorType::Null) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<AnimatedMapsNull, AnimatedMaps>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    auto instanceFactory = createAnimatedMapsOutputInstanceFactory(config, meta->animatedMapCount);
    auto moduleFactory = UniqueInstance<AnimatedMapsImpl, AnimatedMaps>::with(memRes);
    return moduleFactory.create(Vector<std::uint16_t>{memRes}, ConditionalTable{memRes}, instanceFactory);
}

AnimatedMaps::Pointer AnimatedMapsFactory::create(const Configuration& config,
                                                  RigMetadata* meta,
                                                  const dna::Reader* reader,
                                                  Controls* controls,
                                                  MemoryResource* memRes) {
    if (!config.loadAnimatedMaps || (reader->getLODCount() == 0u) || (reader->getAnimatedMapLODs().size() == 0u)) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<AnimatedMapsNull, AnimatedMaps>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    Vector<std::uint16_t> lods{memRes};
    Vector<std::uint16_t> inputIndices{memRes};
    Vector<std::uint16_t> outputIndices{memRes};
    Vector<float> fromValues{memRes};
    Vector<float> toValues{memRes};
    Vector<float> slopeValues{memRes};
    Vector<float> cutValues{memRes};

    extd::copy(reader->getAnimatedMapLODs(), lods);
    extd::copy(reader->getAnimatedMapInputIndices(), inputIndices);
    extd::copy(reader->getAnimatedMapOutputIndices(), outputIndices);
    extd::copy(reader->getAnimatedMapFromValues(), fromValues);
    extd::copy(reader->getAnimatedMapToValues(), toValues);
    extd::copy(reader->getAnimatedMapSlopeValues(), slopeValues);
    extd::copy(reader->getAnimatedMapCutValues(), cutValues);
    // DNAs may contain these parameters in reverse order
    // i.e. the `from` value is actually larger than the `to` value
    assert(fromValues.size() == toValues.size());
    for (std::size_t i = 0ul; i < fromValues.size(); ++i) {
        if (fromValues[i] > toValues[i]) {
            std::swap(fromValues[i], toValues[i]);
        }
    }

    for (std::uint16_t lod = {}; lod < static_cast<std::uint16_t>(lods.size()); ++lod) {
        ConstArrayView<std::uint16_t> inputIndicesForLOD(inputIndices.data(), lods[lod]);
        controls->registerControls(lod, inputIndicesForLOD);
    }

    const auto inputCount = static_cast<std::uint16_t>(reader->getRawControlCount() + reader->getPSDCount());
    const auto outputCount = reader->getAnimatedMapCount();
    ConditionalTable conditionals{std::move(inputIndices),
                                  std::move(outputIndices),
                                  std::move(fromValues),
                                  std::move(toValues),
                                  std::move(slopeValues),
                                  std::move(cutValues),
                                  inputCount,
                                  outputCount,
                                  memRes};

    auto instanceFactory = createAnimatedMapsOutputInstanceFactory(config, outputCount);
    auto moduleFactory = UniqueInstance<AnimatedMapsImpl, AnimatedMaps>::with(memRes);
    return moduleFactory.create(std::move(lods), std::move(conditionals), instanceFactory);
}

}  // namespace rl4
