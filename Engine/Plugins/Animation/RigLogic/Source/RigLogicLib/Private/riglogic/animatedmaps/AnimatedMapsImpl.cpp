// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/animatedmaps/AnimatedMapsImpl.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"
#include "riglogic/conditionaltable/ConditionalTable.h"
#include "riglogic/controls/ControlsInputInstance.h"

#include <cassert>
#include <utility>

namespace rl4 {

AnimatedMapsImpl::AnimatedMapsImpl(Vector<std::uint16_t>&& lods_,
                                   ConditionalTable&& conditionals_,
                                   AnimatedMapsOutputInstance::Factory instanceFactory_) :
    lods{std::move(lods_)},
    conditionals{std::move(conditionals_)},
    instanceFactory{instanceFactory_} {
}

AnimatedMapsOutputInstance::Pointer AnimatedMapsImpl::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(instanceMemRes);
}

ConstArrayView<std::uint16_t> AnimatedMapsImpl::getAnimatedMapIndicesForLOD(std::uint16_t lod) const {
    assert(lod < lods.size());
    const auto outputIndices = conditionals.getOutputIndices();
    return outputIndices.subview(0ul, lods[lod]);
}

void AnimatedMapsImpl::calculate(const ControlsInputInstance* inputs,
                                 AnimatedMapsOutputInstance* outputs,
                                 std::uint16_t lod) const {
    assert(lod < lods.size());
    conditionals.calculateForward(inputs->getInputBuffer().data(), outputs->getOutputBuffer().data(), lods[lod]);
}

void AnimatedMapsImpl::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(lods, conditionals);
}

void AnimatedMapsImpl::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(lods, conditionals);
}

}  // namespace rl4
