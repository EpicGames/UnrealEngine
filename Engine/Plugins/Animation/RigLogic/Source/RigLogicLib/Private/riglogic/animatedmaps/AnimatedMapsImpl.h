// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMaps.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"
#include "riglogic/conditionaltable/ConditionalTable.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class AnimatedMapsImpl : public AnimatedMaps {
public:
    AnimatedMapsImpl(Vector<std::uint16_t>&& lods_,
                     ConditionalTable&& conditionals_,
                     AnimatedMapsOutputInstance::Factory instanceFactory_);
    AnimatedMapsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t lod) const override;
    void calculate(const ControlsInputInstance* inputs, AnimatedMapsOutputInstance* outputs, std::uint16_t lod) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    Vector<std::uint16_t> lods;
    ConditionalTable conditionals;
    AnimatedMapsOutputInstance::Factory instanceFactory;
};

}  // namespace rl4
