// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/animatedmaps/AnimatedMapsImplOutputInstance.h"

#include "riglogic/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

AnimatedMapsImplOutputInstance::AnimatedMapsImplOutputInstance(std::uint16_t animatedMapCount, MemoryResource* memRes) :
    outputBuffer{animatedMapCount, {}, memRes} {
}

ArrayView<float> AnimatedMapsImplOutputInstance::getOutputBuffer() {
    return ArrayView<float>{outputBuffer};
}

void AnimatedMapsImplOutputInstance::resetOutputBuffer() {
    std::fill(outputBuffer.begin(), outputBuffer.end(), 0.0f);
}

}  // namespace rl4
