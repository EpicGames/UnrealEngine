// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"

#include <cstdint>

namespace rl4 {

class AnimatedMapsImplOutputInstance : public AnimatedMapsOutputInstance {
public:
    AnimatedMapsImplOutputInstance(std::uint16_t animatedMapCount, MemoryResource* memRes);
    ArrayView<float> getOutputBuffer() override;
    void resetOutputBuffer() override;

private:
    Vector<float> outputBuffer;
};

}  // namespace rl4
