// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/animatedmaps/AnimatedMapsNull.h"

#include "riglogic/animatedmaps/AnimatedMapsNullOutputInstance.h"

namespace rl4 {

AnimatedMapsOutputInstance::Pointer AnimatedMapsNull::createInstance(MemoryResource* instanceMemRes) const {
    return UniqueInstance<AnimatedMapsNullOutputInstance, AnimatedMapsOutputInstance>::with(instanceMemRes).create();
}

ConstArrayView<std::uint16_t> AnimatedMapsNull::getAnimatedMapIndicesForLOD(std::uint16_t /*unused*/) const {
    return {};
}

void AnimatedMapsNull::calculate(const ControlsInputInstance* /*unused*/,
                                 AnimatedMapsOutputInstance* /*unused*/,
                                 std::uint16_t /*unused*/) const {
}

void AnimatedMapsNull::load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) {
}

void AnimatedMapsNull::save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) {
}

}  // namespace rl4
