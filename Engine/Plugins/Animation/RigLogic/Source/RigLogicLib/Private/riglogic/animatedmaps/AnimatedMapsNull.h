// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMaps.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"
#include "riglogic/conditionaltable/ConditionalTable.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class AnimatedMapsNull : public AnimatedMaps {
public:
    AnimatedMapsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t /*unused*/) const override;
    void calculate(const ControlsInputInstance* /*unused*/,
                   AnimatedMapsOutputInstance* /*unused*/,
                   std::uint16_t /*unused*/) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) override;
};

}  // namespace rl4
