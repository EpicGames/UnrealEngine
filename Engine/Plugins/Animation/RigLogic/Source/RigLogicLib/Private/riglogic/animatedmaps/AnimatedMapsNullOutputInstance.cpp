// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/animatedmaps/AnimatedMapsNullOutputInstance.h"

#include "riglogic/TypeDefs.h"

namespace rl4 {

ArrayView<float> AnimatedMapsNullOutputInstance::getOutputBuffer() {
    return {};
}

void AnimatedMapsNullOutputInstance::resetOutputBuffer() {
}

}  // namespace rl4
