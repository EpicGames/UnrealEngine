// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"

namespace rl4 {

class AnimatedMapsNullOutputInstance : public AnimatedMapsOutputInstance {
public:
    ArrayView<float> getOutputBuffer() override;
    void resetOutputBuffer() override;
};

}  // namespace rl4
