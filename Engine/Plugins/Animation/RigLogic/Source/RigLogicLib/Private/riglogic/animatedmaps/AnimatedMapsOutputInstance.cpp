// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"

namespace rl4 {

AnimatedMapsOutputInstance::~AnimatedMapsOutputInstance() = default;

}  // namespace rl4
