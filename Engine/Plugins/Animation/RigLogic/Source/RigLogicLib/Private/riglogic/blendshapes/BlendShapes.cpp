// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/blendshapes/BlendShapes.h"

namespace rl4 {

BlendShapes::~BlendShapes() = default;

}  // namespace rl4
