// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class BlendShapes {
public:
    using Pointer = UniqueInstance<BlendShapes>::PointerType;

protected:
    virtual ~BlendShapes();

public:
    virtual BlendShapesOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const = 0;
    virtual ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const = 0;
    virtual void calculate(const ControlsInputInstance* inputs, BlendShapesOutputInstance* outputs, std::uint16_t lod) const = 0;
    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive) = 0;
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) = 0;
};

}  // namespace rl4
