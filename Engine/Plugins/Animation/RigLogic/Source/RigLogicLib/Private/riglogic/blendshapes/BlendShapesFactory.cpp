// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/blendshapes/BlendShapesFactory.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapesImpl.h"
#include "riglogic/blendshapes/BlendShapesImplOutputInstance.h"
#include "riglogic/blendshapes/BlendShapesNull.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

static BlendShapesOutputInstance::Factory createBlendShapesOutputInstanceFactory(const Configuration& /*unused*/,
                                                                                 std::uint16_t blendShapeCount) {
    return [=](MemoryResource* memRes) {
        return UniqueInstance<BlendShapesImplOutputInstance, BlendShapesOutputInstance>::with(memRes).create(blendShapeCount,
                                                                                                             memRes);
    };
}

BlendShapes::Pointer BlendShapesFactory::create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes) {
    const EvaluatorType type = meta->popFrontEvaluator();

    if (type == EvaluatorType::Null) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<BlendShapesNull, BlendShapes>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    auto instanceFactory = createBlendShapesOutputInstanceFactory(config, meta->blendShapeCount);
    auto moduleFactory = UniqueInstance<BlendShapesImpl, BlendShapes>::with(memRes);
    return moduleFactory.create(Vector<std::uint16_t>{memRes},
                                Vector<std::uint16_t>{memRes},
                                Vector<std::uint16_t>{memRes},
                                instanceFactory);
}

BlendShapes::Pointer BlendShapesFactory::create(const Configuration& config,
                                                RigMetadata* meta,
                                                const dna::Reader* reader,
                                                Controls* controls,
                                                MemoryResource* memRes) {
    if (!config.loadBlendShapes || (reader->getLODCount() == 0u) || (reader->getBlendShapeChannelLODs().size() == 0u)) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<BlendShapesNull, BlendShapes>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    Vector<std::uint16_t> lods{memRes};
    Vector<std::uint16_t> inputIndices{memRes};
    Vector<std::uint16_t> outputIndices{memRes};

    extd::copy(reader->getBlendShapeChannelLODs(), lods);
    extd::copy(reader->getBlendShapeChannelInputIndices(), inputIndices);
    extd::copy(reader->getBlendShapeChannelOutputIndices(), outputIndices);

    for (std::uint16_t lod = {}; lod < static_cast<std::uint16_t>(lods.size()); ++lod) {
        ConstArrayView<std::uint16_t> inputIndicesForLOD(inputIndices.data(), lods[lod]);
        controls->registerControls(lod, inputIndicesForLOD);
    }

    auto instanceFactory = createBlendShapesOutputInstanceFactory(config, reader->getBlendShapeChannelCount());
    auto moduleFactory = UniqueInstance<BlendShapesImpl, BlendShapes>::with(memRes);
    return moduleFactory.create(std::move(lods), std::move(inputIndices), std::move(outputIndices), instanceFactory);
}

}  // namespace rl4
