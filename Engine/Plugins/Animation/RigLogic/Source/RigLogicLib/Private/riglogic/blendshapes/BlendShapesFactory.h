// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapes.h"

namespace rl4 {

struct Configuration;
struct RigMetadata;
class Controls;

struct BlendShapesFactory {
    static BlendShapes::Pointer
    create(const Configuration& config, RigMetadata* meta, const dna::Reader* reader, Controls* controls, MemoryResource* memRes);
    static BlendShapes::Pointer create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes);
};

}  // namespace rl4
