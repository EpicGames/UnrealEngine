// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/blendshapes/BlendShapesImpl.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"
#include "riglogic/controls/ControlsInputInstance.h"

#include <cassert>
#include <cstdint>

namespace rl4 {

BlendShapesImpl::BlendShapesImpl(Vector<std::uint16_t>&& lods_,
                                 Vector<std::uint16_t>&& inputIndices_,
                                 Vector<std::uint16_t>&& outputIndices_,
                                 BlendShapesOutputInstance::Factory instanceFactory_) :
    lods{std::move(lods_)},
    inputIndices{std::move(inputIndices_)},
    outputIndices{std::move(outputIndices_)},
    instanceFactory{instanceFactory_} {
}

BlendShapesOutputInstance::Pointer BlendShapesImpl::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(instanceMemRes);
}

ConstArrayView<std::uint16_t> BlendShapesImpl::getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const {
    assert(lod < lods.size());
    return {outputIndices.data(), lods[lod]};
}

void BlendShapesImpl::calculate(const ControlsInputInstance* inputs,
                                BlendShapesOutputInstance* outputs,
                                std::uint16_t lod) const {
    assert(lod < lods.size());
    const auto inputBuffer = inputs->getInputBuffer();
    auto outputBuffer = outputs->getOutputBuffer();
    std::fill(outputBuffer.begin(), outputBuffer.end(), 0.0f);
    for (std::uint16_t i = 0u; i < lods[lod]; ++i) {
        outputBuffer[outputIndices[i]] = inputBuffer[inputIndices[i]];
    }
}

void BlendShapesImpl::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(lods, inputIndices, outputIndices);
}

void BlendShapesImpl::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(lods, inputIndices, outputIndices);
}

}  // namespace rl4
