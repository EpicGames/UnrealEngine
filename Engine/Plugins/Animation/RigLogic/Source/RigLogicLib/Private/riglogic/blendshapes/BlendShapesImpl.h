// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapes.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class BlendShapesImpl : public BlendShapes {
public:
    BlendShapesImpl(Vector<std::uint16_t>&& lods_,
                    Vector<std::uint16_t>&& inputIndices_,
                    Vector<std::uint16_t>&& outputIndices_,
                    BlendShapesOutputInstance::Factory instanceFactory_);
    BlendShapesOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const override;
    void calculate(const ControlsInputInstance* inputs, BlendShapesOutputInstance* outputs, std::uint16_t lod) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    Vector<std::uint16_t> lods;
    Vector<std::uint16_t> inputIndices;
    Vector<std::uint16_t> outputIndices;
    BlendShapesOutputInstance::Factory instanceFactory;
};

}  // namespace rl4
