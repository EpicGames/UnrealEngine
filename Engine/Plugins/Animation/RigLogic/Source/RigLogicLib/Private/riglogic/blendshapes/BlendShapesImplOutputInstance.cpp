// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/blendshapes/BlendShapesImplOutputInstance.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

BlendShapesImplOutputInstance::BlendShapesImplOutputInstance(std::uint16_t blendShapeCount, MemoryResource* memRes) :
    outputBuffer{blendShapeCount, {}, memRes} {
}

ArrayView<float> BlendShapesImplOutputInstance::getOutputBuffer() {
    return ArrayView<float>{outputBuffer};
}

void BlendShapesImplOutputInstance::resetOutputBuffer() {
    std::fill(outputBuffer.begin(), outputBuffer.end(), 0.0f);
}

}  // namespace rl4
