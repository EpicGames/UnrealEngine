// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/blendshapes/BlendShapesNull.h"

#include "riglogic/blendshapes/BlendShapesNullOutputInstance.h"

namespace rl4 {

BlendShapesOutputInstance::Pointer BlendShapesNull::createInstance(MemoryResource* instanceMemRes) const {
    return UniqueInstance<BlendShapesNullOutputInstance, BlendShapesOutputInstance>::with(instanceMemRes).create();
}

ConstArrayView<std::uint16_t> BlendShapesNull::getBlendShapeChannelIndicesForLOD(std::uint16_t /*unused*/) const {
    return {};
}

void BlendShapesNull::calculate(const ControlsInputInstance* /*unused*/,
                                BlendShapesOutputInstance* /*unused*/,
                                std::uint16_t /*unused*/) const {
}

void BlendShapesNull::load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) {
}

void BlendShapesNull::save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) {
}

}  // namespace rl4
