// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapes.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class BlendShapesNull : public BlendShapes {
public:
    BlendShapesOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t /*unused*/) const override;
    void calculate(const ControlsInputInstance* /*unused*/,
                   BlendShapesOutputInstance* /*unused*/,
                   std::uint16_t /*unused*/) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) override;
};

}  // namespace rl4
