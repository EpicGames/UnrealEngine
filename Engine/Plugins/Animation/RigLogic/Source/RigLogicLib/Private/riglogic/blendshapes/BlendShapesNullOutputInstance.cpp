// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/blendshapes/BlendShapesNullOutputInstance.h"

#include <cstdint>

namespace rl4 {

ArrayView<float> BlendShapesNullOutputInstance::getOutputBuffer() {
    return {};
}

void BlendShapesNullOutputInstance::resetOutputBuffer() {
}

}  // namespace rl4
