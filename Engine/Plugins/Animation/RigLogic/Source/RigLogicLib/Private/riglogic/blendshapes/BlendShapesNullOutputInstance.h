// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"

namespace rl4 {

class BlendShapesNullOutputInstance : public BlendShapesOutputInstance {
public:
    ArrayView<float> getOutputBuffer() override;
    void resetOutputBuffer() override;
};

}  // namespace rl4
