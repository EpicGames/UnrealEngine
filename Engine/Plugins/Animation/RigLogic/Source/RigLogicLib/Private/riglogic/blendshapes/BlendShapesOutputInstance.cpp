// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/blendshapes/BlendShapesOutputInstance.h"

namespace rl4 {

BlendShapesOutputInstance::~BlendShapesOutputInstance() = default;

}  // namespace rl4
