// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

class BlendShapesOutputInstance {
public:
    using Pointer = UniqueInstance<BlendShapesOutputInstance>::PointerType;
    using Factory = std::function<Pointer(MemoryResource*)>;

public:
    virtual ~BlendShapesOutputInstance();
    virtual ArrayView<float> getOutputBuffer() = 0;
    virtual void resetOutputBuffer() = 0;
};

}  // namespace rl4
