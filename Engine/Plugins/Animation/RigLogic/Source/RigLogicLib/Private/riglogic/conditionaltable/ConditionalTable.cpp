// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/conditionaltable/ConditionalTable.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstddef>
#include <cstdint>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace {

const float clampMin = 0.0f;
const float clampMax = 1.0f;

Vector<std::uint16_t> buildIntervalSkipMap(ConstArrayView<std::uint16_t> inputIndices,
                                           ConstArrayView<std::uint16_t> outputIndices,
                                           MemoryResource* memRes) {
    assert(inputIndices.size() == outputIndices.size());
    Vector<std::uint16_t> intervalsRemaining{inputIndices.size(), {}, memRes};
    for (std::size_t i = {}; i < inputIndices.size();) {
        std::uint16_t intervalCount = 1u;
        const std::uint16_t currentInputIndex = inputIndices[i];
        const std::uint16_t currentOutputIndex = outputIndices[i];
        for (std::size_t j = i + 1ul;
             (j < inputIndices.size()) && (currentInputIndex == inputIndices[j]) && (currentOutputIndex == outputIndices[j]);
             ++j) {
            intervalsRemaining[j] = intervalCount++;
        }
        auto start = intervalsRemaining.data() + i;
        std::reverse(start, start + intervalCount);
        i += intervalCount;
    }
    return intervalsRemaining;
}

Vector<RangeMap> buildRangeMap(ConstArrayView<std::uint16_t> inputIndices,
                               ConstArrayView<float> fromValues,
                               ConstArrayView<float> toValues,
                               MemoryResource* memRes) {
    const std::size_t maxIndex = extd::maxOf(inputIndices);
    const std::size_t rangeMapCount = (maxIndex + 1ul);
    Vector<RangeMap> rangeMaps{rangeMapCount, RangeMap{memRes}, memRes};

    for (std::size_t i = {}; i < inputIndices.size(); ++i) {
        const std::size_t ri = inputIndices[i];
        auto& map = rangeMaps[ri];
        auto range = map.addRange(fromValues[i], toValues[i]);
        range->rows.push_back(static_cast<std::uint16_t>(i));
    }

    return rangeMaps;
}

}  // namespace

ConditionalTable::ConditionalTable(MemoryResource* memRes) :
    rangeMaps{memRes},
    intervalsRemaining{memRes},
    inputIndices{memRes},
    outputIndices{memRes},
    fromValues{memRes},
    toValues{memRes},
    slopeValues{memRes},
    cutValues{memRes},
    inputCount{},
    outputCount{} {
}

ConditionalTable::ConditionalTable(Vector<std::uint16_t>&& inputIndices_,
                                   Vector<std::uint16_t>&& outputIndices_,
                                   Vector<float>&& fromValues_,
                                   Vector<float>&& toValues_,
                                   Vector<float>&& slopeValues_,
                                   Vector<float>&& cutValues_,
                                   std::uint16_t inputCount_,
                                   std::uint16_t outputCount_,
                                   MemoryResource* memRes) :
    rangeMaps{buildRangeMap(ConstArrayView<std::uint16_t>{inputIndices_},
                            ConstArrayView<float>{fromValues_},
                            ConstArrayView<float>{toValues_},
                            memRes)},
    intervalsRemaining{buildIntervalSkipMap(ConstArrayView<std::uint16_t>{inputIndices_},
                                            ConstArrayView<std::uint16_t>{outputIndices_},
                                            memRes)},
    inputIndices{std::move(inputIndices_)},
    outputIndices{std::move(outputIndices_)},
    fromValues{std::move(fromValues_)},
    toValues{std::move(toValues_)},
    slopeValues{std::move(slopeValues_)},
    cutValues{std::move(cutValues_)},
    inputCount{inputCount_},
    outputCount{outputCount_} {
}

std::uint16_t ConditionalTable::getRowCount() const {
    return static_cast<std::uint16_t>(inputIndices.size());
}

std::uint16_t ConditionalTable::getInputCount() const {
    return inputCount;
}

std::uint16_t ConditionalTable::getOutputCount() const {
    return outputCount;
}

ConstArrayView<std::uint16_t> ConditionalTable::getInputIndices() const {
    return inputIndices;
}

ConstArrayView<std::uint16_t> ConditionalTable::getOutputIndices() const {
    return outputIndices;
}

void ConditionalTable::calculateForward(const float* inputs, float* outputs, std::uint16_t rowCount) const {
    std::fill_n(outputs, outputCount, 0.0f);

    for (std::uint16_t row = {}; row < rowCount; ++row) {
        const float inValue = inputs[inputIndices[row]];
        const float from = fromValues[row];
        const float to = toValues[row];
        if ((from <= inValue) && (inValue <= to)) {
            const std::uint16_t outIndex = outputIndices[row];
            const float slope = slopeValues[row];
            const float cut = cutValues[row];
            outputs[outIndex] += (slope * inValue + cut);
            row = static_cast<std::uint16_t>(row + intervalsRemaining[row]);
        }
    }

    for (std::size_t i = 0ul; i < outputCount; ++i) {
        outputs[i] = extd::clamp(outputs[i], clampMin, clampMax);
    }
}

void ConditionalTable::calculateForward(const float* inputs, float* outputs) const {
    calculateForward(inputs, outputs, static_cast<std::uint16_t>(outputIndices.size()));
}

void ConditionalTable::calculateReverse(float* inputs, const float* outputs, std::uint16_t rowCount) const {
    std::fill_n(inputs, inputCount, 0.0f);

    auto isValidOutput = [this, outputs, rowCount](std::uint16_t row) {
        if (row >= rowCount) {
            return false;
        }
        const std::uint16_t outIndex = outputIndices[row];
        const float from = fromValues[row];
        const float to = toValues[row];
        const float slope = slopeValues[row];
        const float cut = cutValues[row];
        const float outValue = outputs[outIndex];
        const float inValue = (outValue - cut) / slope;
        return (outValue != 0.0f) && ((from <= inValue) && (inValue <= to));
    };

    auto findRangeWithMostSolutions = [isValidOutput](ConstArrayView<Range> ranges) {
        // In reverse mapping, there is some ambiguity about finding out which row was utilized to calculate an output,
        // as by looking purely at the values, the same output value can be mapped back to different input values through
        // multiple rows in some cases.
        // To resolve this ambiguity and get the original input values back, the whole table is partitioned into groups,
        // where a group is made up of all the rows that rely on the same input index.
        // Within a single group, rows are further partitioned into even smaller groups based on the range (from, to) of
        // input values that they accept.
        // When a single group (containing all rows that map back to the same input index) is being reverse mapped, each
        // from-to range, within the group is checked for how many valid solutions they generate by the rows they contain.
        // The from-to range with the largest number of valid solutions is chosen as the most likely candidate that was
        // used in the original forward mapping, and so the reverse calculation is performed on the first row that gives
        // a valid input value back within this winning range (rows retain their relative order within these groups, so
        // because the first matching row is picked as the winner in forward calculations, the same is done in reverse).
        std::size_t maxSolutionIndex = {};
        std::size_t maxSolutionCount = {};
        for (std::size_t i = {}; i < ranges.size(); ++i) {
            const auto& range = ranges[i];
            const auto solutionCount =
                static_cast<std::size_t>(std::count_if(range.rows.begin(), range.rows.end(), isValidOutput));
            if (solutionCount > maxSolutionCount) {
                maxSolutionCount = solutionCount;
                maxSolutionIndex = i;
            }
        }
        return maxSolutionIndex;
    };

    auto solveInput = [this, inputs, outputs, rowCount](ConstArrayView<std::uint16_t> rows) {
        for (auto row : rows) {
            if (row < rowCount) {
                const std::uint16_t inIndex = inputIndices[row];
                const std::uint16_t outIndex = outputIndices[row];
                const float from = fromValues[row];
                const float to = toValues[row];
                const float slope = slopeValues[row];
                const float cut = cutValues[row];
                const float outValue = outputs[outIndex];
                const float inValue = (outValue - cut) / slope;
                if ((from <= inValue) && (inValue <= to)) {
                    inputs[inIndex] = inValue;
                    return true;
                }
            }
        }
        return false;
    };

    for (const auto& map : rangeMaps) {
        const auto rangeIndex = findRangeWithMostSolutions(ConstArrayView<Range>{map.ranges});
        solveInput(ConstArrayView<std::uint16_t>{map.ranges[rangeIndex].rows});
    }
}

void ConditionalTable::calculateReverse(float* inputs, const float* outputs) const {
    calculateReverse(inputs, outputs, static_cast<std::uint16_t>(outputIndices.size()));
}

}  // namespace rl4
