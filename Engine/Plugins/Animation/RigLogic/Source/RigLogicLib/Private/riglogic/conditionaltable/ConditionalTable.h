// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

#include <cstddef>

namespace rl4 {

struct Range {
    float from;
    float to;
    Vector<std::uint16_t> rows;

    explicit Range(MemoryResource* memRes) :
        from{},
        to{},
        rows{memRes} {
    }

    Range(float from_, float to_, MemoryResource* memRes) :
        from{from_},
        to{to_},
        rows{memRes} {
    }

    std::size_t size() const {
        return rows.size();
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(from, to, rows);
    }
};

struct RangeMap {
    Vector<Range> ranges;

    explicit RangeMap(MemoryResource* memRes) :
        ranges{memRes} {
    }

    Range* findRange(float from, float to) {
        auto it = std::find_if(ranges.begin(), ranges.end(), [from, to](const Range& range) {
            return ((range.from == from) && (range.to == to));
        });
        return (it == ranges.end() ? nullptr : &(*it));
    }

    Range* addRange(float from, float to) {
        Range* range = findRange(from, to);
        if (range != nullptr) {
            return range;
        }
        ranges.emplace_back(from, to, ranges.get_allocator().getMemoryResource());
        return &ranges.back();
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(ranges);
    }
};

class ConditionalTable {
public:
    explicit ConditionalTable(MemoryResource* memRes);
    ConditionalTable(Vector<std::uint16_t>&& inputIndices_,
                     Vector<std::uint16_t>&& outputIndices_,
                     Vector<float>&& fromValues_,
                     Vector<float>&& toValues_,
                     Vector<float>&& slopeValues_,
                     Vector<float>&& cutValues_,
                     std::uint16_t inputCount_,
                     std::uint16_t outputCount_,
                     MemoryResource* memRes);

    std::uint16_t getRowCount() const;
    std::uint16_t getInputCount() const;
    std::uint16_t getOutputCount() const;
    ConstArrayView<std::uint16_t> getInputIndices() const;
    ConstArrayView<std::uint16_t> getOutputIndices() const;
    void calculateForward(const float* inputs, float* outputs) const;
    void calculateForward(const float* inputs, float* outputs, std::uint16_t rowCount) const;
    void calculateReverse(float* inputs, const float* outputs) const;
    void calculateReverse(float* inputs, const float* outputs, std::uint16_t rowCount) const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(rangeMaps,
                intervalsRemaining,
                inputIndices,
                outputIndices,
                fromValues,
                toValues,
                slopeValues,
                cutValues,
                inputCount,
                outputCount);
    }

private:
    Vector<RangeMap> rangeMaps;
    Vector<std::uint16_t> intervalsRemaining;
    Vector<std::uint16_t> inputIndices;
    Vector<std::uint16_t> outputIndices;
    Vector<float> fromValues;
    Vector<float> toValues;
    Vector<float> slopeValues;
    Vector<float> cutValues;
    std::uint16_t inputCount;
    std::uint16_t outputCount;
};

}  // namespace rl4
