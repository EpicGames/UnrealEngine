// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/controls/Controls.h"

#include "riglogic/conditionaltable/ConditionalTable.h"
#include "riglogic/controls/ControlsInputInstance.h"

#include <cassert>
#include <cstdint>

namespace rl4 {

Controls::Controls(ConditionalTable&& guiToRawMapping_,
                   Vector<ControlInitializer>&& initialValues_,
                   ControlsInputInstance::Factory instanceFactory_,
                   std::uint16_t lodCount,
                   MemoryResource* memRes) :
    registeredControls{lodCount, Vector<std::uint16_t>{memRes}, memRes},
    guiToRawMapping{std::move(guiToRawMapping_)},
    initialValues{std::move(initialValues_)},
    instanceFactory{instanceFactory_} {
}

ControlsInputInstance::Pointer Controls::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(initialValues, instanceMemRes);
}

void Controls::registerControls(std::uint16_t lod, ConstArrayView<std::uint16_t> controlIndices) {
    assert(lod < registeredControls.size());
    auto& controlsForLOD = registeredControls[lod];
    controlsForLOD.insert(controlsForLOD.end(), controlIndices.begin(), controlIndices.end());
    std::sort(controlsForLOD.begin(), controlsForLOD.end());
    controlsForLOD.erase(std::unique(controlsForLOD.begin(), controlsForLOD.end()), controlsForLOD.end());
}

ConstArrayView<std::uint16_t> Controls::getRegisteredControls(std::uint16_t lod) const {
    assert(lod < registeredControls.size());
    return registeredControls[lod];
}

void Controls::mapGUIToRaw(ControlsInputInstance* instance) const {
    auto guiControlBuffer = instance->getGUIControlBuffer();
    assert(guiControlBuffer.size() == guiToRawMapping.getInputCount());
    auto inputBuffer = instance->getInputBuffer();
    guiToRawMapping.calculateForward(guiControlBuffer.data(), inputBuffer.data());
}

void Controls::mapRawToGUI(ControlsInputInstance* instance) const {
    auto guiControlBuffer = instance->getGUIControlBuffer();
    assert(guiControlBuffer.size() == guiToRawMapping.getInputCount());
    auto inputBuffer = instance->getInputBuffer();
    guiToRawMapping.calculateReverse(guiControlBuffer.data(), inputBuffer.data());
}

}  // namespace rl4
