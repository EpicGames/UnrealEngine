// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/conditionaltable/ConditionalTable.h"
#include "riglogic/controls/ControlsInputInstance.h"

#include <cstdint>

namespace rl4 {

class Controls {
public:
    using Pointer = UniqueInstance<Controls>::PointerType;

public:
    Controls(ConditionalTable&& guiToRawMapping_,
             Vector<ControlInitializer>&& initialValues_,
             ControlsInputInstance::Factory instanceFactory_,
             std::uint16_t lodCount,
             MemoryResource* memRes);

    ControlsInputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const;
    void registerControls(std::uint16_t lod, ConstArrayView<std::uint16_t> controlIndices);
    ConstArrayView<std::uint16_t> getRegisteredControls(std::uint16_t lod) const;
    void mapGUIToRaw(ControlsInputInstance* instance) const;
    void mapRawToGUI(ControlsInputInstance* instance) const;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(registeredControls, guiToRawMapping, initialValues);
    }

private:
    Matrix<std::uint16_t> registeredControls;
    ConditionalTable guiToRawMapping;
    Vector<ControlInitializer> initialValues;
    ControlsInputInstance::Factory instanceFactory;
};

}  // namespace rl4
