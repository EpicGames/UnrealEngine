// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/controls/ControlsFactory.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/conditionaltable/ConditionalTable.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/controls/instances/StandardControlsInputInstance.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/utils/Extd.h"

#include <cstdint>

namespace rl4 {

static ConditionalTable createConditionalTable(const dna::Reader* reader, MemoryResource* memRes) {
    Vector<std::uint16_t> inputIndices{memRes};
    Vector<std::uint16_t> outputIndices{memRes};
    Vector<float> fromValues{memRes};
    Vector<float> toValues{memRes};
    Vector<float> slopeValues{memRes};
    Vector<float> cutValues{memRes};
    extd::copy(reader->getGUIToRawInputIndices(), inputIndices);
    extd::copy(reader->getGUIToRawOutputIndices(), outputIndices);
    extd::copy(reader->getGUIToRawFromValues(), fromValues);
    extd::copy(reader->getGUIToRawToValues(), toValues);
    extd::copy(reader->getGUIToRawSlopeValues(), slopeValues);
    extd::copy(reader->getGUIToRawCutValues(), cutValues);
    // DNAs may contain these parameters in reverse order
    // i.e. the `from` value is actually larger than the `to` value
    assert(fromValues.size() == toValues.size());
    for (std::size_t i = 0ul; i < fromValues.size(); ++i) {
        if (fromValues[i] > toValues[i]) {
            std::swap(fromValues[i], toValues[i]);
        }
    }

    const auto guiControlCount = reader->getGUIControlCount();
    const auto rawControlCount = reader->getRawControlCount();

    return ConditionalTable{std::move(inputIndices),
                            std::move(outputIndices),
                            std::move(fromValues),
                            std::move(toValues),
                            std::move(slopeValues),
                            std::move(cutValues),
                            guiControlCount,
                            rawControlCount,
                            memRes};
}

static Vector<ControlInitializer> createInitialControlValues(const dna::Reader* reader, MemoryResource* memRes) {
    auto endsWith = [](StringView str, StringView suffix) {
#if __cplusplus >= 201402L || (defined(_MSC_VER) && _MSC_VER >= 1900)
        return (suffix.size() <= str.size()) &&
               std::equal(extd::advanced(str.begin(), str.size() - suffix.size()), str.end(), suffix.begin(), suffix.end());
#else
        return (suffix.size() <= str.size()) &&
               std::equal(extd::advanced(str.begin(), str.size() - suffix.size()), str.end(), suffix.begin());
#endif
    };
    auto makeSuffix = [](const char* str) { return StringView{str, std::strlen(str)}; };
    const auto wSuffix = makeSuffix(".w");
    const auto qwSuffix = makeSuffix(".qw");
    const auto rawControlCount = static_cast<std::size_t>(reader->getRawControlCount());
    const auto psdControlCount = static_cast<std::size_t>(reader->getPSDCount());
    const auto mlControlCount = static_cast<std::size_t>(reader->getMLControlCount());
    const auto rbfControlCount = static_cast<std::size_t>(reader->getRBFPoseControlCount());
    const auto controlCount = rawControlCount + psdControlCount + mlControlCount + rbfControlCount;
    Vector<ControlInitializer> initialValues{memRes};
    initialValues.reserve(controlCount - rawControlCount);

    const auto addQuaternionRawControls = [&](ConstArrayView<std::uint16_t> rawControlIndices) {
        for (const auto rawControlIndex : rawControlIndices) {
            const auto rawControlName = reader->getRawControlName(rawControlIndex);
            if (endsWith(rawControlName, wSuffix) || endsWith(rawControlName, qwSuffix)) {
                initialValues.push_back({rawControlIndex, 1.0f});
            }
        }
    };

    for (std::uint16_t solverIndex = {}; solverIndex < reader->getRBFSolverCount(); ++solverIndex) {
        const auto solverRawControlIndices = reader->getRBFSolverRawControlIndices(solverIndex);
        addQuaternionRawControls(solverRawControlIndices);
    }

    for (std::uint16_t twistIndex = {}; twistIndex < reader->getTwistCount(); ++twistIndex) {
        const auto twistInputIndices = reader->getTwistInputControlIndices(twistIndex);
        addQuaternionRawControls(twistInputIndices);
    }

    for (std::uint16_t swingIndex = {}; swingIndex < reader->getSwingCount(); ++swingIndex) {
        const auto swingInputIndices = reader->getSwingInputControlIndices(swingIndex);
        addQuaternionRawControls(swingInputIndices);
    }

    const auto mlCtrlOffset = rawControlCount + psdControlCount;
    for (std::uint16_t mlCtrlIndex = {}; mlCtrlIndex < reader->getMLControlCount(); ++mlCtrlIndex) {
        const auto mlControlName = reader->getMLControlName(mlCtrlIndex);
        if (endsWith(mlControlName, wSuffix) || endsWith(mlControlName, qwSuffix)) {
            initialValues.push_back({static_cast<std::uint32_t>(mlCtrlOffset + mlCtrlIndex), 1.0f});
        }
    }

    return initialValues;
}

static ControlsInputInstance::Factory createInstanceFactory(const Configuration& /*unused*/,
                                                            std::uint16_t guiControlCount,
                                                            std::uint16_t rawControlCount,
                                                            std::uint16_t psdControlCount,
                                                            std::uint16_t mlControlCount,
                                                            std::uint16_t rbfControlCount) {
    return [=](ConstArrayView<ControlInitializer> initialValues, MemoryResource* memRes) {
        auto factory = UniqueInstance<StandardControlsInputInstance, ControlsInputInstance>::with(memRes);
        return factory
            .create(guiControlCount, rawControlCount, psdControlCount, mlControlCount, rbfControlCount, initialValues, memRes);
    };
}

Controls::Pointer ControlsFactory::create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes) {
    auto instanceFactory = createInstanceFactory(config,
                                                 meta->guiControlCount,
                                                 meta->rawControlCount,
                                                 meta->psdControlCount,
                                                 meta->mlControlCount,
                                                 meta->rbfControlCount);
    return UniqueInstance<Controls>::with(memRes).create(ConditionalTable{memRes},
                                                         Vector<ControlInitializer>{memRes},
                                                         instanceFactory,
                                                         meta->lodCount,
                                                         memRes);
}

Controls::Pointer ControlsFactory::create(const Configuration& config,
                                          RigMetadata* meta,
                                          const dna::Reader* reader,
                                          MemoryResource* memRes) {
    RL_UNUSED(meta);

    const auto guiControlCount = reader->getGUIControlCount();
    const auto rawControlCount = reader->getRawControlCount();
    const auto psdControlCount = reader->getPSDCount();
    const auto mlControlCount = reader->getMLControlCount();
    const auto rbfControlCount = reader->getRBFPoseControlCount();
    const auto lodCount = reader->getLODCount();

    ConditionalTable conditionals = createConditionalTable(reader, memRes);
    Vector<ControlInitializer> initialValues = createInitialControlValues(reader, memRes);
    auto instanceFactory =
        createInstanceFactory(config, guiControlCount, rawControlCount, psdControlCount, mlControlCount, rbfControlCount);

    return UniqueInstance<Controls>::with(memRes).create(std::move(conditionals),
                                                         std::move(initialValues),
                                                         instanceFactory,
                                                         lodCount,
                                                         memRes);
}

}  // namespace rl4
