// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/controls/ControlsInputInstance.h"

namespace rl4 {

ControlsInputInstance::~ControlsInputInstance() = default;

}  // namespace rl4
