// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

struct ControlInitializer {
    std::uint32_t index;
    float value;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(index, value);
    }
};

class ControlsInputInstance {
public:
    using Pointer = UniqueInstance<ControlsInputInstance>::PointerType;
    using Factory = std::function<Pointer(ConstArrayView<ControlInitializer>, MemoryResource*)>;

public:
    virtual ~ControlsInputInstance();
    virtual void resetInputBuffer() = 0;
    virtual ArrayView<float> getGUIControlBuffer() = 0;
    virtual ArrayView<float> getInputBuffer() = 0;
    virtual ConstArrayView<float> getGUIControlBuffer() const = 0;
    virtual ConstArrayView<float> getInputBuffer() const = 0;
    virtual std::uint16_t getGUIControlCount() const = 0;
    virtual std::uint16_t getRawControlCount() const = 0;
    virtual std::uint16_t getPSDControlCount() const = 0;
    virtual std::uint16_t getMLControlCount() const = 0;
    virtual std::uint16_t getRBFControlCount() const = 0;
};

}  // namespace rl4
