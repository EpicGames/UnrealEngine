// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/controls/instances/StandardControlsInputInstance.h"

#include "riglogic/utils/Extd.h"
#include "riglogic/utils/Macros.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <cstdint>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

StandardControlsInputInstance::StandardControlsInputInstance(std::uint16_t guiControlCount_,
                                                             std::uint16_t rawControlCount_,
                                                             std::uint16_t psdControlCount_,
                                                             std::uint16_t mlControlCount_,
                                                             std::uint16_t rbfControlCount_,
                                                             ConstArrayView<ControlInitializer> initialValues_,
                                                             MemoryResource* memRes) :
    guiControlBuffer{guiControlCount_, {}, memRes},
    inputBuffer{memRes},
    initialValues{initialValues_.begin(), initialValues_.end(), memRes},
    guiControlCount{guiControlCount_},
    rawControlCount{rawControlCount_},
    psdControlCount{psdControlCount_},
    mlControlCount{mlControlCount_},
    rbfControlCount{rbfControlCount_} {

    const auto controlCount = static_cast<std::size_t>(rawControlCount_) + static_cast<std::size_t>(psdControlCount_) +
                              static_cast<std::size_t>(mlControlCount_) + static_cast<std::size_t>(rbfControlCount_);
    inputBuffer.resize(controlCount);
    resetInputBuffer();
}

void StandardControlsInputInstance::resetInputBuffer() {
    std::fill(extd::advanced(inputBuffer.begin(), rawControlCount), inputBuffer.end(), 0.0f);
    for (const auto& initializer : initialValues) {
        inputBuffer[initializer.index] = initializer.value;
    }
}

ArrayView<float> StandardControlsInputInstance::getGUIControlBuffer() {
    return ArrayView<float>{guiControlBuffer};
}

ArrayView<float> StandardControlsInputInstance::getInputBuffer() {
    return inputBuffer;
}

ConstArrayView<float> StandardControlsInputInstance::getGUIControlBuffer() const {
    return ConstArrayView<float>{guiControlBuffer};
}

ConstArrayView<float> StandardControlsInputInstance::getInputBuffer() const {
    return inputBuffer;
}

std::uint16_t StandardControlsInputInstance::getGUIControlCount() const {
    return guiControlCount;
}

std::uint16_t StandardControlsInputInstance::getRawControlCount() const {
    return rawControlCount;
}

std::uint16_t StandardControlsInputInstance::getPSDControlCount() const {
    return psdControlCount;
}

std::uint16_t StandardControlsInputInstance::getMLControlCount() const {
    return mlControlCount;
}

std::uint16_t StandardControlsInputInstance::getRBFControlCount() const {
    return rbfControlCount;
}

}  // namespace rl4
