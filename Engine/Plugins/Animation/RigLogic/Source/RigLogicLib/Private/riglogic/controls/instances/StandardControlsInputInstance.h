// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"

#include <cstdint>

namespace rl4 {

class StandardControlsInputInstance : public ControlsInputInstance {
public:
    StandardControlsInputInstance(std::uint16_t guiControlCount_,
                                  std::uint16_t rawControlCount_,
                                  std::uint16_t psdControlCount_,
                                  std::uint16_t mlControlCount_,
                                  std::uint16_t rbfControlCount_,
                                  ConstArrayView<ControlInitializer> initialValues_,
                                  MemoryResource* memRes);

    void resetInputBuffer() override;
    ArrayView<float> getGUIControlBuffer() override;
    ArrayView<float> getInputBuffer() override;
    ConstArrayView<float> getGUIControlBuffer() const override;
    ConstArrayView<float> getInputBuffer() const override;
    std::uint16_t getGUIControlCount() const override;
    std::uint16_t getRawControlCount() const override;
    std::uint16_t getPSDControlCount() const override;
    std::uint16_t getMLControlCount() const override;
    std::uint16_t getRBFControlCount() const override;

private:
    Vector<float> guiControlBuffer;
    Vector<float> inputBuffer;
    Vector<ControlInitializer> initialValues;
    std::uint16_t guiControlCount;
    std::uint16_t rawControlCount;
    std::uint16_t psdControlCount;
    std::uint16_t mlControlCount;
    std::uint16_t rbfControlCount;
};

}  // namespace rl4
