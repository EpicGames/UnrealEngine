// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/JointBehaviorFilter.h"

#include "riglogic/utils/Extd.h"

#include <cstdint>

namespace rl4 {

namespace {

enum class JointFilterAttributeType : std::uint8_t {
    Translation = 0,
    Rotation = 1,
    Scale = 2
};

}  // namespace

JointBehaviorFilter::JointBehaviorFilter(const dna::Reader* reader_, MemoryResource* memRes) :
    reader{reader_},
    filters{memRes} {
}

JointBehaviorFilter& JointBehaviorFilter::include(dna::TranslationRepresentation translationType) {
    const FilterType includable{static_cast<std::uint8_t>(JointFilterAttributeType::Translation),
                                static_cast<std::uint8_t>(translationType)};
    if (std::find(filters.begin(), filters.end(), includable) == filters.end()) {
        filters.push_back(includable);
    }
    return *this;
}

JointBehaviorFilter& JointBehaviorFilter::include(dna::RotationRepresentation rotationType) {
    const FilterType includable{static_cast<std::uint8_t>(JointFilterAttributeType::Rotation),
                                static_cast<std::uint8_t>(rotationType)};
    if (std::find(filters.begin(), filters.end(), includable) == filters.end()) {
        filters.push_back(includable);
    }
    return *this;
}

JointBehaviorFilter& JointBehaviorFilter::include(dna::ScaleRepresentation scaleType) {
    const FilterType includable{static_cast<std::uint8_t>(JointFilterAttributeType::Scale), static_cast<std::uint8_t>(scaleType)};
    if (std::find(filters.begin(), filters.end(), includable) == filters.end()) {
        filters.push_back(includable);
    }
    return *this;
}

JointBehaviorFilter& JointBehaviorFilter::exclude(dna::TranslationRepresentation translationType) {
    const FilterType excludable{static_cast<std::uint8_t>(JointFilterAttributeType::Translation),
                                static_cast<std::uint8_t>(translationType)};
    auto it = std::find(filters.begin(), filters.end(), excludable);
    if (it != filters.end()) {
        filters.erase(it);
    }
    return *this;
}

JointBehaviorFilter& JointBehaviorFilter::exclude(dna::RotationRepresentation rotationType) {
    const FilterType excludable{static_cast<std::uint8_t>(JointFilterAttributeType::Rotation),
                                static_cast<std::uint8_t>(rotationType)};
    auto it = std::find(filters.begin(), filters.end(), excludable);
    if (it != filters.end()) {
        filters.erase(it);
    }
    return *this;
}

JointBehaviorFilter& JointBehaviorFilter::exclude(dna::ScaleRepresentation scaleType) {
    const FilterType excludable{static_cast<std::uint8_t>(JointFilterAttributeType::Scale), static_cast<std::uint8_t>(scaleType)};
    auto it = std::find(filters.begin(), filters.end(), excludable);
    if (it != filters.end()) {
        filters.erase(it);
    }
    return *this;
}

bool JointBehaviorFilter::isAttributeEnabled(std::uint16_t absAttrIndex) const {
    static constexpr std::uint16_t relAttrCount = 9;
    static constexpr std::uint32_t attrTypeCount = 3;
    const auto jointIndex = static_cast<std::uint16_t>(absAttrIndex / relAttrCount);
    const auto relAttrIndex = static_cast<std::uint16_t>(absAttrIndex % relAttrCount);
    const auto attrType = relAttrIndex / attrTypeCount;
    for (const auto& f : filters) {
        if (attrType == f.attr) {
            if (static_cast<JointFilterAttributeType>(f.attr) == JointFilterAttributeType::Translation) {
                return static_cast<dna::TranslationRepresentation>(f.repr) ==
                       reader->getJointTranslationRepresentation(jointIndex);
            }
            if (static_cast<JointFilterAttributeType>(f.attr) == JointFilterAttributeType::Rotation) {
                return static_cast<dna::RotationRepresentation>(f.repr) == reader->getJointRotationRepresentation(jointIndex);
            }
            if (static_cast<JointFilterAttributeType>(f.attr) == JointFilterAttributeType::Scale) {  // -V547
                return static_cast<dna::ScaleRepresentation>(f.repr) == reader->getJointScaleRepresentation(jointIndex);
            }
        }
    }

    return false;
}

dna::TranslationUnit JointBehaviorFilter::getTranslationUnit() const {
    return reader->getTranslationUnit();
}

dna::RotationUnit JointBehaviorFilter::getRotationUnit() const {
    return reader->getRotationUnit();
}

std::uint16_t JointBehaviorFilter::getLODCount() const {
    return reader->getLODCount();
}

std::uint16_t JointBehaviorFilter::getControlCount() const {
    std::size_t controlCount = reader->getRawControlCount();
    controlCount += reader->getPSDCount();
    controlCount += reader->getMLControlCount();
    controlCount += reader->getRBFPoseControlCount();
    return static_cast<std::uint16_t>(controlCount);
}

std::uint16_t JointBehaviorFilter::getJointCount() const {
    return reader->getJointCount();
}

std::uint16_t JointBehaviorFilter::getRowCount() const {
    return reader->getJointRowCount();
}

std::uint16_t JointBehaviorFilter::getColumnCount() const {
    return reader->getJointColumnCount();
}

std::uint16_t JointBehaviorFilter::getJointGroupCount() const {
    return reader->getJointGroupCount();
}

void JointBehaviorFilter::copyInputIndices(std::uint16_t jointGroupIndex, ArrayView<std::uint16_t> dest) const {
    const auto inputIndices = reader->getJointGroupInputIndices(jointGroupIndex);
    extd::copy(inputIndices, dest);
}

void JointBehaviorFilter::copyOutputIndices(std::uint16_t jointGroupIndex, ArrayView<std::uint16_t> dest) const {
    const auto outputIndices = reader->getJointGroupOutputIndices(jointGroupIndex);
    std::uint16_t* pDst = dest.data();
    for (auto outputIndex : outputIndices) {
        if (isAttributeEnabled(outputIndex)) {
            *pDst++ = outputIndex;
        }
    }
}

void JointBehaviorFilter::copyValues(std::uint16_t jointGroupIndex, ArrayView<float> dest) const {
    const auto values = reader->getJointGroupValues(jointGroupIndex);
    const auto outputIndices = reader->getJointGroupOutputIndices(jointGroupIndex);
    const auto rowCount = outputIndices.size();
    const auto colCount = reader->getJointGroupInputIndices(jointGroupIndex).size();
    float* pDst = dest.data();
    for (std::size_t row = {}; row < rowCount; ++row) {
        if (isAttributeEnabled(outputIndices[row])) {
            const auto rowOfValues = values.subview(row * colCount, colCount);
            extd::copy(rowOfValues, ArrayView<float>(pDst, colCount));
            pDst += rowOfValues.size();
        }
    }
}

std::uint16_t JointBehaviorFilter::getRowCountForLOD(std::uint16_t jointGroupIndex, std::uint16_t lod) const {
    const auto lods = reader->getJointGroupLODs(jointGroupIndex);
    assert(lods.size() == reader->getLODCount());

    std::uint16_t rowCount = {};
    if (lod >= lods.size()) {
        return rowCount;
    }

    const auto outputIndices = reader->getJointGroupOutputIndices(jointGroupIndex);
    for (std::size_t row = {}; row < lods[lod]; ++row) {
        if (isAttributeEnabled(outputIndices[row])) {
            ++rowCount;
        }
    }
    return rowCount;
}

std::uint16_t JointBehaviorFilter::getRowCount(std::uint16_t jointGroupIndex) const {
    return getRowCountForLOD(jointGroupIndex, 0);
}

std::uint16_t JointBehaviorFilter::getColumnCount(std::uint16_t jointGroupIndex) const {
    return static_cast<std::uint16_t>(reader->getJointGroupInputIndices(jointGroupIndex).size());
}

const dna::Reader* JointBehaviorFilter::getReader() const {
    return reader;
}

}  // namespace rl4
