// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

namespace rl4 {

class JointBehaviorFilter {
private:
    struct FilterType {
        std::uint8_t attr;
        std::uint8_t repr;

        bool operator==(const JointBehaviorFilter::FilterType& rhs) {
            return (attr == rhs.attr) && (repr == rhs.repr);
        }
    };

public:
    JointBehaviorFilter(const dna::Reader* reader_, MemoryResource* memRes);

    JointBehaviorFilter& include(dna::TranslationRepresentation translationType);
    JointBehaviorFilter& include(dna::RotationRepresentation rotationType);
    JointBehaviorFilter& include(dna::ScaleRepresentation scaleType);
    JointBehaviorFilter& exclude(dna::TranslationRepresentation translationType);
    JointBehaviorFilter& exclude(dna::RotationRepresentation rotationType);
    JointBehaviorFilter& exclude(dna::ScaleRepresentation scaleType);

    template<typename... TAttr>
    JointBehaviorFilter included(TAttr... attrTypes) const {
        JointBehaviorFilter filtered{*this};
        using expander = int[];
        static_cast<void>(expander{0, (static_cast<void>(filtered.include(attrTypes)), 0)...});
        return filtered;
    }

    template<typename... TAttr>
    JointBehaviorFilter excluded(TAttr... attrTypes) const {
        JointBehaviorFilter filtered{*this};
        using expander = int[];
        static_cast<void>(expander{0, (static_cast<void>(filtered.exclude(attrTypes)), 0)...});
        return filtered;
    }

    template<typename... TAttr>
    JointBehaviorFilter only(TAttr... attrTypes) const {
        JointBehaviorFilter filtered{reader, filters.get_allocator().getMemoryResource()};
        return filtered.included(attrTypes...);
    }

    dna::TranslationUnit getTranslationUnit() const;
    dna::RotationUnit getRotationUnit() const;
    std::uint16_t getLODCount() const;
    std::uint16_t getControlCount() const;
    std::uint16_t getJointCount() const;
    std::uint16_t getRowCount() const;
    std::uint16_t getColumnCount() const;
    std::uint16_t getJointGroupCount() const;
    std::uint16_t getRowCountForLOD(std::uint16_t jointGroupIndex, std::uint16_t lod) const;
    std::uint16_t getRowCount(std::uint16_t jointGroupIndex) const;
    std::uint16_t getColumnCount(std::uint16_t jointGroupIndex) const;
    void copyInputIndices(std::uint16_t jointGroupIndex, ArrayView<std::uint16_t> dest) const;
    void copyOutputIndices(std::uint16_t jointGroupIndex, ArrayView<std::uint16_t> dest) const;
    void copyValues(std::uint16_t jointGroupIndex, ArrayView<float> dest) const;

    const dna::Reader* getReader() const;

private:
    bool isAttributeEnabled(std::uint16_t absAttrIndex) const;

private:
    const dna::Reader* reader;
    Vector<FilterType> filters;
};

}  // namespace rl4
