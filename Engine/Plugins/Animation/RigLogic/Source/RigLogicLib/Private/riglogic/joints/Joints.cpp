// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/Joints.h"

#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/riglogic/RigInstance.h"
#include "riglogic/types/Aliases.h"

#include <cstdint>

namespace rl4 {

Joints::Joints(JointsEvaluator::Pointer evaluator_, RigMetadata* meta, MemoryResource* memRes) :
    evaluator{std::move(evaluator_)},
    neutralValues{memRes},
    variableAttributeIndices{memRes},
    jointIndices{memRes},
    jointGroupCount{} {

    variableAttributeIndices.resize(meta->lodCount);
    jointIndices.resize(meta->lodCount);
}

Joints::Joints(JointsEvaluator::Pointer evaluator_,
               Vector<float>&& neutralValues_,
               Matrix<std::uint16_t>&& variableAttributeIndices_,
               Matrix<std::uint16_t>&& jointIndices_,
               std::uint16_t jointGroupCount_) :
    evaluator{std::move(evaluator_)},
    neutralValues{std::move(neutralValues_)},
    variableAttributeIndices{std::move(variableAttributeIndices_)},
    jointIndices{std::move(jointIndices_)},
    jointGroupCount{jointGroupCount_} {
}

JointsOutputInstance::Pointer Joints::createInstance(MemoryResource* instanceMemRes) const {
    return evaluator->createInstance(instanceMemRes);
}

ConstArrayView<std::uint16_t> Joints::getJointIndicesForLOD(std::uint16_t lod) const {
    assert(lod < jointIndices.size());
    return jointIndices[lod];
}

std::uint32_t Joints::getJointDeltaValueCountForLOD(std::uint16_t lod) const {
    return evaluator->getJointDeltaValueCountForLOD(lod);
}

void Joints::calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const {
    evaluator->calculate(inputs, outputs, lod);
}

void Joints::calculate(ControlsInputInstance* inputs,
                       JointsOutputInstance* outputs,
                       std::uint16_t lod,
                       std::uint16_t jointGroupIndex) const {
    evaluator->calculate(inputs, outputs, lod, jointGroupIndex);
}

ConstArrayView<float> Joints::getNeutralValues() const {
    return ConstArrayView<float>{neutralValues};
}

std::uint16_t Joints::getJointGroupCount() const {
    return jointGroupCount;
}

ConstArrayView<std::uint16_t> Joints::getVariableAttributeIndices(std::uint16_t lod) const {
    assert(lod < variableAttributeIndices.size());
    return variableAttributeIndices[lod];
}

}  // namespace rl4
