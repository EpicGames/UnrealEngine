// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/riglogic/RigMetadata.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;
class JointsOutputInstance;

class Joints {
public:
    using Pointer = UniqueInstance<Joints>::PointerType;

public:
    Joints(JointsEvaluator::Pointer evaluator_, RigMetadata* meta, MemoryResource* memRes);
    Joints(JointsEvaluator::Pointer evaluator_,
           Vector<float>&& neutralValues_,
           Matrix<std::uint16_t>&& variableAttributeIndices_,
           Matrix<std::uint16_t>&& jointIndices_,
           std::uint16_t jointGroupCount_);

    JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const;
    ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const;
    std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t lod) const;
    void calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const;
    void calculate(ControlsInputInstance* inputs,
                   JointsOutputInstance* outputs,
                   std::uint16_t lod,
                   std::uint16_t jointGroupIndex) const;

    template<class Archive>
    void load(Archive& archive) {
        evaluator->load(archive);
        archive >> neutralValues >> variableAttributeIndices >> jointIndices >> jointGroupCount;
    }

    template<class Archive>
    void save(Archive& archive) {
        evaluator->save(archive);
        archive << neutralValues << variableAttributeIndices << jointIndices << jointGroupCount;
    }

    std::uint16_t getJointGroupCount() const;
    ConstArrayView<float> getNeutralValues() const;
    ConstArrayView<std::uint16_t> getVariableAttributeIndices(std::uint16_t lod) const;

private:
    JointsEvaluator::Pointer evaluator;
    Vector<float> neutralValues;
    Matrix<std::uint16_t> variableAttributeIndices;
    Matrix<std::uint16_t> jointIndices;
    std::uint16_t jointGroupCount;
};

}  // namespace rl4
