// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/JointsBuilder.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsBuilder.h"

namespace rl4 {

JointsBuilder::~JointsBuilder() = default;

JointsBuilder::Pointer JointsBuilder::create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes) {
    return UniqueInstance<CPUJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
}

}  // namespace rl4
