// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/riglogic/Configuration.h"

namespace rl4 {

class Controls;
class JointBehaviorFilter;
struct RigMetadata;

class JointsBuilder {
public:
    using Pointer = UniqueInstance<JointsBuilder>::PointerType;

public:
    virtual ~JointsBuilder();

    static Pointer create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes);

    virtual void computeStorageRequirements() = 0;
    virtual void computeStorageRequirements(const JointBehaviorFilter& source) = 0;
    virtual void allocateStorage(const JointBehaviorFilter& source) = 0;
    virtual void fillStorage(const JointBehaviorFilter& source) = 0;
    virtual void registerControls(Controls* controls) = 0;
    virtual JointsEvaluator::Pointer build() = 0;
};

}  // namespace rl4
