// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/JointsEvaluator.h"

namespace rl4 {

JointsEvaluator::~JointsEvaluator() = default;

}  // namespace rl4
