// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointsOutputInstance.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class JointsEvaluator {
public:
    using Pointer = UniqueInstance<JointsEvaluator>::PointerType;

protected:
    virtual ~JointsEvaluator();

public:
    virtual JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const = 0;
    virtual std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t lod) const = 0;
    virtual void calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const = 0;
    virtual void calculate(ControlsInputInstance* inputs,
                           JointsOutputInstance* outputs,
                           std::uint16_t lod,
                           std::uint16_t jointGroupIndex) const = 0;
    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive) = 0;
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) = 0;
};

}  // namespace rl4
