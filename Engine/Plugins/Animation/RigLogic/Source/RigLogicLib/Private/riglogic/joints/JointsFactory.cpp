// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/JointsFactory.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsNullEvaluator.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"

#include <tdm/Quat.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstddef>
#include <cstdint>
#include <iterator>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

template<class TSource, class TDestination>
static void scatter(const TSource& source, TDestination& destination, std::size_t stride, std::size_t offset) {
    const auto size = static_cast<std::size_t>(std::distance(std::begin(source), std::end(source)));
    for (std::size_t i = 0ul; i < size; ++i) {
        destination[i * stride + offset] = source[i];
    }
}

static Vector<float> copyNeutralValues(const Configuration& config,
                                       RigMetadata* meta,
                                       const dna::Reader* reader,
                                       MemoryResource* memRes) {
    Vector<float> neutralValues{memRes};
    const auto numAttrsPerJoint =
        static_cast<std::size_t>(static_cast<std::uint8_t>(config.translationType) +
                                 static_cast<std::uint8_t>(config.rotationType) + static_cast<std::uint8_t>(config.scaleType));
    neutralValues.resize(reader->getJointCount() * numAttrsPerJoint, 1.0f);
    scatter(reader->getNeutralJointTranslationXs(), neutralValues, numAttrsPerJoint, 0ul);
    scatter(reader->getNeutralJointTranslationYs(), neutralValues, numAttrsPerJoint, 1ul);
    scatter(reader->getNeutralJointTranslationZs(), neutralValues, numAttrsPerJoint, 2ul);

    if (config.rotationType == RotationType::EulerAngles) {
        scatter(reader->getNeutralJointRotationXs(), neutralValues, numAttrsPerJoint, 3ul);
        scatter(reader->getNeutralJointRotationYs(), neutralValues, numAttrsPerJoint, 4ul);
        scatter(reader->getNeutralJointRotationZs(), neutralValues, numAttrsPerJoint, 5ul);
    } else {
        const auto rotationXs = reader->getNeutralJointRotationXs();
        const auto rotationYs = reader->getNeutralJointRotationYs();
        const auto rotationZs = reader->getNeutralJointRotationZs();
        const auto rotationUnit = reader->getRotationUnit();
        auto toRad = [rotationUnit](float x) {
            return (rotationUnit == dna::RotationUnit::radians) ? tdm::frad{x} : tdm::frad{tdm::fdeg{x}};
        };
        for (std::size_t jointIndex = {}; jointIndex < rotationXs.size(); ++jointIndex) {
            const std::size_t absAttrIndexBase = jointIndex * numAttrsPerJoint;
            const tdm::frad3 euler{toRad(rotationXs[jointIndex]), toRad(rotationYs[jointIndex]), toRad(rotationZs[jointIndex])};
            const tdm::fquat q{euler, meta->rotationSequence, meta->rotationSigns};
            neutralValues[absAttrIndexBase + 3ul] = q.x;
            neutralValues[absAttrIndexBase + 4ul] = q.y;
            neutralValues[absAttrIndexBase + 5ul] = q.z;
            neutralValues[absAttrIndexBase + 6ul] = q.w;
        }
    }

    return neutralValues;
}

static Matrix<std::uint16_t> copyVariableAttributeIndices(const Configuration& config,
                                                          const dna::Reader* reader,
                                                          MemoryResource* memRes) {
    Matrix<std::uint16_t> variableAttributeIndices{memRes};
    variableAttributeIndices.resize(reader->getLODCount());

    if (config.rotationType == RotationType::EulerAngles) {
        for (std::uint16_t lod = 0u; lod < reader->getLODCount(); ++lod) {
            const auto indices = reader->getJointVariableAttributeIndices(lod);
            variableAttributeIndices[lod].assign(indices.begin(), indices.end());
        }
    } else {
        const auto translationOffset = static_cast<std::uint8_t>(0);
        const auto rotationOffset = translationOffset + static_cast<std::uint8_t>(config.translationType);
        const auto scaleOffset = rotationOffset + static_cast<std::uint8_t>(config.rotationType);
        const auto numAttrsPerJoint = static_cast<std::size_t>(static_cast<std::uint8_t>(config.translationType) +
                                                               static_cast<std::uint8_t>(config.rotationType) +
                                                               static_cast<std::uint8_t>(config.scaleType));
        for (std::uint16_t lod = 0u; lod < reader->getLODCount(); ++lod) {
            const auto indices = reader->getJointVariableAttributeIndices(lod);
            variableAttributeIndices[lod].reserve(indices.size());
            Vector<bool> markers(reader->getJointCount(), false, memRes);
            for (const auto absAttrIndex : indices) {
                const auto jointIndex = static_cast<std::uint16_t>(absAttrIndex / 9);
                const auto relAttrIndex = static_cast<std::uint8_t>(absAttrIndex % 9);
                const auto remappedBaseIndex = static_cast<std::uint16_t>(jointIndex * numAttrsPerJoint);
                // Within the DNA, the structure is always fixed [tx, ty, tz, rx, ry, rz, sx, sy, sz]
                if (relAttrIndex < 3) {
                    const auto remapped = static_cast<std::uint16_t>(remappedBaseIndex + translationOffset + (relAttrIndex % 3));
                    variableAttributeIndices[lod].push_back(remapped);
                } else if (relAttrIndex < 6) {
                    // Even if only a single rotation attribute is variable by DNA definition, when working with quaternions
                    // all four attributes must be provided
                    if (!markers[jointIndex]) {
                        const auto attrBase = static_cast<std::uint16_t>(remappedBaseIndex + rotationOffset);
                        std::uint16_t attrIndices[4] = {attrBase,
                                                        static_cast<std::uint16_t>(attrBase + 1),
                                                        static_cast<std::uint16_t>(attrBase + 2),
                                                        static_cast<std::uint16_t>(attrBase + 3)};
                        variableAttributeIndices[lod].insert(variableAttributeIndices[lod].end(),
                                                             std::begin(attrIndices),
                                                             std::end(attrIndices));
                        markers[jointIndex] = true;
                    }
                } else {
                    const auto remapped = static_cast<std::uint16_t>(remappedBaseIndex + scaleOffset + (relAttrIndex % 3));
                    variableAttributeIndices[lod].push_back(remapped);
                }
            }
        }
    }

    return variableAttributeIndices;
}

static Matrix<std::uint16_t> copyJointIndices(const Configuration& /*unused*/,
                                              const dna::Reader* reader,
                                              MemoryResource* memRes) {
    Matrix<std::uint16_t> jointIndices{memRes};
    jointIndices.resize(reader->getLODCount());
    for (std::uint16_t lod = {}; lod < reader->getLODCount(); ++lod) {
        const auto indicesForLOD = reader->getJointIndicesForLOD(lod);
        jointIndices[lod].assign(indicesForLOD.begin(), indicesForLOD.end());
    }
    return jointIndices;
}

Joints::Pointer JointsFactory::create(const Configuration& config,
                                      RigMetadata* meta,
                                      const dna::Reader* reader,
                                      Controls* controls,
                                      MemoryResource* memRes) {
    if (!config.loadJoints || (reader->getLODCount() == 0u) || (reader->getJointCount() == 0u)) {
        auto evaluator = UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
        return UniqueInstance<Joints>::with(memRes).create(std::move(evaluator), meta, memRes);
    }

    JointBehaviorFilter filter{reader, memRes};
    filter.include(dna::TranslationRepresentation::Vector);
    filter.include(dna::RotationRepresentation::EulerAngles);
    filter.include(dna::RotationRepresentation::Quaternion);
    filter.include(dna::ScaleRepresentation::Vector);

    auto builder = JointsBuilder::create(config, meta, memRes);
    builder->computeStorageRequirements(filter);
    builder->allocateStorage(filter);
    builder->fillStorage(filter);
    builder->registerControls(controls);
    auto evaluator = builder->build();
    auto neutralValues = copyNeutralValues(config, meta, reader, memRes);
    auto variableAttributeIndices = copyVariableAttributeIndices(config, reader, memRes);
    auto jointIndices = copyJointIndices(config, reader, memRes);
    return UniqueInstance<Joints>::with(memRes).create(std::move(evaluator),
                                                       std::move(neutralValues),
                                                       std::move(variableAttributeIndices),
                                                       std::move(jointIndices),
                                                       filter.getJointGroupCount());
}

Joints::Pointer JointsFactory::create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes) {
    if (!config.loadJoints || (meta->lodCount == 0u) || (meta->jointAttributeCount == 0u)) {
        auto evaluator = UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
        return UniqueInstance<Joints>::with(memRes).create(std::move(evaluator), meta, memRes);
    }

    auto builder = JointsBuilder::create(config, meta, memRes);
    builder->computeStorageRequirements();
    auto evaluator = builder->build();
    return UniqueInstance<Joints>::with(memRes).create(std::move(evaluator), meta, memRes);
}

}  // namespace rl4
