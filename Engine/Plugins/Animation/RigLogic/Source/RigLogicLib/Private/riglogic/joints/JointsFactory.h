// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/Joints.h"

namespace rl4 {

struct Configuration;
struct RigMetadata;
class Controls;

struct JointsFactory {
    static Joints::Pointer
    create(const Configuration& config, RigMetadata* meta, const dna::Reader* reader, Controls* controls, MemoryResource* memRes);
    static Joints::Pointer create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes);
};

}  // namespace rl4
