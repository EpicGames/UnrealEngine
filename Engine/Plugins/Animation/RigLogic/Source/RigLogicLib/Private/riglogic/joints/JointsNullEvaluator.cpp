// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/JointsNullEvaluator.h"

namespace rl4 {

JointsOutputInstance::Pointer JointsNullEvaluator::createInstance(MemoryResource* instanceMemRes) const {
    return UniqueInstance<JointsNullOutputInstance, JointsOutputInstance>::with(instanceMemRes).create();
}

std::uint32_t JointsNullEvaluator::getJointDeltaValueCountForLOD(std::uint16_t /*unused*/) const {
    return {};
}

void JointsNullEvaluator::calculate(ControlsInputInstance* /*unused*/,
                                    JointsOutputInstance* /*unused*/,
                                    std::uint16_t /*unused*/) const {
}

void JointsNullEvaluator::calculate(ControlsInputInstance* /*unused*/,
                                    JointsOutputInstance* /*unused*/,
                                    std::uint16_t /*unused*/,
                                    std::uint16_t /*unused*/) const {
}

void JointsNullEvaluator::load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) {
}

void JointsNullEvaluator::save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) {
}

}  // namespace rl4
