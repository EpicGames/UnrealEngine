// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsNullOutputInstance.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class JointsNullEvaluator : public JointsEvaluator {
public:
    JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t /*unused*/) const override;
    void calculate(ControlsInputInstance* /*unused*/, JointsOutputInstance* /*unused*/, std::uint16_t /*unused*/) const override;
    void calculate(ControlsInputInstance* /*unused*/,
                   JointsOutputInstance* /*unused*/,
                   std::uint16_t /*unused*/,
                   std::uint16_t /*unused*/) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) override;
};

}  // namespace rl4
