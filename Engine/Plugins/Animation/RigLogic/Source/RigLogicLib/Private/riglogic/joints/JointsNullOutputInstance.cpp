// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/JointsNullOutputInstance.h"

namespace rl4 {

ArrayView<float> JointsNullOutputInstance::getOutputBuffer() {
    return {};
}

void JointsNullOutputInstance::resetOutputBuffer() {
}

}  // namespace rl4
