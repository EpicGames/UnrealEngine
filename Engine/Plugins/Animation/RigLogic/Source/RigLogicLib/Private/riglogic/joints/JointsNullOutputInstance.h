// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/JointsOutputInstance.h"

namespace rl4 {

class JointsNullOutputInstance : public JointsOutputInstance {
public:
    ArrayView<float> getOutputBuffer() override;
    void resetOutputBuffer() override;
};

}  // namespace rl4
