// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/JointsOutputInstance.h"

namespace rl4 {

JointsOutputInstance::~JointsOutputInstance() = default;

}  // namespace rl4
