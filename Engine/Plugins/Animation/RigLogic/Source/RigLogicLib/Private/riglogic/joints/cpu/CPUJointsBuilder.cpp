// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/CPUJointsBuilder.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsEvaluator.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsBuilderFactory.h"
#include "riglogic/joints/cpu/ml/MLJointsBuilderFactory.h"
#include "riglogic/joints/cpu/quaternions/QuaternionJointsBuilderFactory.h"
#include "riglogic/joints/cpu/twistswing/TwistSwingJointsBuilderFactory.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Detect.h"

namespace rl4 {

CPUJointsBuilder::CPUJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_) :
    memRes{memRes_},
    config{config_},
    meta{meta_},
    bpcmBuilder{},
    quaternionBuilder{},
    twistSwingBuilder{},
    mlBuilder{},
    numAttrsPerJoint{} {

    numAttrsPerJoint =
        static_cast<std::uint16_t>(static_cast<std::uint8_t>(config.translationType) +
                                   static_cast<std::uint8_t>(config.rotationType) + static_cast<std::uint8_t>(config.scaleType));
    bpcmBuilder = BPCMJointsBuilderFactory::create(config, meta, memRes);
    quaternionBuilder = QuaternionJointsBuilderFactory::create(config, meta, memRes);
    twistSwingBuilder = TwistSwingJointsBuilderFactory::create(config, meta, memRes);
    mlBuilder = MLJointsBuilderFactory::create(config, meta, memRes);
}

void CPUJointsBuilder::computeStorageRequirements() {
    bpcmBuilder->computeStorageRequirements();
    quaternionBuilder->computeStorageRequirements();
    twistSwingBuilder->computeStorageRequirements();
    mlBuilder->computeStorageRequirements();
}

void CPUJointsBuilder::computeStorageRequirements(const JointBehaviorFilter& source) {
    bpcmBuilder->computeStorageRequirements(source.excluded(dna::RotationRepresentation::Quaternion));
    quaternionBuilder->computeStorageRequirements(source.only(dna::RotationRepresentation::Quaternion));
    twistSwingBuilder->computeStorageRequirements(source);
    mlBuilder->computeStorageRequirements(source);
}

void CPUJointsBuilder::allocateStorage(const JointBehaviorFilter& source) {
    bpcmBuilder->allocateStorage(source.excluded(dna::RotationRepresentation::Quaternion));
    quaternionBuilder->allocateStorage(source.only(dna::RotationRepresentation::Quaternion));
    twistSwingBuilder->allocateStorage(source);
    mlBuilder->allocateStorage(source);
}

void CPUJointsBuilder::fillStorage(const JointBehaviorFilter& source) {
    bpcmBuilder->fillStorage(source.excluded(dna::RotationRepresentation::Quaternion));
    quaternionBuilder->fillStorage(source.only(dna::RotationRepresentation::Quaternion));
    twistSwingBuilder->fillStorage(source);
    mlBuilder->fillStorage(source);
}

void CPUJointsBuilder::registerControls(Controls* controls) {
    bpcmBuilder->registerControls(controls);
    quaternionBuilder->registerControls(controls);
    twistSwingBuilder->registerControls(controls);
    mlBuilder->registerControls(controls);
}

JointsEvaluator::Pointer CPUJointsBuilder::build() {
    const auto jointAttributeCount = meta->jointAttributeCount;
    const auto translationType = config.translationType;
    const auto rotationType = config.rotationType;
    const auto scaleType = config.scaleType;
    auto instanceFactory = [jointAttributeCount, translationType, rotationType, scaleType](MemoryResource* instanceMemRes) {
        return UniqueInstance<CPUJointsOutputInstance, JointsOutputInstance>::with(instanceMemRes)
            .create(jointAttributeCount, translationType, rotationType, scaleType, instanceMemRes);
    };
    auto bpcmEvaluator = bpcmBuilder->build();
    auto quaternionEvaluator = quaternionBuilder->build();
    auto twistSwingEvaluator = twistSwingBuilder->build();
    auto mlEvaluator = mlBuilder->build();
    auto factory = UniqueInstance<CPUJointsEvaluator, JointsEvaluator>::with(memRes);
    return factory.create(std::move(bpcmEvaluator),
                          std::move(quaternionEvaluator),
                          std::move(twistSwingEvaluator),
                          std::move(mlEvaluator),
                          instanceFactory);
}

}  // namespace rl4
