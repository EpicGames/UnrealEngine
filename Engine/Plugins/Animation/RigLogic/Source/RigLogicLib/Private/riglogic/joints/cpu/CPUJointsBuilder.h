// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/riglogic/Configuration.h"

#include <cstdint>

namespace rl4 {

class CPUJointsBuilder : public JointsBuilder {
public:
    CPUJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_);
    void computeStorageRequirements() override;
    void computeStorageRequirements(const JointBehaviorFilter& source) override;
    void allocateStorage(const JointBehaviorFilter& source) override;
    void fillStorage(const JointBehaviorFilter& source) override;
    void registerControls(Controls* controls) override;
    JointsEvaluator::Pointer build() override;

private:
    MemoryResource* memRes;
    Configuration config;
    RigMetadata* meta;
    UniqueInstance<JointsBuilder>::PointerType bpcmBuilder;
    UniqueInstance<JointsBuilder>::PointerType quaternionBuilder;
    UniqueInstance<JointsBuilder>::PointerType twistSwingBuilder;
    UniqueInstance<JointsBuilder>::PointerType mlBuilder;
    std::uint16_t numAttrsPerJoint;
};

}  // namespace rl4
