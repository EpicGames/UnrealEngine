// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/CPUJointsEvaluator.h"

#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"

namespace rl4 {

CPUJointsEvaluator::CPUJointsEvaluator(JointsEvaluator::Pointer bpcmEvaluator_,
                                       JointsEvaluator::Pointer quaternionEvaluator_,
                                       JointsEvaluator::Pointer twistSwingEvaluator_,
                                       JointsEvaluator::Pointer mlEvaluator_,
                                       JointsOutputInstance::Factory instanceFactory_) :
    bpcmEvaluator{std::move(bpcmEvaluator_)},
    quaternionEvaluator{std::move(quaternionEvaluator_)},
    twistSwingEvaluator{std::move(twistSwingEvaluator_)},
    mlEvaluator{std::move(mlEvaluator_)},
    instanceFactory{instanceFactory_} {
}

JointsOutputInstance::Pointer CPUJointsEvaluator::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(instanceMemRes);
}

std::uint32_t CPUJointsEvaluator::getJointDeltaValueCountForLOD(std::uint16_t lod) const {
    const auto bpcmDeltaCount = bpcmEvaluator->getJointDeltaValueCountForLOD(lod);
    const auto quaternionDeltaCount = quaternionEvaluator->getJointDeltaValueCountForLOD(lod);
    return bpcmDeltaCount + quaternionDeltaCount;
}

void CPUJointsEvaluator::calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const {
    outputs->resetOutputBuffer();
    bpcmEvaluator->calculate(inputs, outputs, lod);
    quaternionEvaluator->calculate(inputs, outputs, lod);
    twistSwingEvaluator->calculate(inputs, outputs, lod);
    mlEvaluator->calculate(inputs, outputs, lod);
}

void CPUJointsEvaluator::calculate(ControlsInputInstance* inputs,
                                   JointsOutputInstance* outputs,
                                   std::uint16_t lod,
                                   std::uint16_t jointGroupIndex) const {
    bpcmEvaluator->calculate(inputs, outputs, lod, jointGroupIndex);
    quaternionEvaluator->calculate(inputs, outputs, lod, jointGroupIndex);
    // No twist swing or ML evaluation per joint group
}

void CPUJointsEvaluator::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    bpcmEvaluator->load(archive);
    quaternionEvaluator->load(archive);
    twistSwingEvaluator->load(archive);
    mlEvaluator->load(archive);
}

void CPUJointsEvaluator::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    bpcmEvaluator->save(archive);
    quaternionEvaluator->save(archive);
    twistSwingEvaluator->save(archive);
    mlEvaluator->save(archive);
}

}  // namespace rl4
