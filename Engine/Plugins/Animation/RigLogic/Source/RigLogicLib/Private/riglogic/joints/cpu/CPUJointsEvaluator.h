// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"

#include <cstdint>

namespace rl4 {

class CPUJointsEvaluator : public JointsEvaluator {
public:
    struct Accessor;
    friend Accessor;

public:
    CPUJointsEvaluator(JointsEvaluator::Pointer bpcmEvaluator_,
                       JointsEvaluator::Pointer quaternionEvaluator_,
                       JointsEvaluator::Pointer twistSwingEvaluator_,
                       JointsEvaluator::Pointer mlEvaluator_,
                       JointsOutputInstance::Factory instanceFactory_);
    JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs,
                   JointsOutputInstance* outputs,
                   std::uint16_t lod,
                   std::uint16_t jointGroupIndex) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    JointsEvaluator::Pointer bpcmEvaluator;
    JointsEvaluator::Pointer quaternionEvaluator;
    JointsEvaluator::Pointer twistSwingEvaluator;
    JointsEvaluator::Pointer mlEvaluator;
    JointsOutputInstance::Factory instanceFactory;
};

}  // namespace rl4
