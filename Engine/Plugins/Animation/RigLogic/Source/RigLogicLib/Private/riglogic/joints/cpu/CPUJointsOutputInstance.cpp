// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"

#include "riglogic/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstddef>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

CPUJointsOutputInstance::CPUJointsOutputInstance(std::uint16_t jointAttributeCount,
                                                 TranslationType translationType,
                                                 RotationType rotationType,
                                                 ScaleType scaleType,
                                                 MemoryResource* memRes) :
    outputBuffer{jointAttributeCount, {}, memRes} {

    if (rotationType == RotationType::Quaternions) {
        const auto numAttrsPerJoint =
            static_cast<std::size_t>(static_cast<std::uint8_t>(translationType) + static_cast<std::uint8_t>(rotationType) +
                                     static_cast<std::uint8_t>(scaleType));
        resetBuffer = [numAttrsPerJoint, translationType](ArrayView<float> buffer) {
            // For quaternions, set the w component to 1.0, so the empty buffer will contain identity quaternions
            const auto qwOffset = static_cast<std::size_t>(static_cast<std::uint8_t>(translationType) + 3u);
            std::fill(buffer.begin(), buffer.end(), 0.0f);
            for (std::size_t i = qwOffset; i < buffer.size(); i += numAttrsPerJoint) {
                buffer[i] = 1.0f;
            }
        };
    } else {
        resetBuffer = [](ArrayView<float> buffer) { std::fill(buffer.begin(), buffer.end(), 0.0f); };
    }

    resetBuffer(outputBuffer);
}

ArrayView<float> CPUJointsOutputInstance::getOutputBuffer() {
    return outputBuffer;
}

void CPUJointsOutputInstance::resetOutputBuffer() {
    resetBuffer(outputBuffer);
}

}  // namespace rl4
