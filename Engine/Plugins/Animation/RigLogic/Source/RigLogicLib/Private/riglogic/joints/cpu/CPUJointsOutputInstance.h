// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/riglogic/Configuration.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

class CPUJointsOutputInstance : public JointsOutputInstance {
public:
    CPUJointsOutputInstance(std::uint16_t jointAttributeCount,
                            TranslationType translationType,
                            RotationType rotationType,
                            ScaleType scaleType,
                            MemoryResource* memRes);
    ArrayView<float> getOutputBuffer() override;
    void resetOutputBuffer() override;

private:
    AlignedVector<float> outputBuffer;
    std::function<void(ArrayView<float>)> resetBuffer;
};

}  // namespace rl4
