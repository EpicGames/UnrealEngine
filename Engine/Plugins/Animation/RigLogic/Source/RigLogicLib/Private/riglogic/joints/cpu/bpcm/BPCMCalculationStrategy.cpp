// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/bpcm/BPCMCalculationStrategy.h"

namespace rl4 {

namespace bpcm {

JointGroupLinearCalculationStrategy::~JointGroupLinearCalculationStrategy() = default;

}  // namespace bpcm

}  // namespace rl4
