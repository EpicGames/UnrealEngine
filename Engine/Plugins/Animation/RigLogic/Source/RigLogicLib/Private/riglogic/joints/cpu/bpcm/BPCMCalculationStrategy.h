// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/cpu/bpcm/JointGroup.h"
#include "riglogic/joints/cpu/bpcm/Storage.h"
#include "riglogic/system/simd/SIMD.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <cstddef>
#include <cstdint>

// *INDENT-OFF*
namespace rl4 {

namespace bpcm {

/*
 * Process the remainder portion after 8x4 blocks
 *
 * Consumes 4 x 2 values per input index, but input indexes are processed one
 * by one. The maximum number of iterations performed is 3, as anything larger
 * would be already consumed by the 8x4 block processor. It's executed only
 * after the 8x4 block processor in cases when the matrix column count is not
 * divisible by 8.
 *
 *  [ x x x x x x x x X Y Z ] Input Vector
 *
 *    0 1 2 3 4 5 6 7 8 9 0
 *  0 x x x x x x x x A C E  |  X * A + Y * C + Z * E
 *  1 x x x x x x x x A C E  |  X * A + Y * C + Z * E
 *  2 x x x x x x x x A C E  |  X * A + Y * C + Z * E
 *  3 x x x x x x x x A C E  |  X * A + Y * C + Z * E
 *  4 x x x x x x x x B D F  |  X * B + Y * D + Z * F
 *  5 x x x x x x x x B D F  |  X * B + Y * D + Z * F
 *  6 x x x x x x x x B D F  |  X * B + Y * D + Z * F
 *  7 x x x x x x x x B D F  |  X * B + Y * D + Z * F
 *  8 x x x x x x x x x x x  |
 *  9 x x x x x x x x x x x  |
 * 10 x x x x x x x x x x x  |
 * 11 x x x x x x x x x x x  |
 *
 */
template<typename TFVec, typename T>
static FORCE_INLINE void processBlocks8x1(const std::uint16_t* inputIndicesEndAlignedTo4,
                                          const std::uint16_t* inputIndicesEnd,
                                          ConstArrayView<float> inputs,
                                          const T* values,
                                          TFVec& sum1,
                                          TFVec& sum2) {
    for (const std::uint16_t* inputIndices = inputIndicesEndAlignedTo4; inputIndices < inputIndicesEnd;
         ++inputIndices, values += (2ul * TFVec::size())) {
        const TFVec inputVec{inputs[*inputIndices]};
        const TFVec blk1 = TFVec::fromAlignedSource(values);
        const TFVec blk2 = TFVec::fromAlignedSource(values + TFVec::size());
        sum1 += (blk1 * inputVec);
        sum2 += (blk2 * inputVec);
    }
}

/*
 * Process the main portion of the matrix that's partitionable into 8x4 blocks
 *
 * Consumes 4 x 2 values per input index, and loads 4 input indexes per iteration,
 * thus a single iteration processes 32 values. This is the most performant of all
 * the block processors as for every 4 x 2 values only a single input vector needs
 * to be filled (which is expensive as it involves a lot of shuffle instructions).
 *
 *  [ S T U V W X Y Z x x x ] Input Vector
 *
 *    0 1 2 3 4 5 6 7 8 9 0
 *  0 A C E G I K M O x x x  |  S * A + T * C + U * E + V * G + W * I + X * K + Y * M + Z * O
 *  1 A C E G I K M O x x x  |  S * A + T * C + U * E + V * G + W * I + X * K + Y * M + Z * O
 *  2 A C E G I K M O x x x  |  S * A + T * C + U * E + V * G + W * I + X * K + Y * M + Z * O
 *  3 A C E G I K M O x x x  |  S * A + T * C + U * E + V * G + W * I + X * K + Y * M + Z * O
 *  4 B D F H J L N P x x x  |  S * B + T * D + U * F + V * H + W * J + X * L + Y * N + Z * P
 *  5 B D F H J L N P x x x  |  S * B + T * D + U * F + V * H + W * J + X * L + Y * N + Z * P
 *  6 B D F H J L N P x x x  |  S * B + T * D + U * F + V * H + W * J + X * L + Y * N + Z * P
 *  7 B D F H J L N P x x x  |  S * B + T * D + U * F + V * H + W * J + X * L + Y * N + Z * P
 *  8 x x x x x x x x x x x  |
 *  9 x x x x x x x x x x x  |
 * 10 x x x x x x x x x x x  |
 * 11 x x x x x x x x x x x  |
 *
 */
template<typename TFVec, typename T>
static FORCE_INLINE void processBlocks8x4(const std::uint16_t* inputIndicesStart,
                                          const std::uint16_t* inputIndicesEndAlignedTo4,
                                          const std::uint16_t* inputIndicesEnd,
                                          ConstArrayView<float> inputs,
                                          const T* values,
                                          float* outbuf) {
    TFVec sum1{};
    TFVec sum2{};
    TFVec sum3{};
    TFVec sum4{};
    TFVec sum5{};
    TFVec sum6{};
    TFVec sum7{};
    TFVec sum8{};
    for (const std::uint16_t* inputIndices = inputIndicesStart; inputIndices < inputIndicesEndAlignedTo4;
         inputIndices += 4ul, values += (8ul * TFVec::size())) {
        const TFVec inputVec1{inputs[inputIndices[0]]};
        const TFVec inputVec2{inputs[inputIndices[1]]};
        const TFVec inputVec3{inputs[inputIndices[2]]};
        const TFVec inputVec4{inputs[inputIndices[3]]};
        const TFVec blk1 = TFVec::fromAlignedSource(values);
        const TFVec blk2 = TFVec::fromAlignedSource(values + TFVec::size());
        const TFVec blk3 = TFVec::fromAlignedSource(values + TFVec::size() * 2);
        const TFVec blk4 = TFVec::fromAlignedSource(values + TFVec::size() * 3);
        const TFVec blk5 = TFVec::fromAlignedSource(values + TFVec::size() * 4);
        const TFVec blk6 = TFVec::fromAlignedSource(values + TFVec::size() * 5);
        const TFVec blk7 = TFVec::fromAlignedSource(values + TFVec::size() * 6);
        const TFVec blk8 = TFVec::fromAlignedSource(values + TFVec::size() * 7);
        sum1 += (blk1 * inputVec1);
        sum2 += (blk2 * inputVec1);
        sum3 += (blk3 * inputVec2);
        sum4 += (blk4 * inputVec2);
        sum5 += (blk5 * inputVec3);
        sum6 += (blk6 * inputVec3);
        sum7 += (blk7 * inputVec4);
        sum8 += (blk8 * inputVec4);
    }
    // Process 8x1 horizontal remainder portion after 8x4 blocks are consumed
    processBlocks8x1(inputIndicesEndAlignedTo4, inputIndicesEnd, inputs, values, sum1, sum2);

    sum1 += sum3;
    sum2 += sum4;
    sum5 += sum7;
    sum6 += sum8;
    sum1 += sum5;
    sum2 += sum6;

    sum1.alignedStore(outbuf);
    sum2.alignedStore(outbuf + TFVec::size());
}

/*
 * Process the remainder portion after 4x8 blocks
 *
 * Consumes 4 x 1 values per input index, but input indexes are processed one
 * by one. The maximum number of iterations performed is 7, as anything larger
 * would be already consumed by the 4x8 block processor. It's executed only
 * after the 4x8 block processor, in cases when the matrix column count is not
 * divisible by 8.
 *
 *  [ x x x x x x x x T U V W X Y Z ] Input Vector
 *
 *    0 1 2 3 4 5 6 7 8 9 0 1 2 3 4
 *  0 x x x x x x x x x x x x x x x  |
 *  1 x x x x x x x x x x x x x x x  |
 *  2 x x x x x x x x x x x x x x x  |
 *  3 x x x x x x x x x x x x x x x  |
 *  4 x x x x x x x x x x x x x x x  |
 *  5 x x x x x x x x x x x x x x x  |
 *  6 x x x x x x x x x x x x x x x  |
 *  7 x x x x x x x x x x x x x x x  |
 *  8 x x x x x x x x A B C D E F G  |  T * A + U * B + V * C + W * D + X * E + Y * F + Z * G
 *  9 x x x x x x x x A B C D E F G  |  T * A + U * B + V * C + W * D + X * E + Y * F + Z * G
 * 10 x x x x x x x x A B C D E F G  |  T * A + U * B + V * C + W * D + X * E + Y * F + Z * G
 * 11 x x x x x x x x A B C D E F G  |  T * A + U * B + V * C + W * D + X * E + Y * F + Z * G
 *
 */
template<typename TFVec, typename T>
static FORCE_INLINE void processBlocks4x1(const std::uint16_t* inputIndicesEndAlignedTo8,
                                          const std::uint16_t* inputIndicesEnd,
                                          ConstArrayView<float> inputs,
                                          const T* values,
                                          TFVec& sum1) {
    for (const std::uint16_t* inputIndices = inputIndicesEndAlignedTo8; inputIndices < inputIndicesEnd;
         ++inputIndices, values += TFVec::size()) {
        const TFVec inputVec{inputs[*inputIndices]};
        const TFVec blk = TFVec::fromAlignedSource(values);
        sum1 += (blk * inputVec);
    }
}

/*
 * Process the vertical remainder portion of the matrix that's partitionable into 4x8 blocks
 *
 * Consumes 4 x 1 values per input index, and loads 8 input indexes per
 * iteration, thus a single iteration processes 32 values. Although it
 * processes the same number of values per iteration as the 8x4 block
 * processor, it does so by having to fill 8 input vectors (one for each
 * 4 x 1 value-block), thus requiring twice the number of shuffle instructions
 * which makes it somewhat less performant than the 8x4 variant. It gets
 * executed only in cases when the matrix row count is not divisible by 8.
 *
 *  [ S T U V W X Y Z x x x ] Input Vector
 *
 *    0 1 2 3 4 5 6 7 8 9 0
 *  0 x x x x x x x x x x x  |
 *  1 x x x x x x x x x x x  |
 *  2 x x x x x x x x x x x  |
 *  3 x x x x x x x x x x x  |
 *  4 x x x x x x x x x x x  |
 *  5 x x x x x x x x x x x  |
 *  6 x x x x x x x x x x x  |
 *  7 x x x x x x x x x x x  |
 *  8 A B C D E F G H x x x  |  S * A + T * B + U * C + V * D + W * E + X * F + Y * G + Z * H
 *  9 A B C D E F G H x x x  |  S * A + T * B + U * C + V * D + W * E + X * F + Y * G + Z * H
 * 10 A B C D E F G H x x x  |  S * A + T * B + U * C + V * D + W * E + X * F + Y * G + Z * H
 * 11 A B C D E F G H x x x  |  S * A + T * B + U * C + V * D + W * E + X * F + Y * G + Z * H
 *
 */
template<typename TFVec, typename T>
static FORCE_INLINE void processBlocks4x8(const std::uint16_t* inputIndicesStart,
                                          const std::uint16_t* inputIndicesEndAlignedTo8,
                                          const std::uint16_t* inputIndicesEnd,
                                          ConstArrayView<float> inputs,
                                          const T* values,
                                          float* outbuf) {
    TFVec sum1{};
    TFVec sum2{};
    TFVec sum3{};
    TFVec sum4{};
    TFVec sum5{};
    TFVec sum6{};
    TFVec sum7{};
    TFVec sum8{};
    for (const std::uint16_t* inputIndices = inputIndicesStart; inputIndices < inputIndicesEndAlignedTo8;
         inputIndices += 8ul, values += (TFVec::size() * 8ul)) {
        const TFVec inputVec1{inputs[inputIndices[0]]};
        const TFVec inputVec2{inputs[inputIndices[1]]};
        const TFVec inputVec3{inputs[inputIndices[2]]};
        const TFVec inputVec4{inputs[inputIndices[3]]};
        const TFVec inputVec5{inputs[inputIndices[4]]};
        const TFVec inputVec6{inputs[inputIndices[5]]};
        const TFVec inputVec7{inputs[inputIndices[6]]};
        const TFVec inputVec8{inputs[inputIndices[7]]};
        const TFVec blk1 = TFVec::fromAlignedSource(values);
        const TFVec blk2 = TFVec::fromAlignedSource(values + TFVec::size());
        const TFVec blk3 = TFVec::fromAlignedSource(values + TFVec::size() * 2);
        const TFVec blk4 = TFVec::fromAlignedSource(values + TFVec::size() * 3);
        const TFVec blk5 = TFVec::fromAlignedSource(values + TFVec::size() * 4);
        const TFVec blk6 = TFVec::fromAlignedSource(values + TFVec::size() * 5);
        const TFVec blk7 = TFVec::fromAlignedSource(values + TFVec::size() * 6);
        const TFVec blk8 = TFVec::fromAlignedSource(values + TFVec::size() * 7);
        sum1 += (blk1 * inputVec1);
        sum2 += (blk2 * inputVec2);
        sum3 += (blk3 * inputVec3);
        sum4 += (blk4 * inputVec4);
        sum5 += (blk5 * inputVec5);
        sum6 += (blk6 * inputVec6);
        sum7 += (blk7 * inputVec7);
        sum8 += (blk8 * inputVec8);
    }
    // Process 4x1 horizontal remainder portion after 4x8 blocks are consumed
    processBlocks4x1(inputIndicesEndAlignedTo8, inputIndicesEnd, inputs, values, sum1);

    sum1 += sum2;
    sum3 += sum4;
    sum5 += sum6;
    sum7 += sum8;
    sum1 += sum3;
    sum5 += sum7;
    sum1 += sum5;

    sum1.alignedStore(outbuf);
}

/*
 * Orchestrate the execution of the needed block processors for a given joint group
 */
template<typename TFVec, typename T>
static FORCE_INLINE void processJointGroupBlock4(const JointGroupView& jointGroup,
                                                 ConstArrayView<float> inputs,
                                                 ArrayView<float> outputs,
                                                 std::uint16_t lod) {
    const T* values = static_cast<const T*>(jointGroup.values);
    const LODRegion& lodRegion = jointGroup.lods[lod];
    const std::uint16_t* const inputIndices = jointGroup.inputIndices;
    const std::uint16_t* const inputIndicesEnd = inputIndices + lodRegion.inputLODs.size;
    const std::uint16_t* const inputIndicesEndAlignedTo4 = inputIndices + lodRegion.inputLODs.sizeAlignedTo4;
    const std::uint16_t* const inputIndicesEndAlignedTo8 = inputIndices + lodRegion.inputLODs.sizeAlignedTo8;
    const std::uint16_t* outputIndices = jointGroup.outputIndices;
    const std::uint16_t* const outputIndicesEnd = outputIndices + lodRegion.outputLODs.size;
    const std::uint16_t* const outputIndicesEndPaddedToLastFullBlock =
        outputIndices + lodRegion.outputLODs.sizePaddedToLastFullBlock;
    const std::uint16_t* const outputIndicesEndPaddedToSecondLastFullBlock =
        outputIndices + lodRegion.outputLODs.sizePaddedToSecondLastFullBlock;
    constexpr std::size_t halfBlockHeight = TFVec::size();
    constexpr std::size_t fullBlockHeight = 2ul * TFVec::size();
    const std::size_t halfBlockSize = jointGroup.colCount * halfBlockHeight;
    const std::size_t fullBlockSize = jointGroup.colCount * fullBlockHeight;
    // Process portion of matrix that's partitionable into 8x4 blocks
    for (; outputIndices < outputIndicesEndPaddedToSecondLastFullBlock;
         outputIndices += fullBlockHeight, values += fullBlockSize) {
        alignas(TFVec::alignment()) float outbuf[fullBlockHeight];
        processBlocks8x4<TFVec>(inputIndices,
                                inputIndicesEndAlignedTo4,
                                inputIndicesEnd,
                                inputs,
                                values,
                                static_cast<float*>(outbuf));
        for (std::size_t i = 0ul; i < fullBlockHeight; ++i) {
            outputs[outputIndices[i]] = outbuf[i];
        }
    }
    // Process the last 8x4 block which needs special handling when some of
    // the output values need to be masked-off because of LODs
    for (; outputIndices < outputIndicesEndPaddedToLastFullBlock; outputIndices += fullBlockHeight, values += fullBlockSize) {
        alignas(TFVec::alignment()) float outbuf[fullBlockHeight];
        processBlocks8x4<TFVec>(inputIndices,
                                inputIndicesEndAlignedTo4,
                                inputIndicesEnd,
                                inputs,
                                values,
                                static_cast<float*>(outbuf));
        // Ignore results that came from rows after the last LOD row
        for (std::size_t i = 0ul; i < (lodRegion.outputLODs.size % fullBlockHeight); ++i) {
            outputs[outputIndices[i]] = outbuf[i];
        }
    }
    // Process vertical remainder portion of matrix that's partitionable into 4x8 blocks
    for (; outputIndices < outputIndicesEnd; outputIndices += halfBlockHeight, values += halfBlockSize) {
        alignas(TFVec::alignment()) float outbuf[halfBlockHeight];
        processBlocks4x8<TFVec>(inputIndices,
                                inputIndicesEndAlignedTo8,
                                inputIndicesEnd,
                                inputs,
                                values,
                                static_cast<float*>(outbuf));
        // Ignore results that came from rows after the last LOD row
        auto maskOffStart = static_cast<std::size_t>(outputIndicesEnd - outputIndices);
        for (std::size_t i = 0; i < maskOffStart; ++i) {
            outputs[outputIndices[i]] = outbuf[i];
        }
    }
}

struct JointGroupLinearCalculationStrategy {
    virtual ~JointGroupLinearCalculationStrategy();

    virtual void calculate(const JointGroupView& jointGroup,
                           ConstArrayView<float> inputs,
                           ArrayView<float> outputs,
                           std::uint16_t lod) const = 0;
};

template<typename T, typename TFVec, class TRotationAdapter>
struct VectorizedJointGroupLinearCalculationStrategy : public JointGroupLinearCalculationStrategy {
    TRotationAdapter rotationAdapter;

    explicit VectorizedJointGroupLinearCalculationStrategy(TRotationAdapter&& rotationAdapter_) :
        rotationAdapter{std::move(rotationAdapter_)} {
    }

    void calculate(const JointGroupView& jointGroup,
                   ConstArrayView<float> inputs,
                   ArrayView<float> outputs,
                   std::uint16_t lod) const override {
        processJointGroupBlock4<TFVec, T>(jointGroup, inputs, outputs, lod);
        rotationAdapter.forward(jointGroup, outputs, lod);
    }
};

}  // namespace bpcm

}  // namespace rl4
// *INDENT-ON*
