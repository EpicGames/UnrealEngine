// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/bpcm/BPCMJointsBuilder.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsNullEvaluator.h"
#include "riglogic/joints/cpu/bpcm/BPCMCalculationStrategy.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/RotationAdapters.h"
#include "riglogic/joints/cpu/bpcm/Storage.h"
#include "riglogic/joints/cpu/utils/JointGroupOptimizer.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/types/bpcm/Optimizer.h"
#include "riglogic/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstddef>
#include <cstdint>
#include <functional>
#include <memory>
#include <type_traits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace bpcm {

template<typename TF256, typename TF128>
using TBPCMVec = typename std::conditional<std::is_same<TF256, trimd::fallback::T256<TF128>>::value, TF128, TF256>::type;

template<typename TFVec>
static constexpr std::uint32_t BlockHeight() {
    return static_cast<std::uint32_t>(TFVec::size() * 2ul);
}

template<typename TFVec>
static constexpr std::uint32_t PadTo() {
    return static_cast<std::uint32_t>(TFVec::size());
}

template<typename TFVec>
static constexpr std::uint32_t Stride() {
    return 1u;
}

template<typename TIterator>
static void remapOutputIndicesForQuaternions(TIterator begin, TIterator end) {
    for (auto it = begin; it != end; ++it) {
        const auto absAttrIndex = *it;
        const auto jointIndex = static_cast<std::uint16_t>(absAttrIndex / 9u);
        const auto relAttrIndex = static_cast<std::uint16_t>(absAttrIndex % 9u);
        const auto newAttrBase = static_cast<std::uint16_t>(jointIndex * 10u);
        // Only scale relative attribute index is offset by one when output is in quaternions
        const auto newRelAttrIndex = (relAttrIndex < 6u ? relAttrIndex : static_cast<std::uint16_t>(relAttrIndex + 1u));
        *it = static_cast<std::uint16_t>(newAttrBase + newRelAttrIndex);
    }
}

struct VectorizationParameters {
    std::uint32_t blockHeight;
    std::uint32_t padTo;
    std::uint32_t stride;
};

template<typename T, typename TF256, typename TF128>
struct DetectVectorizationParameters {

    VectorizationParameters operator()() {
        using TFVec = TBPCMVec<TF256, TF128>;
        VectorizationParameters params = {};
        params.blockHeight = BlockHeight<TFVec>();
        params.padTo = PadTo<TFVec>();
        params.stride = Stride<TFVec>();
        return params;
    }
};

template<typename T, typename TF256, typename TF128>
struct MatrixOptimizer {

    std::uint32_t operator()(ConstArrayView<float> src, Extent srcDims, Extent dstDims, FloatArray& dst, std::uint32_t offset) {
        dst.resize<T>(dst.size<T>() + dstDims.size());
        using TFVec = TBPCMVec<TF256, TF128>;
        using BPCMOptimizer = bpcm::Optimizer<TFVec, BlockHeight<TFVec>(), PadTo<TFVec>(), Stride<TFVec>()>;
        return BPCMOptimizer::optimize(dst.data<T>() + offset, src.data(), srcDims);
    }
};

template<typename T, typename TF256, typename TF128>
struct JointGroupLinearStrategyFactory {
    using BasePointer = UniqueInstance<JointGroupLinearCalculationStrategy>::PointerType;

    BasePointer operator()(RotationType rotationType,
                           tdm::rot_seq rotationSequence,
                           const tdm::rot_sign& rotationSigns,
                           dna::RotationUnit rotationUnit,
                           MemoryResource* memRes) {

        using TFVec = TBPCMVec<TF256, TF128>;

        if (rotationType == RotationType::EulerAngles) {
            using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, NoopAdapter>;
            return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                NoopAdapter{rotationSigns});
        }

#ifdef RL_BUILD_WITH_XYZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xyz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::xyz>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            } else {
                using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::xyz>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_XYZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_XZY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xzy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::xzy>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            } else {
                using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::xzy>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_XZY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YXZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yxz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::yxz>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            } else {
                using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::yxz>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_YXZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YZX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yzx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::yzx>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            } else {
                using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::yzx>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_YZX_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZXY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zxy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zxy>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            } else {
                using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::zxy>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_ZXY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZYX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zyx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            } else {
                using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::zyx>;
                using CalculationStrategy = VectorizedJointGroupLinearCalculationStrategy<T, TFVec, E2Q>;
                return UniqueInstance<CalculationStrategy, JointGroupLinearCalculationStrategy>::with(memRes).create(
                    E2Q{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_ZYX_ROTATION_ORDER

        return nullptr;
    }
};

BPCMJointsBuilder::BPCMJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_) :
    config{config_},
    meta{meta_},
    memRes{memRes_},
    storage{memRes},
    rotationUnit{},
    lodCount{} {
}

void BPCMJointsBuilder::computeStorageRequirements() {
}

void BPCMJointsBuilder::computeStorageRequirements(const JointBehaviorFilter& /*unused*/) {
}

void BPCMJointsBuilder::allocateStorage(const JointBehaviorFilter& source) {
    lodCount = source.getLODCount();
    storage.jointGroups.resize(source.getJointGroupCount());
    storage.lodRegions.resize(storage.jointGroups.size() * static_cast<std::size_t>(lodCount));
    if (config.rotationType == RotationType::Quaternions) {
        storage.outputRotationLODs.resize(storage.lodRegions.size());
    }
}

void BPCMJointsBuilder::fillStorage(const JointBehaviorFilter& source) {
    rotationUnit = source.getRotationUnit();

    Vector<float> values{memRes};
    Vector<std::uint16_t> inputIndices{memRes};
    Vector<std::uint16_t> outputIndices{memRes};
    std::uint32_t valueOffset = {};
    std::uint32_t inputOffset = {};
    std::uint32_t outputOffset = {};
    std::uint32_t lodOffset = {};
    for (std::uint16_t i = {}; i < source.getJointGroupCount(); ++i) {
        const auto colCount = static_cast<std::uint32_t>(source.getColumnCount(i));
        const auto rowCount = static_cast<std::uint32_t>(source.getRowCount(i));

        values.resize(rowCount * colCount);
        source.copyValues(i, values);

        inputIndices.resize(colCount);
        source.copyInputIndices(i, inputIndices);

        outputIndices.resize(rowCount);
        source.copyOutputIndices(i, outputIndices);

        // This step might reduce both value count and input index count (by eliminating empty columns)
        ArrayView<LODRegion> lods{storage.lodRegions.data() + lodOffset, source.getLODCount()};
        JointGroupOptimizer::defragment(source,
                                        i,
                                        values,
                                        inputIndices,
                                        outputIndices,
                                        lods,
                                        config.translationPruningThreshold,
                                        config.rotationPruningThreshold,
                                        config.scalePruningThreshold);
        const auto optimizedColCount = static_cast<std::uint32_t>(inputIndices.size());
        const auto optimizedRowCount = static_cast<std::uint32_t>(outputIndices.size());

        RuntimeTemplateInstantiator instantiator{&config};
        VectorizationParameters vecParams = instantiator.invoke<DetectVectorizationParameters, VectorizationParameters>();

        const auto padding = extd::roundUp(optimizedRowCount, vecParams.padTo) - optimizedRowCount;
        const auto paddedRowCount = optimizedRowCount + padding;
        // Compute padding and actual matrix size after optimization is done
        storage.jointGroups[i].valuesSize = optimizedColCount * paddedRowCount;
        storage.inputIndices.resize(storage.inputIndices.size() + inputIndices.size());
        storage.outputIndices.resize(storage.outputIndices.size() + paddedRowCount);

        // Needs to use the original unpadded rowcount with the number of optimized columns (that were eliminated)
        const Extent srcDims{optimizedRowCount, optimizedColCount};
        const Extent dstDims{paddedRowCount, optimizedColCount};
        storage.jointGroups[i].valuesOffset = valueOffset;
        valueOffset += instantiator.invoke<MatrixOptimizer, std::uint32_t>(values, srcDims, dstDims, storage.values, valueOffset);

        std::copy(inputIndices.begin(), inputIndices.end(), extd::advanced(storage.inputIndices.begin(), inputOffset));
        storage.jointGroups[i].inputIndicesOffset = inputOffset;
        inputOffset += optimizedColCount;

        std::copy(outputIndices.begin(), outputIndices.end(), extd::advanced(storage.outputIndices.begin(), outputOffset));
        storage.jointGroups[i].outputIndicesOffset = outputOffset;
        outputOffset += paddedRowCount;

        for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
            auto& lodRegion = lods[lod];
            lodRegion.outputLODs = RowLOD(lodRegion.outputLODs.size, paddedRowCount, vecParams.blockHeight, vecParams.padTo);
        }

        storage.jointGroups[i].lodsOffset = lodOffset;
        lodOffset += source.getLODCount();
        storage.jointGroups[i].colCount = optimizedColCount;
        storage.jointGroups[i].rowCount = paddedRowCount;
    }

    if (config.rotationType == RotationType::Quaternions) {
        // Remap output indices from 9-attribute joints to 10-attribute joints
        remapOutputIndicesForQuaternions(storage.outputIndices.begin(), storage.outputIndices.end());
        const JointBehaviorFilter filtered = source.only(dna::RotationRepresentation::EulerAngles);
        setOutputRotationIndices(filtered);
    }
}

void BPCMJointsBuilder::registerControls(Controls* controls) {
    for (const auto& group : storage.jointGroups) {
        ConstArrayView<LODRegion> lods{storage.lodRegions.data() + group.lodsOffset, lodCount};
        for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
            ConstArrayView<std::uint16_t> inputIndicesForLOD(storage.inputIndices.data() + group.inputIndicesOffset,
                                                             lods[lod].inputLODs.size);
            controls->registerControls(lod, inputIndicesForLOD);
        }
    }
}

void BPCMJointsBuilder::setOutputRotationIndices(const JointBehaviorFilter& source) {
    std::uint32_t outputOffset = {};
    for (std::uint16_t jgi = {}; jgi < source.getJointGroupCount(); ++jgi) {
        auto colCount = static_cast<std::uint32_t>(source.getColumnCount(jgi));
        auto rowCount = static_cast<std::uint32_t>(source.getRowCount(jgi));

        Vector<float> values{rowCount * colCount, {}, memRes};
        source.copyValues(jgi, values);

        Vector<std::uint16_t> inputIndices{colCount, {}, memRes};
        source.copyInputIndices(jgi, inputIndices);

        Vector<std::uint16_t> outputRotationIndices{rowCount, {}, memRes};
        source.copyOutputIndices(jgi, outputRotationIndices);

        Vector<LODRegion> lods{source.getLODCount(), {}, memRes};
        for (std::uint16_t lod = {}; lod < source.getLODCount(); ++lod) {
            lods[lod].outputLODs.size = source.getRowCountForLOD(jgi, lod);
        }
        JointGroupOptimizer::defragment(source,
                                        jgi,
                                        values,
                                        inputIndices,
                                        outputRotationIndices,
                                        lods,
                                        config.translationPruningThreshold,
                                        config.rotationPruningThreshold,
                                        config.scalePruningThreshold);

        // Remap output indices from 9-attribute joints to 10-attribute joints rx -> qx
        remapOutputIndicesForQuaternions(outputRotationIndices.begin(), outputRotationIndices.end());
// Given any rotation indices (qx, qy, qz), return only qx indices for all joints in the group
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wattributes"
#endif
        auto deduplicate = [this](Vector<std::uint16_t>& v) {
            UnorderedSet<std::uint16_t> deduplicator{memRes};
            v.erase(v.rend().base(), std::remove_if(v.rbegin(), v.rend(), [&deduplicator](const std::uint16_t value) {
                                         return !deduplicator.insert(value).second;
                                     }).base());
        };
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        Vector<std::uint16_t> outputRotationBaseIndices{memRes};
        outputRotationBaseIndices.reserve(outputRotationIndices.size() / 3ul);
        std::transform(outputRotationIndices.begin(),
                       outputRotationIndices.end(),
                       std::back_inserter(outputRotationBaseIndices),
                       [](std::uint16_t outputIndex) { return static_cast<std::uint16_t>((outputIndex / 10) * 10 + 3); });
        deduplicate(outputRotationBaseIndices);
        // Copy remapped qx indices into destination storage
        std::copy(outputRotationBaseIndices.begin(),
                  outputRotationBaseIndices.end(),
                  std::back_inserter(storage.outputRotationIndices));
        storage.jointGroups[jgi].outputRotationIndicesOffset = outputOffset;
        // Must be called before outputOffset is adjusted
        setOutputRotationLODs(lods, outputRotationIndices, outputOffset, jgi);
        outputOffset += static_cast<std::uint32_t>(outputRotationBaseIndices.size());
    }
}

void BPCMJointsBuilder::setOutputRotationLODs(ConstArrayView<LODRegion> lods,
                                              ConstArrayView<std::uint16_t> outputRotationIndices,
                                              std::uint32_t outputOffset,
                                              std::uint16_t jointGroupIndex) {
    const auto offset = static_cast<std::uint32_t>(jointGroupIndex * lodCount);
    for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
        const auto oldLODRowCount = lods[lod].outputLODs.size;
        if (oldLODRowCount == 0) {
            storage.outputRotationLODs[offset + lod] = 0;
            continue;
        }

        assert(oldLODRowCount <= outputRotationIndices.size());
        const auto qxRotationIndexAtOldLODRowCount = (outputRotationIndices[oldLODRowCount - 1ul] / 10) * 10 + 3;
        auto start = extd::advanced(storage.outputRotationIndices.begin(), outputOffset);
        auto it = std::find(start, storage.outputRotationIndices.end(), qxRotationIndexAtOldLODRowCount);
        assert(it != storage.outputRotationIndices.end());
        const auto newLODRowCount = static_cast<std::uint16_t>(std::distance(start, it) + 1);
        storage.outputRotationLODs[offset + lod] = newLODRowCount;
    }
    storage.jointGroups[jointGroupIndex].outputRotationLODsOffset = offset;
}

JointsEvaluator::Pointer BPCMJointsBuilder::build() {
    const EvaluatorType type =
        (meta->initializationMethod == InitializationMethod::Restore) ? meta->popFrontEvaluator() : EvaluatorType::Auto;

    auto jointGroupsEmpty = [this]() {
        for (std::size_t i = {}; i < storage.jointGroups.size(); ++i) {
            if (storage.jointGroups[i].rowCount != 0u) {
                return false;
            }
        }
        return true;
    };

    if ((type == EvaluatorType::Null) || ((type == EvaluatorType::Auto) && jointGroupsEmpty())) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    RuntimeTemplateInstantiator instantiator{&config};
    using StrategyPointer = UniqueInstance<JointGroupLinearCalculationStrategy>::PointerType;
    auto strategy = instantiator.invoke<JointGroupLinearStrategyFactory, StrategyPointer>(config.rotationType,
                                                                                          meta->rotationSequence,
                                                                                          meta->rotationSigns,
                                                                                          rotationUnit,
                                                                                          memRes);

    if (strategy == nullptr) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    auto jointGroups = instantiator.invoke<StorageSnapshot, Vector<JointGroupView>>(storage, memRes);
    auto factory = UniqueInstance<Evaluator, JointsEvaluator>::with(memRes);
    return factory.create(std::move(storage), std::move(jointGroups), std::move(strategy), nullptr, memRes);
}

}  // namespace bpcm

}  // namespace rl4
