// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/Storage.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"

namespace rl4 {

namespace bpcm {

class BPCMJointsBuilder : public JointsBuilder {
public:
    BPCMJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_);

    void computeStorageRequirements() override;
    void computeStorageRequirements(const JointBehaviorFilter& /*unused*/) override;
    void allocateStorage(const JointBehaviorFilter& source) override;
    void fillStorage(const JointBehaviorFilter& source) override;
    void registerControls(Controls* controls) override;
    JointsEvaluator::Pointer build() override;

private:
    void setOutputRotationIndices(const JointBehaviorFilter& source);
    void setOutputRotationLODs(ConstArrayView<LODRegion> lods,
                               ConstArrayView<std::uint16_t> outputRotationIndices,
                               std::uint32_t outputOffset,
                               std::uint16_t jointGroupIndex);

private:
    Configuration config;
    RigMetadata* meta;
    MemoryResource* memRes;
    JointStorage storage;
    dna::RotationUnit rotationUnit;
    std::uint16_t lodCount;
};

}  // namespace bpcm

}  // namespace rl4
