// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/bpcm/BPCMJointsBuilderFactory.h"

#include "riglogic/joints/cpu/bpcm/BPCMJointsBuilder.h"

namespace rl4 {

UniqueInstance<JointsBuilder>::PointerType BPCMJointsBuilderFactory::create(const Configuration& config,
                                                                            RigMetadata* meta,
                                                                            MemoryResource* memRes) {
    return UniqueInstance<bpcm::BPCMJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
}

}  // namespace rl4
