// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/riglogic/Configuration.h"

namespace rl4 {

struct BPCMJointsBuilderFactory {
    static UniqueInstance<JointsBuilder>::PointerType create(const Configuration& config,
                                                             RigMetadata* meta,
                                                             MemoryResource* memRes);
};

}  // namespace rl4
