// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"

#include "riglogic/system/simd/Utils.h"

namespace rl4 {

namespace bpcm {

Evaluator::Evaluator(JointStorage&& storage_,
                     Vector<JointGroupView>&& jointGroups_,
                     CalculationStrategyPointer strategy_,
                     JointsOutputInstance::Factory instanceFactory_,
                     MemoryResource* memRes_) :
    memRes{memRes_},
    storage{std::move(storage_)},
    jointGroups{std::move(jointGroups_)},
    strategy{std::move(strategy_)},
    instanceFactory{instanceFactory_} {
}

JointsOutputInstance::Pointer Evaluator::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(instanceMemRes);
}

std::uint32_t Evaluator::getJointDeltaValueCountForLOD(std::uint16_t lod) const {
    std::uint32_t deltaCount = {};
    for (const auto& group : jointGroups) {
        deltaCount += (group.lods[lod].inputLODs.size * group.lods[lod].outputLODs.size);
    }
    return deltaCount;
}

void Evaluator::calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const {
    for (std::size_t i = {}; i < storage.jointGroups.size(); ++i) {
        calculate(inputs, outputs, lod, static_cast<std::uint16_t>(i));
    }
}

void Evaluator::calculate(ControlsInputInstance* inputs,
                          JointsOutputInstance* outputs,
                          std::uint16_t lod,
                          std::uint16_t jointGroupIndex) const {
    assert(strategy != nullptr);
    const auto& jointGroup = jointGroups[jointGroupIndex];
    if (jointGroup.rowCount != 0u) {
        strategy->calculate(jointGroup, inputs->getInputBuffer(), outputs->getOutputBuffer(), lod);
    }
}

void Evaluator::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(storage);
    const Configuration* config = static_cast<Configuration*>(archive.getUserData());
    RuntimeTemplateInstantiator instantiator{config};
    jointGroups = instantiator.invoke<StorageSnapshot, Vector<JointGroupView>>(storage, memRes);
}

void Evaluator::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(storage);
}

}  // namespace bpcm

}  // namespace rl4
