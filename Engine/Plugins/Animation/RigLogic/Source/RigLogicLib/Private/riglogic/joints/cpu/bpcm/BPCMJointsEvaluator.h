// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/joints/cpu/bpcm/BPCMCalculationStrategy.h"
#include "riglogic/joints/cpu/bpcm/Storage.h"
#include "riglogic/riglogic/RigInstanceImpl.h"

#include <cstdint>

namespace rl4 {

namespace bpcm {

class Evaluator : public JointsEvaluator {
public:
    using CalculationStrategyPointer = typename UniqueInstance<JointGroupLinearCalculationStrategy>::PointerType;

    struct Accessor;
    friend Accessor;

public:
    Evaluator(JointStorage&& storage_,
              Vector<JointGroupView>&& jointGroups_,
              CalculationStrategyPointer strategy_,
              JointsOutputInstance::Factory instanceFactory_,
              MemoryResource* memRes_);

    JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs,
                   JointsOutputInstance* outputs,
                   std::uint16_t lod,
                   std::uint16_t jointGroupIndex) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    MemoryResource* memRes;
    JointStorage storage;
    Vector<JointGroupView> jointGroups;
    CalculationStrategyPointer strategy;
    JointsOutputInstance::Factory instanceFactory;
};

}  // namespace bpcm

}  // namespace rl4
