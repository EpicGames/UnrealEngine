// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/cpu/utils/LODRegion.h"
#include "riglogic/types/PaddedBlockView.h"

#include <cstdint>

namespace rl4 {

namespace bpcm {

struct JointGroup {
    // Start of non-zero values in storage
    std::uint32_t valuesOffset;
    // Start of sub-matrix col -> input vector mapping in storage
    std::uint32_t inputIndicesOffset;
    // Start of sub-matrix row -> output vector mapping in storage
    std::uint32_t outputIndicesOffset;
    // Start of LODs in storage
    std::uint32_t lodsOffset;
    // Start of output rotation indices in storage
    std::uint32_t outputRotationIndicesOffset;
    // Start of output rotation index LODs in storage
    std::uint32_t outputRotationLODsOffset;
    // Sizes associated with start offsets
    std::uint32_t valuesSize;
    std::uint32_t colCount;
    std::uint32_t rowCount;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(valuesOffset,
                inputIndicesOffset,
                outputIndicesOffset,
                lodsOffset,
                outputRotationIndicesOffset,
                outputRotationLODsOffset,
                valuesSize,
                colCount,
                rowCount);
    }
};

}  // namespace bpcm

}  // namespace rl4
