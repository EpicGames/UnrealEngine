// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/cpu/bpcm/Storage.h"
#include "riglogic/system/simd/SIMD.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <tdm/Quat.h>

namespace rl4 {

namespace bpcm {

struct NoopAdapter {
    tdm::rot_sign rotSigns;

    FORCE_INLINE void forward(const JointGroupView& /*unused*/, ArrayView<float> /*unused*/, std::uint16_t /*unused*/) const {
    }

    FORCE_INLINE void reverse(const JointGroupView& /*unused*/, ArrayView<float> /*unused*/, std::uint16_t /*unused*/) const {
    }
};

template<typename TAngle, tdm::rot_seq Order>
struct EulerAnglesToQuaternions {
    tdm::rot_sign rotSigns;

    static_assert(std::is_same<TAngle, tdm::fdeg>::value || std::is_same<TAngle, tdm::frad>::value,
                  "TAngle must be either tdm::fdeg or tdm::frad.");

    FORCE_INLINE void forward(const JointGroupView& jointGroup, ArrayView<float> outputs, std::uint16_t lod) const {
        for (std::size_t row = {}; row < jointGroup.outputRotationLODs[lod]; ++row) {
            const auto rotationStartIndex = jointGroup.outputRotationIndices[row];
            tdm::frad3 euler{tdm::frad{TAngle{outputs[rotationStartIndex + 0ul]}},
                             tdm::frad{TAngle{outputs[rotationStartIndex + 1ul]}},
                             tdm::frad{TAngle{outputs[rotationStartIndex + 2ul]}}};
            const tdm::fquat q = tdm::fquat::from_euler<Order>(euler, rotSigns);
            outputs[rotationStartIndex + 0ul] = q.x;
            outputs[rotationStartIndex + 1ul] = q.y;
            outputs[rotationStartIndex + 2ul] = q.z;
            outputs[rotationStartIndex + 3ul] = q.w;
        }
    }

    FORCE_INLINE void reverse(const JointGroupView& jointGroup, ArrayView<float> outputs, std::uint16_t lod) const {
        for (std::size_t row = {}; row < jointGroup.outputRotationLODs[lod]; ++row) {
            const auto rotationStartIndex = jointGroup.outputRotationIndices[row];
            const tdm::fquat q{outputs[rotationStartIndex + 0ul],
                               outputs[rotationStartIndex + 1ul],
                               outputs[rotationStartIndex + 2ul],
                               outputs[rotationStartIndex + 3ul]};
            const tdm::frad3 e = q.euler<Order>(rotSigns);
            outputs[rotationStartIndex + 0ul] = TAngle{e[0]}.value;
            outputs[rotationStartIndex + 1ul] = TAngle{e[1]}.value;
            outputs[rotationStartIndex + 2ul] = TAngle{e[2]}.value;
        }
    }
};

}  // namespace bpcm

}  // namespace rl4
