// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/bpcm/JointGroup.h"
#include "riglogic/types/bpcm/FloatArray.h"

#include <cstdint>

namespace rl4 {

namespace bpcm {

struct JointStorage {
    // All non-zero values
    FloatArray values;
    // Sub-matrix col -> input vector
    AlignedVector<std::uint16_t> inputIndices;
    // Sub-matrix row -> output vector
    AlignedVector<std::uint16_t> outputIndices;
    // Output index boundaries for each LOD
    Vector<LODRegion> lodRegions;
    // Rotation indices (the start index for each rotation, used for conversion to quaternions)
    Vector<std::uint16_t> outputRotationIndices;
    // Rotation index boundaries for each LOD
    Vector<std::uint16_t> outputRotationLODs;
    // Delineate storage into joint-groups
    Vector<JointGroup> jointGroups;

    explicit JointStorage(MemoryResource* memRes) :
        values{memRes},
        inputIndices{memRes},
        outputIndices{memRes},
        lodRegions{memRes},
        outputRotationIndices{memRes},
        outputRotationLODs{memRes},
        jointGroups{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(values, inputIndices, outputIndices, lodRegions, outputRotationIndices, outputRotationLODs, jointGroups);
    }
};

struct JointGroupView {
    void* values;
    std::uint32_t colCount;
    std::uint32_t rowCount;
    std::uint16_t* inputIndices;
    std::uint16_t* outputIndices;
    std::uint16_t* outputRotationIndices;
    std::uint16_t* outputRotationLODs;
    LODRegion* lods;
};

template<typename T, typename TF256, typename TF128>
struct StorageSnapshot {

    Vector<JointGroupView> operator()(JointStorage& storage, MemoryResource* memRes) {
        Vector<JointGroupView> snapshot{storage.jointGroups.size(), {}, memRes};
        for (std::size_t i = 0ul; i < storage.jointGroups.size(); ++i) {
            const auto& jointGroup = storage.jointGroups[i];
            snapshot[i].values = storage.values.data<T>() + jointGroup.valuesOffset;
            snapshot[i].colCount = jointGroup.colCount;
            snapshot[i].rowCount = jointGroup.rowCount;
            snapshot[i].inputIndices = storage.inputIndices.data() + jointGroup.inputIndicesOffset;
            snapshot[i].outputIndices = storage.outputIndices.data() + jointGroup.outputIndicesOffset;
            snapshot[i].outputRotationIndices = storage.outputRotationIndices.data() + jointGroup.outputRotationIndicesOffset;
            snapshot[i].outputRotationLODs = storage.outputRotationLODs.data() + jointGroup.outputRotationLODsOffset;
            snapshot[i].lods = storage.lodRegions.data() + jointGroup.lodsOffset;
        }
        return snapshot;
    }
};

}  // namespace bpcm

}  // namespace rl4
