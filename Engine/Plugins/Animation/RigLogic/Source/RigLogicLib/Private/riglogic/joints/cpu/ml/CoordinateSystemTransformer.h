// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/utils/Macros.h"

namespace rl4 {

namespace ml {

struct NoopTransformer {

    static FORCE_INLINE void transform(ArrayView<float> values,
                                       ConstArrayView<std::uint16_t> baseIndices,
                                       const tdm::fmat3& changeOfBasis,
                                       tdm::rot_seq srcSeq,
                                       const tdm::rot_sign& srcSigns,
                                       tdm::rot_seq dstSeq,
                                       const tdm::rot_sign& dstSigns) {
        RL_UNUSED(values);
        RL_UNUSED(baseIndices);
        RL_UNUSED(changeOfBasis);
        RL_UNUSED(srcSeq);
        RL_UNUSED(srcSigns);
        RL_UNUSED(dstSeq);
        RL_UNUSED(dstSigns);
    }
};

struct TranslationTransformer {

    static FORCE_INLINE void transform(ArrayView<float> values,
                                       ConstArrayView<std::uint16_t> baseIndices,
                                       const tdm::fmat3& changeOfBasis,
                                       tdm::rot_seq srcSeq,
                                       const tdm::rot_sign& srcSigns,
                                       tdm::rot_seq dstSeq,
                                       const tdm::rot_sign& dstSigns) {
        RL_UNUSED(srcSeq);
        RL_UNUSED(srcSigns);
        RL_UNUSED(dstSeq);
        RL_UNUSED(dstSigns);
        for (std::size_t i = {}; i < baseIndices.size(); ++i) {
            const auto baseIndex = baseIndices[i];
            const tdm::fvec3 src{values[baseIndex + 0ul], values[baseIndex + 1ul], values[baseIndex + 2ul]};
            const tdm::fvec3 dst = tdm::convert_position(src, changeOfBasis);
            values[baseIndex + 0ul] = dst[0];
            values[baseIndex + 1ul] = dst[1];
            values[baseIndex + 2ul] = dst[2];
        }
    }
};

struct QuaternionTransformer {

    static FORCE_INLINE void transform(ArrayView<float> values,
                                       ConstArrayView<std::uint16_t> baseIndices,
                                       const tdm::fmat3& changeOfBasis,
                                       tdm::rot_seq srcSeq,
                                       const tdm::rot_sign& srcSigns,
                                       tdm::rot_seq dstSeq,
                                       const tdm::rot_sign& dstSigns) {
        for (std::size_t i = {}; i < baseIndices.size(); ++i) {
            const auto baseIndex = baseIndices[i];
            const tdm::fquat q{values[baseIndex + 0ul],
                               values[baseIndex + 1ul],
                               values[baseIndex + 2ul],
                               values[baseIndex + 3ul]};
            const tdm::fquat nq = tdm::normalize(q);
            const tdm::frad3 srcEuler = nq.euler(srcSeq, srcSigns);
            const tdm::frad3 dstEuler = tdm::convert_rotation(srcEuler, changeOfBasis, srcSeq, srcSigns, dstSeq, dstSigns);
            const tdm::fquat dq = tdm::fquat(dstEuler, dstSeq, dstSigns);
            values[baseIndex + 0ul] = dq.x;
            values[baseIndex + 1ul] = dq.y;
            values[baseIndex + 2ul] = dq.z;
            values[baseIndex + 3ul] = dq.w;
        }
    }
};

template<typename TAngle>
struct EulerAnglesTransformer {

    static_assert(std::is_same<TAngle, tdm::fdeg>::value || std::is_same<TAngle, tdm::frad>::value,
                  "TAngle must be either tdm::fdeg or tdm::frad.");

    static FORCE_INLINE void transform(ArrayView<float> values,
                                       ConstArrayView<std::uint16_t> baseIndices,
                                       const tdm::fmat3& changeOfBasis,
                                       tdm::rot_seq srcSeq,
                                       const tdm::rot_sign& srcSigns,
                                       tdm::rot_seq dstSeq,
                                       const tdm::rot_sign& dstSigns) {
        for (std::size_t i = {}; i < baseIndices.size(); ++i) {
            const auto baseIndex = baseIndices[i];
            const tdm::frad3 srcEuler{tdm::frad{TAngle{values[baseIndex + 0ul]}},
                                      tdm::frad{TAngle{values[baseIndex + 1ul]}},
                                      tdm::frad{TAngle{values[baseIndex + 2ul]}}};
            const tdm::frad3 dstEuler = tdm::convert_rotation(srcEuler, changeOfBasis, srcSeq, srcSigns, dstSeq, dstSigns);
            values[baseIndex + 0ul] = TAngle{dstEuler[0]}.value;
            values[baseIndex + 1ul] = TAngle{dstEuler[1]}.value;
            values[baseIndex + 2ul] = TAngle{dstEuler[2]}.value;
        }
    }
};

struct ScaleTransformer {

    static FORCE_INLINE void transform(ArrayView<float> values,
                                       ConstArrayView<std::uint16_t> baseIndices,
                                       const tdm::fmat3& changeOfBasis,
                                       tdm::rot_seq srcSeq,
                                       const tdm::rot_sign& srcSigns,
                                       tdm::rot_seq dstSeq,
                                       const tdm::rot_sign& dstSigns) {
        RL_UNUSED(srcSeq);
        RL_UNUSED(srcSigns);
        RL_UNUSED(dstSeq);
        RL_UNUSED(dstSigns);
        for (std::size_t i = {}; i < baseIndices.size(); ++i) {
            const auto baseIndex = baseIndices[i];
            const tdm::fvec3 src{values[baseIndex + 0ul], values[baseIndex + 1ul], values[baseIndex + 2ul]};
            const tdm::fvec3 dst = tdm::convert_scale(src, changeOfBasis);
            values[baseIndex + 0ul] = dst[0];
            values[baseIndex + 1ul] = dst[1];
            values[baseIndex + 2ul] = dst[2];
        }
    }
};

}  // namespace ml

}  // namespace rl4
