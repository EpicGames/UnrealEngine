// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/ml/MLJointsBuilder.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsNullEvaluator.h"
#include "riglogic/joints/cpu/ml/CoordinateSystemTransformer.h"
#include "riglogic/joints/cpu/ml/MLJointsEvaluator.h"
#include "riglogic/joints/cpu/ml/RotationAdapters.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Detect.h"
#include "riglogic/utils/Extd.h"

namespace rl4 {

namespace ml {

MLJointsBuilder::MLJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_) :
    memRes{memRes_},
    config{config_},
    meta{meta_},
    inputIndices{memRes},
    outputIndices{memRes},
    inputRotationBaseIndices{memRes},
    outputRotationBaseIndices{memRes},
    uniqueTranslationBaseIndices{memRes},
    uniqueRotationBaseIndices{memRes},
    uniqueScaleBaseIndices{memRes},
    changeOfBasis{tdm::fmat3::identity()},
    srcSeq{meta->rotationSequence},
    srcSigns{meta->rotationSigns},
    dstSeq{meta->rotationSequence},
    dstSigns{meta->rotationSigns},
    inputJointAttrCount{},
    outputJointAttrCount{},
    rotationUnit{},
    mlTranslationType{config.translationType},
    mlRotationType{config.rotationType},
    mlScaleType{config.scaleType} {

    outputJointAttrCount =
        static_cast<std::uint16_t>(static_cast<std::uint8_t>(config.translationType) +
                                   static_cast<std::uint8_t>(config.rotationType) + static_cast<std::uint8_t>(config.scaleType));
}

void MLJointsBuilder::computeStorageRequirements() {
}

void MLJointsBuilder::computeStorageRequirements(const JointBehaviorFilter& source) {
    RL_UNUSED(source);
}

void MLJointsBuilder::allocateStorage(const JointBehaviorFilter& source) {
    RL_UNUSED(source);
}

void MLJointsBuilder::remapIndices(std::uint16_t lod) {
    auto remap = [this](ArrayView<std::uint16_t> indices) {
        for (auto& absAttrIndex : indices) {
            const auto jointIndex = static_cast<std::uint16_t>(absAttrIndex / inputJointAttrCount);
            const auto relAttrIndex = static_cast<std::uint16_t>(absAttrIndex % inputJointAttrCount);
            const auto newAttrBase = static_cast<std::uint16_t>(jointIndex * outputJointAttrCount);
            const auto delta = static_cast<std::int32_t>(outputJointAttrCount) - static_cast<std::int32_t>(inputJointAttrCount);
            // Only the relative attribute indices for scale are offset by one when output is in quaternions
            const auto newRelAttrIndex = (relAttrIndex < 6u ? relAttrIndex : static_cast<std::uint16_t>(delta + relAttrIndex));
            absAttrIndex = static_cast<std::uint16_t>(newAttrBase + newRelAttrIndex);
        }
    };
    remap(outputIndices[lod]);
    remap(outputRotationBaseIndices[lod]);

    // Map all qx, qy, qz, qw indices to qx, qx, qx, qx
    for (auto& rotationIndex : outputRotationBaseIndices[lod]) {
        rotationIndex = static_cast<std::uint16_t>((rotationIndex / outputJointAttrCount) * outputJointAttrCount +
                                                   static_cast<std::uint16_t>(mlTranslationType));
    }

    auto& inputRotationIndices = inputRotationBaseIndices[lod];
    auto& outputRotationIndices = outputRotationBaseIndices[lod];

    UnorderedSet<std::uint16_t> deduplicator{memRes};
    deduplicator.reserve(outputRotationIndices.size());
    for (std::size_t i = {}; i < outputRotationIndices.size();) {
        if (!deduplicator.insert(outputRotationIndices[i]).second) {
            outputRotationIndices.erase(extd::advanced(outputRotationIndices.begin(), i));
            inputRotationIndices.erase(extd::advanced(inputRotationIndices.begin(), i));
        } else {
            ++i;
        }
    }
}

void MLJointsBuilder::fillStorage(const JointBehaviorFilter& source) {
    const auto reader = source.getReader();
    const auto lodCount = reader->getLODCount();
    rotationUnit = reader->getRotationUnit();

    if (reader->getMLTypeCount() == 0) {
        return;
    }

    auto findMLRotationType = [this, reader]() {
        const auto paramKeys = reader->getMLJointsParameterKeys();
        const auto paramValues = reader->getMLJointsParameterValues();
        for (std::size_t i = {}; i < paramKeys.size(); ++i) {
            if (paramKeys[i] == static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationType)) {
                const auto rotationType = static_cast<dna::RotationRepresentation>(paramValues[i]);
                if (rotationType == dna::RotationRepresentation::EulerAngles) {
                    return RotationType::EulerAngles;
                } else if (rotationType == dna::RotationRepresentation::Quaternion) {
                    return RotationType::Quaternions;
                }
            }
        }
        return config.rotationType;
    };

    auto getCoordinateSystem = [reader]() {
        const auto paramKeys = reader->getMLJointsParameterKeys();
        const auto paramValues = reader->getMLJointsParameterValues();
        tdm::coord_sys coordSys = {};
        for (std::size_t i = {}; i < paramKeys.size(); ++i) {
            const auto xAxis = static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointCoordinateSystemAxisX);
            const auto yAxis = static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointCoordinateSystemAxisY);
            const auto zAxis = static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointCoordinateSystemAxisZ);
            if (paramKeys[i] == xAxis) {
                coordSys.x = static_cast<tdm::axis_dir>(paramValues[i]);
            } else if (paramKeys[i] == yAxis) {
                coordSys.y = static_cast<tdm::axis_dir>(paramValues[i]);
            } else if (paramKeys[i] == zAxis) {
                coordSys.z = static_cast<tdm::axis_dir>(paramValues[i]);
            }
        }
        return coordSys;
    };

    auto getRotationSigns = [reader]() {
        const auto paramKeys = reader->getMLJointsParameterKeys();
        const auto paramValues = reader->getMLJointsParameterValues();
        tdm::rot_sign rotSigns = {};
        for (std::size_t i = {}; i < paramKeys.size(); ++i) {
            const auto xAxis = static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSignAxisX);
            const auto yAxis = static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSignAxisY);
            const auto zAxis = static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSignAxisZ);
            if (paramKeys[i] == xAxis) {
                rotSigns.x = (paramValues[i] == 1 ? tdm::rot_dir::positive : tdm::rot_dir::negative);
            } else if (paramKeys[i] == yAxis) {
                rotSigns.y = (paramValues[i] == 1 ? tdm::rot_dir::positive : tdm::rot_dir::negative);
            } else if (paramKeys[i] == zAxis) {
                rotSigns.z = (paramValues[i] == 1 ? tdm::rot_dir::positive : tdm::rot_dir::negative);
            }
        }
        return rotSigns;
    };

    auto getRotationSequence = [reader]() {
        const auto paramKeys = reader->getMLJointsParameterKeys();
        const auto paramValues = reader->getMLJointsParameterValues();
        for (std::size_t i = {}; i < paramKeys.size(); ++i) {
            if (paramKeys[i] == static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSequence)) {
                return static_cast<tdm::rot_seq>(paramValues[i]);
            }
        }
        return tdm::rot_seq{};
    };

    const bool isCoordinateSystemSpecified = [reader]() {
        const auto paramKeys = reader->getMLJointsParameterKeys();
        bool isCoordSysSpecified = true;
        const std::uint16_t expectedKeys[] = {
            static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointCoordinateSystemAxisX),
            static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointCoordinateSystemAxisY),
            static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointCoordinateSystemAxisZ),
            static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSignAxisX),
            static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSignAxisY),
            static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSignAxisZ),
            static_cast<std::uint16_t>(dna::MachineLearnedBehaviorParameterKey::JointRotationSequence)};
        for (std::size_t i = {}; i < sizeof(expectedKeys) / sizeof(*expectedKeys); ++i) {
            isCoordSysSpecified = isCoordSysSpecified && extd::contains(paramKeys, expectedKeys[i]);
        }
        return isCoordSysSpecified;
    }();

    if (isCoordinateSystemSpecified) {
        changeOfBasis = tdm::change_of_basis<float>(getCoordinateSystem(), meta->coordinateSystem);
        srcSeq = getRotationSequence();
        srcSigns = getRotationSigns();
        dstSeq = meta->rotationSequence;
        dstSigns = meta->rotationSigns;
    }

    mlRotationType = findMLRotationType();
    inputJointAttrCount =
        static_cast<std::uint16_t>(static_cast<std::uint8_t>(mlTranslationType) + static_cast<std::uint8_t>(mlRotationType) +
                                   static_cast<std::uint8_t>(mlScaleType));

    inputIndices.resize(lodCount);
    outputIndices.resize(lodCount);
    inputRotationBaseIndices.resize(lodCount);
    outputRotationBaseIndices.resize(lodCount);
    uniqueTranslationBaseIndices.resize(lodCount);
    uniqueRotationBaseIndices.resize(lodCount);
    uniqueScaleBaseIndices.resize(lodCount);

    auto isBaseTranslationAttribute = [this](std::uint16_t outputIndex) {
        const auto relTranslationStartIndex = static_cast<std::uint16_t>(0);
        const auto relAttrIndex = outputIndex % inputJointAttrCount;
        return (relAttrIndex == relTranslationStartIndex);
    };

    auto isRotationAttribute = [this](std::uint16_t outputIndex) {
        const auto relRotationStartIndex = static_cast<std::uint16_t>(mlTranslationType);
        const auto relRotationEndIndex =
            static_cast<std::uint16_t>(relRotationStartIndex + static_cast<std::uint16_t>(mlRotationType));
        const auto relAttrIndex = outputIndex % inputJointAttrCount;
        return (relAttrIndex >= relRotationStartIndex) && (relAttrIndex < relRotationEndIndex);
    };

    auto isBaseScaleAttribute = [this](std::uint16_t outputIndex) {
        const auto relScaleStartIndex = static_cast<std::uint16_t>(static_cast<std::uint16_t>(mlTranslationType) +
                                                                   static_cast<std::uint16_t>(mlRotationType));
        const auto relAttrIndex = outputIndex % inputJointAttrCount;
        return (relAttrIndex == relScaleStartIndex);
    };

    auto isJointInLOD = [](std::uint16_t outputIndex, std::uint16_t attrCount, ConstArrayView<std::uint16_t> jointIndices) {
        const auto jointIndex = outputIndex / attrCount;
        return std::find(jointIndices.begin(), jointIndices.end(), jointIndex) != jointIndices.end();
    };

    auto deduplicate = [](Vector<std::uint16_t>& indices) {
        std::sort(indices.begin(), indices.end());
        indices.erase(std::unique(indices.begin(), indices.end()), indices.end());
    };

    for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
        const auto jointIndices = reader->getJointIndicesForLOD(lod);
        const auto mlJointsInputIndices = reader->getMLJointsInputIndices();
        const auto mlJointsOutputIndices = reader->getMLJointsOutputIndices();
        assert(mlJointsInputIndices.size() == mlJointsOutputIndices.size());

        inputIndices[lod].reserve(mlJointsInputIndices.size());
        outputIndices[lod].reserve(mlJointsOutputIndices.size());
        inputRotationBaseIndices[lod].reserve(mlJointsInputIndices.size());
        outputRotationBaseIndices[lod].reserve(mlJointsOutputIndices.size());
        uniqueTranslationBaseIndices[lod].reserve(mlJointsInputIndices.size());
        uniqueRotationBaseIndices[lod].reserve(mlJointsInputIndices.size());
        uniqueScaleBaseIndices[lod].reserve(mlJointsInputIndices.size());

        for (std::size_t mi = {}; mi < mlJointsOutputIndices.size(); ++mi) {
            const auto outputControlIndex = mlJointsInputIndices[mi];
            const auto jointAttrIndex = mlJointsOutputIndices[mi];
            if (isJointInLOD(jointAttrIndex, inputJointAttrCount, jointIndices)) {
                if (isRotationAttribute(jointAttrIndex)) {
                    inputRotationBaseIndices[lod].push_back(outputControlIndex);
                    outputRotationBaseIndices[lod].push_back(jointAttrIndex);
                } else {
                    inputIndices[lod].push_back(outputControlIndex);
                    outputIndices[lod].push_back(jointAttrIndex);
                    if (isCoordinateSystemSpecified) {
                        if (isBaseTranslationAttribute(jointAttrIndex)) {
                            uniqueTranslationBaseIndices[lod].push_back(outputControlIndex);
                        } else if (isBaseScaleAttribute(jointAttrIndex)) {
                            uniqueScaleBaseIndices[lod].push_back(outputControlIndex);
                        }
                    }
                }
            }
        }

        remapIndices(lod);
        if (isCoordinateSystemSpecified) {
            uniqueRotationBaseIndices[lod] = inputRotationBaseIndices[lod];
            deduplicate(uniqueRotationBaseIndices[lod]);
            deduplicate(uniqueTranslationBaseIndices[lod]);
            deduplicate(uniqueScaleBaseIndices[lod]);
        }
    }
}

void MLJointsBuilder::registerControls(Controls* controls) {
    for (std::uint16_t lod = {}; lod < static_cast<std::uint16_t>(inputIndices.size()); ++lod) {
        controls->registerControls(lod, inputIndices[lod]);
        controls->registerControls(lod, inputRotationBaseIndices[lod]);
    }
}

struct MLJointsEvaluatorFactory {

    JointsEvaluator::Pointer operator()(RotationType targetRotationType,
                                        RotationType mlRotationType,
                                        tdm::rot_seq rotationSequence,
                                        dna::RotationUnit rotationUnit,
                                        Matrix<std::uint16_t>&& inputIndices,
                                        Matrix<std::uint16_t>&& outputIndices,
                                        Matrix<std::uint16_t>&& inputRotationBaseIndices,
                                        Matrix<std::uint16_t>&& outputRotationBaseIndices,
                                        Matrix<std::uint16_t>&& uniqueTranslationBaseIndices,
                                        Matrix<std::uint16_t>&& uniqueRotationBaseIndices,
                                        Matrix<std::uint16_t>&& uniqueScaleBaseIndices,
                                        tdm::fmat3 changeOfBasis,
                                        tdm::rot_seq srcSeq,
                                        tdm::rot_sign srcSigns,
                                        tdm::rot_seq dstSeq,
                                        tdm::rot_sign dstSigns,
                                        MemoryResource* memRes) {

        const bool isCoordSysTransformed = !(changeOfBasis == tdm::fmat3::identity() && srcSeq == dstSeq && srcSigns == dstSigns);

        if (targetRotationType == mlRotationType) {
            if (isCoordSysTransformed) {
                if (targetRotationType == RotationType::EulerAngles) {
                    if (rotationUnit == dna::RotationUnit::degrees) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer,
                                                       EulerAnglesTransformer<tdm::fdeg>,
                                                       ScaleTransformer,
                                                       NoopAdapter>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              NoopAdapter{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<TranslationTransformer,
                                                       EulerAnglesTransformer<tdm::frad>,
                                                       ScaleTransformer,
                                                       NoopAdapter>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              NoopAdapter{dstSigns},
                                              nullptr);
                    }
                } else {
                    using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, NoopAdapter>;
                    auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                    return factory.create(std::move(inputIndices),
                                          std::move(outputIndices),
                                          std::move(inputRotationBaseIndices),
                                          std::move(outputRotationBaseIndices),
                                          std::move(uniqueTranslationBaseIndices),
                                          std::move(uniqueRotationBaseIndices),
                                          std::move(uniqueScaleBaseIndices),
                                          changeOfBasis,
                                          srcSeq,
                                          srcSigns,
                                          dstSeq,
                                          dstSigns,
                                          NoopAdapter{dstSigns},
                                          nullptr);
                }
            } else {
                using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, NoopAdapter>;
                auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                return factory.create(std::move(inputIndices),
                                      std::move(outputIndices),
                                      std::move(inputRotationBaseIndices),
                                      std::move(outputRotationBaseIndices),
                                      std::move(uniqueTranslationBaseIndices),
                                      std::move(uniqueRotationBaseIndices),
                                      std::move(uniqueScaleBaseIndices),
                                      changeOfBasis,
                                      srcSeq,
                                      srcSigns,
                                      dstSeq,
                                      dstSigns,
                                      NoopAdapter{dstSigns},
                                      nullptr);
            }
        }

#ifdef RL_BUILD_WITH_XYZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xyz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::xyz>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::xyz>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::fdeg>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            } else {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::xyz>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::xyz>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::frad>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            }
        }
#endif  // RL_BUILD_WITH_XYZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_XZY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xzy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::xzy>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::xzy>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::fdeg>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            } else {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::xzy>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::xzy>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::frad>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            }
        }
#endif  // RL_BUILD_WITH_XZY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YXZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yxz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::yxz>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::yxz>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::fdeg>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            } else {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::yxz>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::yxz>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::frad>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            }
        }
#endif  // RL_BUILD_WITH_YXZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YZX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yzx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::yzx>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::yzx>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::fdeg>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            } else {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::yzx>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::yzx>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::frad>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            }
        }
#endif  // RL_BUILD_WITH_YZX_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZXY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zxy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::zxy>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zxy>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::fdeg>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            } else {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zxy>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::zxy>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::frad>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            }
        }
#endif  // RL_BUILD_WITH_ZXY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZYX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zyx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::zyx>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::fdeg>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            } else {
                if (targetRotationType == RotationType::EulerAngles) {
                    using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>;
                    if (isCoordSysTransformed) {
                        using MLJE = MLJointsEvaluator<TranslationTransformer, QuaternionTransformer, ScaleTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, Q2E>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              Q2E{dstSigns},
                                              nullptr);
                    }
                } else {
                    using E2Q = EulerAnglesToQuaternions<tdm::frad, tdm::rot_seq::zyx>;
                    if (isCoordSysTransformed) {
                        using MLJE =
                            MLJointsEvaluator<TranslationTransformer, EulerAnglesTransformer<tdm::frad>, ScaleTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    } else {
                        using MLJE = MLJointsEvaluator<NoopTransformer, NoopTransformer, NoopTransformer, E2Q>;
                        auto factory = UniqueInstance<MLJE, JointsEvaluator>::with(memRes);
                        return factory.create(std::move(inputIndices),
                                              std::move(outputIndices),
                                              std::move(inputRotationBaseIndices),
                                              std::move(outputRotationBaseIndices),
                                              std::move(uniqueTranslationBaseIndices),
                                              std::move(uniqueRotationBaseIndices),
                                              std::move(uniqueScaleBaseIndices),
                                              changeOfBasis,
                                              srcSeq,
                                              srcSigns,
                                              dstSeq,
                                              dstSigns,
                                              E2Q{dstSigns},
                                              nullptr);
                    }
                }
            }
        }
#endif  // RL_BUILD_WITH_ZYX_ROTATION_ORDER

        return nullptr;
    }
};

JointsEvaluator::Pointer MLJointsBuilder::build() {
    const auto targetRotationType = config.rotationType;
    const EvaluatorType type =
        (meta->initializationMethod == InitializationMethod::Restore) ? meta->popFrontEvaluator() : EvaluatorType::Auto;

    const bool isMLDataEmpty = [this]() {
        assert(inputIndices.size() == outputIndices.size());
        bool empty = true;
        for (std::size_t lod = {}; lod < inputIndices.size(); ++lod) {
            empty = empty && (inputIndices[lod].empty() && outputIndices[lod].empty());
        }
        return empty;
    }();

    if ((type == EvaluatorType::Null) || ((type == EvaluatorType::Auto) && isMLDataEmpty)) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    auto evaluator = MLJointsEvaluatorFactory()(targetRotationType,
                                                mlRotationType,
                                                meta->rotationSequence,
                                                rotationUnit,
                                                std::move(inputIndices),
                                                std::move(outputIndices),
                                                std::move(inputRotationBaseIndices),
                                                std::move(outputRotationBaseIndices),
                                                std::move(uniqueTranslationBaseIndices),
                                                std::move(uniqueRotationBaseIndices),
                                                std::move(uniqueScaleBaseIndices),
                                                changeOfBasis,
                                                srcSeq,
                                                srcSigns,
                                                dstSeq,
                                                dstSigns,
                                                memRes);

    if (evaluator == nullptr) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    return evaluator;
}

}  // namespace ml

}  // namespace rl4
