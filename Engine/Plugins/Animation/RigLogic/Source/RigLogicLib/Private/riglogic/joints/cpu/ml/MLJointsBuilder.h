// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/riglogic/Configuration.h"

#include <cstdint>

namespace rl4 {

namespace ml {

class MLJointsBuilder : public JointsBuilder {
public:
    MLJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_);
    void computeStorageRequirements() override;
    void computeStorageRequirements(const JointBehaviorFilter& source) override;
    void allocateStorage(const JointBehaviorFilter& source) override;
    void fillStorage(const JointBehaviorFilter& source) override;
    void registerControls(Controls* controls) override;
    JointsEvaluator::Pointer build() override;

private:
    void remapIndices(std::uint16_t lod);

private:
    MemoryResource* memRes;
    Configuration config;
    RigMetadata* meta;
    Matrix<std::uint16_t> inputIndices;
    Matrix<std::uint16_t> outputIndices;
    Matrix<std::uint16_t> inputRotationBaseIndices;
    Matrix<std::uint16_t> outputRotationBaseIndices;
    Matrix<std::uint16_t> uniqueTranslationBaseIndices;
    Matrix<std::uint16_t> uniqueRotationBaseIndices;
    Matrix<std::uint16_t> uniqueScaleBaseIndices;
    tdm::fmat3 changeOfBasis;
    tdm::rot_seq srcSeq;
    tdm::rot_sign srcSigns;
    tdm::rot_seq dstSeq;
    tdm::rot_sign dstSigns;
    std::uint16_t inputJointAttrCount;
    std::uint16_t outputJointAttrCount;
    dna::RotationUnit rotationUnit;
    TranslationType mlTranslationType;
    RotationType mlRotationType;
    ScaleType mlScaleType;
};

}  // namespace ml

}  // namespace rl4
