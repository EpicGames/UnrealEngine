// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/ml/MLJointsBuilderFactory.h"

#include "riglogic/joints/cpu/ml/MLJointsBuilder.h"
#include "riglogic/system/simd/Detect.h"

namespace rl4 {

UniqueInstance<JointsBuilder>::PointerType MLJointsBuilderFactory::create(const Configuration& config,
                                                                          RigMetadata* meta,
                                                                          MemoryResource* memRes) {
    return UniqueInstance<ml::MLJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
}

}  // namespace rl4
