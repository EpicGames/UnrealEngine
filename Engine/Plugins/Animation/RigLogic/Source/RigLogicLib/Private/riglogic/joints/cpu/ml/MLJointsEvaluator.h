// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"

#include <cstdint>

namespace rl4 {

namespace ml {

template<class TTranslationTransformer, class TRotationTransformer, class TScaleTransformer, class TRotationAdapter>
class MLJointsEvaluator : public JointsEvaluator {
public:
    struct Accessor;
    friend Accessor;

public:
    MLJointsEvaluator(Matrix<std::uint16_t>&& inputIndices_,
                      Matrix<std::uint16_t>&& outputIndices_,
                      Matrix<std::uint16_t>&& inputRotationBaseIndices_,
                      Matrix<std::uint16_t>&& outputRotationBaseIndices_,
                      Matrix<std::uint16_t>&& uniqueTranslationBaseIndices_,
                      Matrix<std::uint16_t>&& uniqueRotationBaseIndices_,
                      Matrix<std::uint16_t>&& uniqueScaleBaseIndices_,
                      const tdm::fmat3& changeOfBasis_,
                      tdm::rot_seq srcSeq_,
                      tdm::rot_sign srcSigns_,
                      tdm::rot_seq dstSeq_,
                      tdm::rot_sign dstSigns_,
                      TRotationAdapter&& rotationAdapter_,
                      JointsOutputInstance::Factory instanceFactory_) :
        inputIndices{std::move(inputIndices_)},
        outputIndices{std::move(outputIndices_)},
        inputRotationBaseIndices{std::move(inputRotationBaseIndices_)},
        outputRotationBaseIndices{std::move(outputRotationBaseIndices_)},
        uniqueTranslationBaseIndices{std::move(uniqueTranslationBaseIndices_)},
        uniqueRotationBaseIndices{std::move(uniqueRotationBaseIndices_)},
        uniqueScaleBaseIndices{std::move(uniqueScaleBaseIndices_)},
        changeOfBasis{std::move(changeOfBasis_)},
        srcSeq{std::move(srcSeq_)},
        srcSigns{std::move(srcSigns_)},
        dstSeq{std::move(dstSeq_)},
        dstSigns{std::move(dstSigns_)},
        rotationAdapter{std::move(rotationAdapter_)},
        instanceFactory{instanceFactory_} {
    }

    JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override {
        return instanceFactory(instanceMemRes);
    }

    std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t lod) const override {
        RL_UNUSED(lod);
        return {};
    }

    void calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const override {
        assert(lod < inputIndices.size());
        assert(lod < outputIndices.size());
        assert(lod < inputRotationBaseIndices.size());
        assert(lod < outputRotationBaseIndices.size());
        assert(lod < uniqueTranslationBaseIndices.size());
        assert(lod < uniqueRotationBaseIndices.size());
        assert(lod < uniqueScaleBaseIndices.size());
        const auto& inputIndicesForLOD = inputIndices[lod];
        const auto& outputIndicesForLOD = outputIndices[lod];
        const auto& inputRotationBaseIndicesForLOD = inputRotationBaseIndices[lod];
        const auto& outputRotationBaseIndicesForLOD = outputRotationBaseIndices[lod];
        const auto& uniqueTranslationBaseIndicesForLOD = uniqueTranslationBaseIndices[lod];
        const auto& uniqueRotationBaseIndicesForLOD = uniqueRotationBaseIndices[lod];
        const auto& uniqueScaleBaseIndicesForLOD = uniqueScaleBaseIndices[lod];
        auto inputBuffer = inputs->getInputBuffer();
        auto outputBuffer = outputs->getOutputBuffer();

        TTranslationTransformer::transform(inputBuffer,
                                           uniqueTranslationBaseIndicesForLOD,
                                           changeOfBasis,
                                           srcSeq,
                                           srcSigns,
                                           dstSeq,
                                           dstSigns);
        TRotationTransformer::transform(inputBuffer,
                                        uniqueRotationBaseIndicesForLOD,
                                        changeOfBasis,
                                        srcSeq,
                                        srcSigns,
                                        dstSeq,
                                        dstSigns);
        TScaleTransformer::transform(inputBuffer,
                                     uniqueScaleBaseIndicesForLOD,
                                     changeOfBasis,
                                     srcSeq,
                                     srcSigns,
                                     dstSeq,
                                     dstSigns);

        for (std::size_t i = {}; i < inputIndicesForLOD.size(); ++i) {
            outputBuffer[outputIndicesForLOD[i]] += inputBuffer[inputIndicesForLOD[i]];
        }

        rotationAdapter.adapt(inputBuffer, outputBuffer, inputRotationBaseIndicesForLOD, outputRotationBaseIndicesForLOD);
    }

    void calculate(ControlsInputInstance* inputs,
                   JointsOutputInstance* outputs,
                   std::uint16_t lod,
                   std::uint16_t jointGroupIndex) const override {
        // Assume all joints are in a single joint group in this case
        RL_UNUSED(jointGroupIndex);
        calculate(inputs, outputs, lod);
    }

    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override {
        archive(inputIndices,
                outputIndices,
                inputRotationBaseIndices,
                outputRotationBaseIndices,
                uniqueTranslationBaseIndices,
                uniqueRotationBaseIndices,
                uniqueScaleBaseIndices,
                changeOfBasis,
                srcSeq,
                srcSigns,
                dstSeq,
                dstSigns);
    }

    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override {
        archive(inputIndices,
                outputIndices,
                inputRotationBaseIndices,
                outputRotationBaseIndices,
                uniqueTranslationBaseIndices,
                uniqueRotationBaseIndices,
                uniqueScaleBaseIndices,
                changeOfBasis,
                srcSeq,
                srcSigns,
                dstSeq,
                dstSigns);
    }

private:
    Matrix<std::uint16_t> inputIndices;
    Matrix<std::uint16_t> outputIndices;
    Matrix<std::uint16_t> inputRotationBaseIndices;
    Matrix<std::uint16_t> outputRotationBaseIndices;
    Matrix<std::uint16_t> uniqueTranslationBaseIndices;
    Matrix<std::uint16_t> uniqueRotationBaseIndices;
    Matrix<std::uint16_t> uniqueScaleBaseIndices;
    tdm::fmat3 changeOfBasis;
    tdm::rot_seq srcSeq;
    tdm::rot_sign srcSigns;
    tdm::rot_seq dstSeq;
    tdm::rot_sign dstSigns;
    TRotationAdapter rotationAdapter;
    JointsOutputInstance::Factory instanceFactory;
};

}  // namespace ml

}  // namespace rl4
