// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <tdm/Quat.h>

namespace rl4 {

namespace ml {

struct NoopAdapter {
    tdm::rot_sign rotationSigns;

    FORCE_INLINE void adapt(ConstArrayView<float> inputs,
                            ArrayView<float> outputs,
                            ConstArrayView<std::uint16_t> inputRotationBaseIndices,
                            ConstArrayView<std::uint16_t> outputRotationBaseIndices) const {
        for (std::size_t i = {}; i < inputRotationBaseIndices.size(); ++i) {
            const auto inputBaseIndex = inputRotationBaseIndices[i];
            const auto outputBaseIndex = outputRotationBaseIndices[i];
            const tdm::fquat q{inputs[inputBaseIndex + 0ul],
                               inputs[inputBaseIndex + 1ul],
                               inputs[inputBaseIndex + 2ul],
                               inputs[inputBaseIndex + 3ul]};
            const tdm::fquat nq = tdm::normalize(q);
            const tdm::fquat oq{outputs[outputBaseIndex + 0ul],
                                outputs[outputBaseIndex + 1ul],
                                outputs[outputBaseIndex + 2ul],
                                outputs[outputBaseIndex + 3ul]};
            const tdm::fquat cq = nq * oq;
            outputs[outputBaseIndex + 0ul] = cq.x;
            outputs[outputBaseIndex + 1ul] = cq.y;
            outputs[outputBaseIndex + 2ul] = cq.z;
            outputs[outputBaseIndex + 3ul] = cq.w;
        }
    }
};

template<typename TAngle, tdm::rot_seq Order>
struct QuaternionsToEulerAngles {
    tdm::rot_sign rotationSigns;

    static_assert(std::is_same<TAngle, tdm::fdeg>::value || std::is_same<TAngle, tdm::frad>::value,
                  "TAngle must be either tdm::fdeg or tdm::frad.");

    FORCE_INLINE void adapt(ConstArrayView<float> inputs,
                            ArrayView<float> outputs,
                            ConstArrayView<std::uint16_t> inputRotationBaseIndices,
                            ConstArrayView<std::uint16_t> outputRotationBaseIndices) const {
        for (std::size_t i = {}; i < inputRotationBaseIndices.size(); ++i) {
            const auto inputBaseIndex = inputRotationBaseIndices[i];
            const auto outputBaseIndex = outputRotationBaseIndices[i];
            const tdm::fquat q{inputs[inputBaseIndex + 0ul],
                               inputs[inputBaseIndex + 1ul],
                               inputs[inputBaseIndex + 2ul],
                               inputs[inputBaseIndex + 3ul]};
            const tdm::fquat nq = tdm::normalize(q);
            const tdm::frad3 e = nq.euler<Order>(rotationSigns);
            outputs[outputBaseIndex + 0ul] += TAngle{e[0]}.value;
            outputs[outputBaseIndex + 1ul] += TAngle{e[1]}.value;
            outputs[outputBaseIndex + 2ul] += TAngle{e[2]}.value;
        }
    }
};

template<typename TAngle, tdm::rot_seq Order>
struct EulerAnglesToQuaternions {
    tdm::rot_sign rotationSigns;

    static_assert(std::is_same<TAngle, tdm::fdeg>::value || std::is_same<TAngle, tdm::frad>::value,
                  "TAngle must be either tdm::fdeg or tdm::frad.");

    FORCE_INLINE void adapt(ConstArrayView<float> inputs,
                            ArrayView<float> outputs,
                            ConstArrayView<std::uint16_t> inputRotationBaseIndices,
                            ConstArrayView<std::uint16_t> outputRotationBaseIndices) const {
        for (std::size_t i = {}; i < inputRotationBaseIndices.size(); ++i) {
            const auto inputBaseIndex = inputRotationBaseIndices[i];
            const auto outputBaseIndex = outputRotationBaseIndices[i];
            tdm::frad3 euler{tdm::frad{TAngle{inputs[inputBaseIndex + 0ul]}},
                             tdm::frad{TAngle{inputs[inputBaseIndex + 1ul]}},
                             tdm::frad{TAngle{inputs[inputBaseIndex + 2ul]}}};
            const tdm::fquat q = tdm::fquat::from_euler<Order>(euler, rotationSigns);
            const tdm::fquat oq{outputs[outputBaseIndex + 0ul],
                                outputs[outputBaseIndex + 1ul],
                                outputs[outputBaseIndex + 2ul],
                                outputs[outputBaseIndex + 3ul]};
            const tdm::fquat cq = q * oq;
            outputs[outputBaseIndex + 0ul] = cq.x;
            outputs[outputBaseIndex + 1ul] = cq.y;
            outputs[outputBaseIndex + 2ul] = cq.z;
            outputs[outputBaseIndex + 3ul] = cq.w;
        }
    }
};

}  // namespace ml

}  // namespace rl4
