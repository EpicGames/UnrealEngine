// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/utils/LODRegion.h"
#include "riglogic/types/bpcm/FloatArray.h"

#include <cstdint>

namespace rl4 {

struct JointGroup {
    // All non-zero values
    FloatArray values;
    // Sub-matrix col -> input vector
    Vector<std::uint16_t> inputIndices;
    // Sub-matrix row -> output vector
    Vector<std::uint16_t> outputIndices;
    // Output index boundaries for each LOD
    Vector<LODRegion> lods;
    // Matrix size
    std::uint32_t colCount;
    std::uint32_t rowCount;

    explicit JointGroup(MemoryResource* memRes) :
        values{memRes},
        inputIndices{memRes},
        outputIndices{memRes},
        lods{memRes},
        colCount{},
        rowCount{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(values, inputIndices, outputIndices, lods, colCount, rowCount);
    }
};

}  // namespace rl4
