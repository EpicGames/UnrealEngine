// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/cpu/quaternions/JointGroup.h"
#include "riglogic/system/simd/SIMD.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <cstddef>
#include <cstdint>

// *INDENT-OFF*
namespace rl4 {

/*
 * FastLerp between 4 arbitrary and 4 identity quaternions
 *
 * fastlerp_with_identity(Q, WEIGHT):
 *     IQ = [0, 0, 0, 1]
 *     // Dot product between IQ and Q
 *     D1 = IQ.x * Q.x + IQ.y * Q.y + IQ.z * Q.z + IQ.w * Q.w;
 *     D1 = 0 * Q.x + 0 * Q.y + 0 * Q.z + 1 * Q.w
 *     D1 = Q.w
 *     // Select bias based on dot product
 *     BIAS1 = D1 >= 0 ? 1.0 : -1.0
 *     AW = BIAS1 * (1.0 - WEIGHT)
 *     // Because:
 *     // LERPED = (Q * WEIGHT) + (IQ * AW)
 *     // And:
 *     // IQ.x * AW = 0 * AW
 *     // IQ.y * AW = 0 * AW
 *     // IQ.z * AW = 0 * AW
 *     // IQ.w * AW = 1 * AW
 *     // It follows that:
 *     LERPED = [
 *         Q.x * WEIGHT
 *         Q.y * WEIGHT
 *         Q.z * WEIGHT
 *         Q.w * WEIGHT + AW
 *     ]
 *
 * ------------------------------
 *
 * fastlerp_with_identity_simplified(Q, WEIGHT):
 *     BIAS1 = Q.w >= 0 ? 1.0 : -1.0
 *     AW = BIAS1 * (1.0 - WEIGHT)
 *     LERPED = [
 *         Q.x * WEIGHT
 *         Q.y * WEIGHT
 *         Q.z * WEIGHT
 *         Q.w * WEIGHT + AW
 *     ]
 *
 * ------------------------------
 *
 * A, C, E, G are quaternions representing delta rotations as an array of 4 floats [x, y, z, w].
 *
 * fastlerp_with_identity_simd(A, C, E, G, WEIGHT):
 *     NEG_ONES = [-1, -1, -1, -1]
 *     ONES = [1, 1, 1, 1]
 *     MASK = [
 *         A.w >= 0
 *         C.w >= 0
 *         E.w >= 0
 *         G.w >= 0
 *     ]
 *     BIAS = ANDNOT(MASK, NEG_ONES) | (ONES & MASK)
 *     ONE_MINUS_WEIGHT = [
 *         (1.0 - WEIGHT)
 *         (1.0 - WEIGHT)
 *         (1.0 - WEIGHT)
 *         (1.0 - WEIGHT)
 *     ]
 *     AWS = BIAS * ONE_MINUS_WEIGHT
 *     WEIGHTS = [WEIGHT, WEIGHT, WEIGHT, WEIGHT]
 *     LERPED_X = [
 *         A.x * WEIGHTS
 *         C.x * WEIGHTS
 *         E.x * WEIGHTS
 *         G.x * WEIGHTS
 *     ]
 *     LERPED_Y = [
 *         A.y * WEIGHTS
 *         C.y * WEIGHTS
 *         E.y * WEIGHTS
 *         G.y * WEIGHTS
 *     ]
 *     LERPED_Z = [
 *         A.z * WEIGHTS
 *         C.z * WEIGHTS
 *         E.z * WEIGHTS
 *         G.z * WEIGHTS
 *     ]
 *     LERPED_W = [
 *         A.w * WEIGHTS + AWS
 *         C.w * WEIGHTS + AWS
 *         E.w * WEIGHTS + AWS
 *         G.w * WEIGHTS + AWS
 *     ]
 *     LERPED_A = [LERPED_X[0], LERPED_Y[0], LERPED_Z[0], LERPED_W[0]]
 *     LERPED_C = [LERPED_X[1], LERPED_Y[1], LERPED_Z[1], LERPED_W[1]]
 *     LERPED_E = [LERPED_X[2], LERPED_Y[2], LERPED_Z[2], LERPED_W[2]]
 *     LERPED_G = [LERPED_X[3], LERPED_Y[3], LERPED_Z[3], LERPED_W[3]]
 */
template<typename TFVec>
static FORCE_INLINE void fastlerpWithIdentity(TFVec& qxACEG, TFVec& qyACEG, TFVec& qzACEG, TFVec& qwACEG, const TFVec& weight) {
    const TFVec negOne{-1.0f};
    const TFVec one{1.0f};
    const TFVec oneMinusWeight = one - weight;
    const TFVec mask = (qwACEG >= TFVec{0.0f});
    const TFVec bias = trimd::andnot(mask, negOne) | (one & mask);
    const TFVec aws = bias * oneMinusWeight;
    qxACEG = qxACEG * weight;
    qyACEG = qyACEG * weight;
    qzACEG = qzACEG * weight;
    qwACEG = qwACEG * weight + aws;
}

/*
 * Normalize quaternion
 *
 * normalize(Q):
 *     SS = Q.x * Q.x + Q.y * Q.y + Q.z * Q.z + Q.w * Q.w
 *     INVSS = 1.0 / SQRT(SS)
 *     NORMALIZED = [
 *         Q.x *= INVSS
 *         Q.y *= INVSS
 *         Q.z *= INVSS
 *         Q.w *= INVSS
 *     ]
 *
 * normalize_simd(A, C, E, G):
 *     XS_SQUARED = [
 *         A.x * A.x
 *         C.x * C.x
 *         E.x * E.x
 *         G.x * G.x
 *     ]
 *     YS_SQUARED = [
 *         A.y * A.y
 *         C.y * C.y
 *         E.y * E.y
 *         G.y * G.y
 *     ]
 *     ZS_SQUARED = [
 *         A.z * A.z
 *         C.z * C.z
 *         E.z * E.z
 *         G.z * G.z
 *     ]
 *     WS_SQUARED = [
 *         A.w * A.w
 *         C.w * C.w
 *         E.w * E.w
 *         G.w * G.w
 *     ]
 *     S_SUM = XS_SQUARED + YS_SQUARED + ZS_SQUARED + WS_SQUARED
 *     INVSS = 1.0 / SQRT(S_SUM)  // _mm_rsqrt_ps(S_SUM)
 *     NORMALIZED_A = [
 *         A.x * INVSS
 *         A.y * INVSS
 *         A.z * INVSS
 *         A.w * INVSS
 *     ]
 *     NORMALIZED_B = [
 *         B.x * INVSS
 *         B.y * INVSS
 *         B.z * INVSS
 *         B.w * INVSS
 *     ]
 *     NORMALIZED_C = [
 *         C.x * INVSS
 *         C.y * INVSS
 *         C.z * INVSS
 *         C.w * INVSS
 *     ]
 *     NORMALIZED_D = [
 *         D.x * INVSS
 *         D.y * INVSS
 *         D.z * INVSS
 *         D.w * INVSS
 *     ]
 */
template<typename TFVec>
static FORCE_INLINE void normalize(TFVec& qxACEG, TFVec& qyACEG, TFVec& qzACEG, TFVec& qwACEG) {
    const TFVec xsSquared = qxACEG * qxACEG;
    const TFVec ysSquared = qyACEG * qyACEG;
    const TFVec zsSquared = qzACEG * qzACEG;
    const TFVec wsSquared = qwACEG * qwACEG;
    const TFVec sSum = xsSquared + ysSquared + zsSquared + wsSquared;
    const TFVec rSqrtSum = trimd::rsqrt(sSum);
    qxACEG *= rSqrtSum;
    qyACEG *= rSqrtSum;
    qzACEG *= rSqrtSum;
    qwACEG *= rSqrtSum;
}

/*
 * Multiply each quaternion of a joint group by the expression weight, and combine them together into a single rotation
 *
 * Rotation = FastLerp(IdentityQuaternion, PoseQuaternion, ExpressionWeight);
 * Rotation = Normalize(Rotation);
 * OutputRotation = Rotation * OutputRotation;  // OutputRotation is the accumulator
 *
 * A, C, E, G are quaternions representing delta rotations as an array of 4 floats [x, y, z, w].
 * B, D, F, H are the accumulator quaternions that will contain the combined, blended rotations.
 *
 * The below operation is how a freshly blended set of 4 quaternions are combined with the accumulator
 * quaternions. This first set of multiplications is how each pair is multiplied individually:
 *
 * A[3]*B[0] + A[0]*B[3] + A[1]*B[2] - A[2]*B[1];
 * A[3]*B[1] - A[0]*B[2] + A[1]*B[3] + A[2]*B[0];
 * A[3]*B[2] + A[0]*B[1] - A[1]*B[0] + A[2]*B[3];
 * A[3]*B[3] - A[0]*B[0] - A[1]*B[1] - A[2]*B[2];
 *
 * C[3]*D[0] + C[0]*D[3] + C[1]*D[2] - C[2]*D[1];
 * C[3]*D[1] - C[0]*D[2] + C[1]*D[3] + C[2]*D[0];
 * C[3]*D[2] + C[0]*D[1] - C[1]*D[0] + C[2]*D[3];
 * C[3]*D[3] - C[0]*D[0] - C[1]*D[1] - C[2]*D[2];
 *
 * E[3]*F[0] + E[0]*F[3] + E[1]*F[2] - E[2]*F[1];
 * E[3]*F[1] - E[0]*F[2] + E[1]*F[3] + E[2]*F[0];
 * E[3]*F[2] + E[0]*F[1] - E[1]*F[0] + E[2]*F[3];
 * E[3]*F[3] - E[0]*F[0] - E[1]*F[1] - E[2]*F[2];
 *
 * G[3]*H[0] + G[0]*H[3] + G[1]*H[2] - G[2]*H[1];
 * G[3]*H[1] - G[0]*H[2] + G[1]*H[3] + G[2]*H[0];
 * G[3]*H[2] + G[0]*H[1] - G[1]*H[0] + G[2]*H[3];
 * G[3]*H[3] - G[0]*H[0] - G[1]*H[1] - G[2]*H[2];
 *
 * And the below transformation is the same as the above shown block, only the operations are
 * interleaved so 4 quaternions can be processed with 1 set of SIMD registers efficiently.
 *
 * A[3]*B[0] + A[0]*B[3] + A[1]*B[2] - A[2]*B[1]
 * C[3]*D[0] + C[0]*D[3] + C[1]*D[2] - C[2]*D[1]
 * E[3]*F[0] + E[0]*F[3] + E[1]*F[2] - E[2]*F[1]
 * G[3]*H[0] + G[0]*H[3] + G[1]*H[2] - G[2]*H[1]
 * ---------------------------------------------
 * A[3]*B[1] - A[0]*B[2] + A[1]*B[3] + A[2]*B[0]
 * C[3]*D[1] - C[0]*D[2] + C[1]*D[3] + C[2]*D[0]
 * E[3]*F[1] - E[0]*F[2] + E[1]*F[3] + E[2]*F[0]
 * G[3]*H[1] - G[0]*H[2] + G[1]*H[3] + G[2]*H[0]
 * ---------------------------------------------
 * A[3]*B[2] + A[0]*B[1] - A[1]*B[0] + A[2]*B[3]
 * C[3]*D[2] + C[0]*D[1] - C[1]*D[0] + C[2]*D[3]
 * E[3]*F[2] + E[0]*F[1] - E[1]*F[0] + E[2]*F[3]
 * G[3]*H[2] + G[0]*H[1] - G[1]*H[0] + G[2]*H[3]
 * ---------------------------------------------
 * A[3]*B[3] - A[0]*B[0] - A[1]*B[1] - A[2]*B[2]
 * C[3]*D[3] - C[0]*D[0] - C[1]*D[1] - C[2]*D[2]
 * E[3]*F[3] - E[0]*F[0] - E[1]*F[1] - E[2]*F[2]
 * G[3]*H[3] - G[0]*H[0] - G[1]*H[1] - G[2]*H[2]
 */
template<typename TFVec, typename T>
static FORCE_INLINE void blendQuaternions(const T* quaternions,
                                          ConstArrayView<std::uint16_t> inputIndices,
                                          ConstArrayView<float> inputs,
                                          float* outbuf) {
    // Initialize accumulators to identity quaternion
    TFVec qxBDFH{0.0f};
    TFVec qyBDFH{0.0f};
    TFVec qzBDFH{0.0f};
    TFVec qwBDFH{1.0f};
    for (std::size_t col = {}; col < inputIndices.size(); ++col, quaternions += (TFVec::size() * 4)) {
        TFVec qxACEG = TFVec::fromAlignedSource(quaternions);
        TFVec qyACEG = TFVec::fromAlignedSource(quaternions + TFVec::size());
        TFVec qzACEG = TFVec::fromAlignedSource(quaternions + TFVec::size() * 2);
        TFVec qwACEG = TFVec::fromAlignedSource(quaternions + TFVec::size() * 3);
        // Blend qxACEG, qyACEG, qzACEG, qwACEG with input weights
        const TFVec weights{inputs[inputIndices[col]]};
        fastlerpWithIdentity(qxACEG, qyACEG, qzACEG, qwACEG, weights);
        normalize(qxACEG, qyACEG, qzACEG, qwACEG);
        // Accumulate blended in q[xyzw]Tmp
        const TFVec qxTmp = (qwACEG * qxBDFH) + (qxACEG * qwBDFH) + (qyACEG * qzBDFH) - (qzACEG * qyBDFH);
        const TFVec qyTmp = (qwACEG * qyBDFH) - (qxACEG * qzBDFH) + (qyACEG * qwBDFH) + (qzACEG * qxBDFH);
        const TFVec qzTmp = (qwACEG * qzBDFH) + (qxACEG * qyBDFH) - (qyACEG * qxBDFH) + (qzACEG * qwBDFH);
        const TFVec qwTmp = (qwACEG * qwBDFH) - (qxACEG * qxBDFH) - (qyACEG * qyBDFH) - (qzACEG * qzBDFH);
        qxBDFH = qxTmp;
        qyBDFH = qyTmp;
        qzBDFH = qzTmp;
        qwBDFH = qwTmp;
    }
    qxBDFH.alignedStore(outbuf);
    qyBDFH.alignedStore(outbuf + TFVec::size());
    qzBDFH.alignedStore(outbuf + TFVec::size() * 2);
    qwBDFH.alignedStore(outbuf + TFVec::size() * 3);
}

struct JointGroupQuaternionCalculationStrategy {
    virtual ~JointGroupQuaternionCalculationStrategy();
    virtual void calculate(const JointGroup& jointGroup,
                           ConstArrayView<float> inputs,
                           ArrayView<float> outputs,
                           std::uint16_t lod) const = 0;
};

template<typename T, typename TFVec256, typename TFVec128, class TRotationAdapter>
struct VectorizedJointGroupQuaternionCalculationStrategy : public JointGroupQuaternionCalculationStrategy {
    TRotationAdapter rotationAdapter;

    explicit VectorizedJointGroupQuaternionCalculationStrategy(TRotationAdapter&& rotationAdapter_) :
        rotationAdapter{std::move(rotationAdapter_)} {
    }

    void calculate(const JointGroup& jointGroup,
                   ConstArrayView<float> inputs,
                   ArrayView<float> outputs,
                   std::uint16_t lod) const override {
        const T* quaternions = jointGroup.values.data<T>();
        const LODRegion& lodRegion = jointGroup.lods[lod];
        ConstArrayView<std::uint16_t> inputIndices{jointGroup.inputIndices.data(), lodRegion.inputLODs.size};
        const std::uint16_t* outputIndices = jointGroup.outputIndices.data();
        const std::uint16_t* const outputIndicesEnd = outputIndices + lodRegion.outputLODs.size;
        const std::uint16_t* const outputIndicesEndPaddedToLastFullBlock =
            outputIndices + lodRegion.outputLODs.sizePaddedToLastFullBlock;
        const std::uint16_t* const outputIndicesEndPaddedToSecondLastFullBlock =
            outputIndices + lodRegion.outputLODs.sizePaddedToSecondLastFullBlock;
        const std::size_t fullBlockSize = (TFVec256::size() * 4) * jointGroup.colCount;
        const std::size_t halfBlockSize = (TFVec128::size() * 4) * jointGroup.colCount;

        for (; outputIndices < outputIndicesEndPaddedToSecondLastFullBlock;
             outputIndices += (TFVec256::size() * 4), quaternions += fullBlockSize) {
            alignas(TFVec256::alignment()) float outbuf[TFVec256::size() * 4];
            blendQuaternions<TFVec256>(quaternions, inputIndices, inputs, static_cast<float*>(outbuf));
            rotationAdapter.forward(outbuf, TFVec256::size(), TFVec256::size(), outputIndices, outputs);
        }

        for (; outputIndices < outputIndicesEndPaddedToLastFullBlock;
             outputIndices += (TFVec256::size() * 4), quaternions += fullBlockSize) {
            alignas(TFVec256::alignment()) float outbuf[TFVec256::size() * 4];
            blendQuaternions<TFVec256>(quaternions, inputIndices, inputs, static_cast<float*>(outbuf));
            // Ignore results that came from rows after the last LOD row
            const auto quaternionCount = (lodRegion.outputLODs.size % (TFVec256::size() * 4)) / 4;
            rotationAdapter.forward(outbuf, quaternionCount, TFVec256::size(), outputIndices, outputs);
        }

        for (; outputIndices < outputIndicesEnd; outputIndices += (TFVec128::size() * 4), quaternions += halfBlockSize) {
            alignas(TFVec128::alignment()) float outbuf[TFVec128::size() * 4];
            blendQuaternions<TFVec128>(quaternions, inputIndices, inputs, static_cast<float*>(outbuf));
            // Ignore results that came from rows after the last LOD row
            const auto quaternionCount = static_cast<std::size_t>(outputIndicesEnd - outputIndices) / 4;
            rotationAdapter.forward(outbuf, quaternionCount, TFVec128::size(), outputIndices, outputs);
        }
    }
};

}  // namespace rl4
// *INDENT-ON*
