// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/quaternions/QuaternionJointsBuilder.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsNullEvaluator.h"
#include "riglogic/joints/cpu/quaternions/QuaternionCalculationStrategy.h"
#include "riglogic/joints/cpu/quaternions/QuaternionJointsEvaluator.h"
#include "riglogic/joints/cpu/quaternions/RotationAdapters.h"
#include "riglogic/joints/cpu/utils/JointGroupOptimizer.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Utils.h"
#include "riglogic/types/bpcm/Optimizer.h"
#include "riglogic/utils/Extd.h"

#include <tdm/Quat.h>

namespace rl4 {

namespace qjc {

static constexpr std::uint32_t BlockHeight = 32u;
static constexpr std::uint32_t PadTo = 16u;
static constexpr std::uint32_t Stride = 4u;

}  // namespace qjc

template<typename T, typename TF256, typename TF128>
struct MatrixOptimizer {

    void operator()(ConstArrayView<float> src, Extent srcDims, Extent dstDims, FloatArray& dst) {
        dst.resize<T>(dstDims.size());
        using BPCMOptimizer = bpcm::Optimizer<TF256, qjc::BlockHeight, qjc::PadTo, qjc::Stride>;
        BPCMOptimizer::optimize(dst.data<T>(), src.data(), srcDims);
    }
};

template<typename T, typename TF256, typename TF128>
struct JointGroupQuaternionStrategyFactory {
    using BasePointer = UniqueInstance<JointGroupQuaternionCalculationStrategy>::PointerType;

    BasePointer operator()(RotationType rotationType,
                           tdm::rot_seq rotationSequence,
                           tdm::rot_sign rotationSigns,
                           dna::RotationUnit rotationUnit,
                           MemoryResource* memRes) {

        if (rotationType == RotationType::Quaternions) {
            using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, PassthroughAdapter>;
            return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                PassthroughAdapter{rotationSigns});
        }

#ifdef RL_BUILD_WITH_XYZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xyz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::xyz>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::xyz>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_XYZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_XZY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xzy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::xzy>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::xzy>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_XZY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YXZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yxz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::yxz>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::yxz>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_YXZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YZX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yzx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::yzx>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::yzx>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_YZX_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZXY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zxy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::zxy>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zxy>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_ZXY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZYX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zyx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::zyx>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>;
                using CalculationStrategy = VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, Q2E>;
                return UniqueInstance<CalculationStrategy, JointGroupQuaternionCalculationStrategy>::with(memRes).create(
                    Q2E{rotationSigns});
            }
        }
#endif  // RL_BUILD_WITH_ZYX_ROTATION_ORDER

        return nullptr;
    }
};

QuaternionJointsBuilder::QuaternionJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_) :
    config{config_},
    meta{meta_},
    memRes{memRes_},
    jointGroups{memRes_},
    rotationUnit{} {
}

void QuaternionJointsBuilder::computeStorageRequirements() {
}

void QuaternionJointsBuilder::computeStorageRequirements(const JointBehaviorFilter& /*unused*/) {
}

void QuaternionJointsBuilder::allocateStorage(const JointBehaviorFilter& source) {
    jointGroups.resize(source.getJointGroupCount(), JointGroup{memRes});
    for (auto& group : jointGroups) {
        group.lods.resize(source.getLODCount());
    }
}

void QuaternionJointsBuilder::setInputIndices(JointGroup& group, ConstArrayView<std::uint16_t> inputIndices) {
    group.inputIndices.assign(inputIndices.begin(), inputIndices.end());
}

void QuaternionJointsBuilder::setOutputIndices(JointGroup& group, ConstArrayView<std::uint16_t> outputIndices) {
// Given any rotation indices (rx, ry, rz), return only rx indices for all joints in the group
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wattributes"
#endif
    auto deduplicate = [this](Vector<std::uint16_t>& v) {
        UnorderedSet<std::uint16_t> deduplicator{memRes};
        v.erase(v.rend().base(), std::remove_if(v.rbegin(), v.rend(), [&deduplicator](const std::uint16_t value) {
                                     return !deduplicator.insert(value).second;
                                 }).base());
    };
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
    Vector<std::uint16_t> outputRotationBaseIndices{memRes};
    outputRotationBaseIndices.reserve(outputIndices.size() / 3ul);
    std::transform(outputIndices.begin(),
                   outputIndices.end(),
                   std::back_inserter(outputRotationBaseIndices),
                   [](std::uint16_t outputIndex) { return static_cast<std::uint16_t>((outputIndex / 9) * 9); });
    deduplicate(outputRotationBaseIndices);
    // Expand output rotation base indices (qx) into (qx, qy, qz, qw) for all rotations
    group.outputIndices.reserve(outputRotationBaseIndices.size() * static_cast<std::uint8_t>(RotationType::Quaternions));
    for (const auto baseIndex : outputRotationBaseIndices) {
        // Remap output indices from 9-attribute joints to 10-attribute joints rx -> qx
        const auto jointIndex = static_cast<std::uint16_t>(baseIndex / 9u);
        const auto remappedBaseIndex = static_cast<std::uint16_t>(jointIndex * 10u);
        group.outputIndices.push_back(static_cast<std::uint16_t>(remappedBaseIndex + 3));
        group.outputIndices.push_back(static_cast<std::uint16_t>(remappedBaseIndex + 4));
        group.outputIndices.push_back(static_cast<std::uint16_t>(remappedBaseIndex + 5));
        group.outputIndices.push_back(static_cast<std::uint16_t>(remappedBaseIndex + 6));
    }
}

void QuaternionJointsBuilder::setValues(JointGroup& group,
                                        ConstArrayView<float> eulers,
                                        ConstArrayView<std::uint16_t> inputIndices,
                                        ConstArrayView<std::uint16_t> outputIndices) {
    // Convert euler angles to quaternions
    std::function<tdm::frad(float)> angConv;
    if (rotationUnit == dna::RotationUnit::degrees) {
        angConv = [](float angle) { return tdm::frad{tdm::fdeg{angle}}; };
    } else {
        angConv = [](float angle) { return tdm::frad{angle}; };
    }

    const auto colCount = static_cast<std::uint16_t>(inputIndices.size());
    const auto rowCount = static_cast<std::uint16_t>(outputIndices.size());

    Vector<float> quaternions(group.outputIndices.size() * colCount, {}, memRes);
    for (std::size_t col = {}; col < colCount; ++col) {
        for (std::size_t row = {}, quatIndex = {}; row < rowCount; ++quatIndex) {
            tdm::frad3 angles;
            const std::uint16_t jointIndex = static_cast<std::uint16_t>(outputIndices[row] / 9u);
            while ((row < rowCount) && (jointIndex == static_cast<std::uint16_t>(outputIndices[row] / 9u))) {
                const auto relAttrIndex = static_cast<std::uint16_t>(outputIndices[row] % 9u);
                // 0 = rx, 1 = ry, 2 = rz
                const auto relRotAttrIndex = static_cast<std::uint16_t>(relAttrIndex % 3u);
                angles[relRotAttrIndex] = angConv(eulers[row * colCount + col]);
                ++row;
            }
            tdm::fquat q{angles, meta->rotationSequence, meta->rotationSigns};
            quaternions[((quatIndex * 4) + 0ul) * colCount + col] = q.x;
            quaternions[((quatIndex * 4) + 1ul) * colCount + col] = q.y;
            quaternions[((quatIndex * 4) + 2ul) * colCount + col] = q.z;
            quaternions[((quatIndex * 4) + 3ul) * colCount + col] = q.w;
        }
    }

    // 8 quaternions x 4 floats per quat = 32
    const auto newRowCount = static_cast<std::uint32_t>(group.outputIndices.size());
    const auto paddedRowCount = extd::roundUp(newRowCount, qjc::PadTo);

    group.colCount = colCount;
    group.rowCount = paddedRowCount;
    RuntimeTemplateInstantiator instantiator{&config};
    Extent srcDims{newRowCount, colCount};
    Extent dstDims{paddedRowCount, colCount};
    instantiator.invoke<MatrixOptimizer, void>(quaternions, srcDims, dstDims, group.values);
}

void QuaternionJointsBuilder::setLODs(JointGroup& group, ConstArrayView<std::uint16_t> outputIndices) {
    const auto maxRemappedRotationIndex = [](std::uint16_t absRotAttrIndex) {
        const auto jointIndex = static_cast<std::uint16_t>(absRotAttrIndex / 9u);
        const auto newAttrBase = static_cast<std::uint16_t>(jointIndex * 10u);
        // Only rotation indices are inputs, and since the goal is to find the maximum rotation index,
        // the last quaternion attribute index is used, which is 6 based on [tx, ty, tz, qx, qy, qz, qw, sx, sy, sz]
        return static_cast<std::uint16_t>(newAttrBase + 6);
    };

    const auto newRowCount = static_cast<std::uint32_t>(group.outputIndices.size());
    const auto paddedRowCount = extd::roundUp(newRowCount, qjc::PadTo);
    const auto lodCount = static_cast<std::uint16_t>(group.lods.size());
    for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
        const std::uint32_t oldLODRowCount = group.lods[lod].outputLODs.size;
        std::uint32_t newLODRowCount = {};
        if (oldLODRowCount != 0) {
            const auto qwRotationIndexAtOldLODRowCount = maxRemappedRotationIndex(outputIndices[oldLODRowCount - 1ul]);
            auto it = std::find(group.outputIndices.begin(), group.outputIndices.end(), qwRotationIndexAtOldLODRowCount);
            assert(it != group.outputIndices.end());
            newLODRowCount = static_cast<std::uint16_t>(std::distance(group.outputIndices.begin(), it) + 1);
        }
        RowLOD outputLOD{newLODRowCount, paddedRowCount, qjc::BlockHeight, qjc::PadTo};
        group.lods[lod].outputLODs = outputLOD;
    }
}

void QuaternionJointsBuilder::remapOutputIndices(JointGroup& group) {
    for (auto& outputIndex : group.outputIndices) {
        const auto jointIndex = static_cast<std::uint16_t>(outputIndex / 10u);
        const auto relAttrIndex = static_cast<std::uint16_t>(outputIndex % 10u);
        const auto newAttrBase = static_cast<std::uint16_t>(jointIndex * 9u);
        // Only rotations are among output indices (no translation or scale)
        // qx, qy, qz are kept, qw is ignored
        outputIndex = (relAttrIndex == 6) ? std::uint16_t{} : static_cast<std::uint16_t>(newAttrBase + relAttrIndex);
    }
}

void QuaternionJointsBuilder::fillStorage(const JointBehaviorFilter& source) {
    rotationUnit = source.getRotationUnit();

    for (std::uint16_t jgi = {}; jgi < static_cast<std::uint16_t>(jointGroups.size()); ++jgi) {
        auto rowCount = source.getRowCount(jgi);
        auto colCount = source.getColumnCount(jgi);
        if ((rowCount == 0u) || (colCount == 0u)) {
            continue;
        }

        JointGroup& group = jointGroups[jgi];

        Vector<float> eulers(static_cast<std::size_t>(rowCount) * static_cast<std::size_t>(colCount), {}, memRes);
        source.copyValues(jgi, eulers);

        Vector<std::uint16_t> inputIndices{colCount, {}, memRes};
        source.copyInputIndices(jgi, inputIndices);

        Vector<std::uint16_t> outputIndices{rowCount, {}, memRes};
        source.copyOutputIndices(jgi, outputIndices);

        JointGroupOptimizer::defragment(source,
                                        jgi,
                                        eulers,
                                        inputIndices,
                                        outputIndices,
                                        group.lods,
                                        config.translationPruningThreshold,
                                        config.rotationPruningThreshold,
                                        config.scalePruningThreshold);

        setInputIndices(group, inputIndices);
        setOutputIndices(group, outputIndices);
        setValues(group, eulers, inputIndices, outputIndices);
        setLODs(group, outputIndices);

        // If the selected RigLogic output is in quaternions, then the output indices are already setup as needed.
        // But if Euler angles were requested, the output indices need to be mapped back to 9-attribute joint output indices
        if (config.rotationType == RotationType::EulerAngles) {
            remapOutputIndices(group);
        }
    }
}

void QuaternionJointsBuilder::registerControls(Controls* controls) {
    for (const auto& group : jointGroups) {
        for (std::uint16_t lod = {}; lod < static_cast<std::uint16_t>(group.lods.size()); ++lod) {
            ConstArrayView<std::uint16_t> inputIndicesForLOD(group.inputIndices.data(), group.lods[lod].inputLODs.size);
            controls->registerControls(lod, inputIndicesForLOD);
        }
    }
}

JointsEvaluator::Pointer QuaternionJointsBuilder::build() {
    const EvaluatorType type =
        (meta->initializationMethod == InitializationMethod::Restore) ? meta->popFrontEvaluator() : EvaluatorType::Auto;

    auto jointGroupsEmpty = [this]() {
        for (std::size_t i = {}; i < jointGroups.size(); ++i) {
            if (jointGroups[i].rowCount != 0u) {
                return false;
            }
        }
        return true;
    };

    if ((type == EvaluatorType::Null) || ((type == EvaluatorType::Auto) && jointGroupsEmpty())) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    RuntimeTemplateInstantiator instantiator{&config};
    using StrategyPointer = UniqueInstance<JointGroupQuaternionCalculationStrategy>::PointerType;
    auto strategy = instantiator.invoke<JointGroupQuaternionStrategyFactory, StrategyPointer>(config.rotationType,
                                                                                              meta->rotationSequence,
                                                                                              meta->rotationSigns,
                                                                                              rotationUnit,
                                                                                              memRes);

    if (strategy == nullptr) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    auto factory = UniqueInstance<QuaternionJointsEvaluator, JointsEvaluator>::with(memRes);
    return factory.create(std::move(strategy), std::move(jointGroups), nullptr, memRes);
}

}  // namespace rl4
