// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/cpu/quaternions/JointGroup.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"

namespace rl4 {

class QuaternionJointsBuilder : public JointsBuilder {
public:
    QuaternionJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_);

    void computeStorageRequirements() override;
    void computeStorageRequirements(const JointBehaviorFilter& source) override;
    void allocateStorage(const JointBehaviorFilter& source) override;
    void fillStorage(const JointBehaviorFilter& source) override;
    void registerControls(Controls* controls) override;
    JointsEvaluator::Pointer build() override;

private:
    void setInputIndices(JointGroup& group, ConstArrayView<std::uint16_t> inputIndices);
    void setOutputIndices(JointGroup& group, ConstArrayView<std::uint16_t> outputIndices);
    void setValues(JointGroup& group,
                   ConstArrayView<float> eulers,
                   ConstArrayView<std::uint16_t> inputIndices,
                   ConstArrayView<std::uint16_t> outputIndices);
    void setLODs(JointGroup& group, ConstArrayView<std::uint16_t> outputIndices);
    void remapOutputIndices(JointGroup& group);

private:
    Configuration config;
    RigMetadata* meta;
    MemoryResource* memRes;
    Vector<JointGroup> jointGroups;
    dna::RotationUnit rotationUnit;
};

}  // namespace rl4
