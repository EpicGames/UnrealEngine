// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/quaternions/QuaternionJointsBuilderFactory.h"

#include "riglogic/joints/cpu/quaternions/QuaternionJointsBuilder.h"

namespace rl4 {

UniqueInstance<JointsBuilder>::PointerType QuaternionJointsBuilderFactory::create(const Configuration& config,
                                                                                  RigMetadata* meta,
                                                                                  MemoryResource* memRes) {
    return UniqueInstance<QuaternionJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
}

}  // namespace rl4
