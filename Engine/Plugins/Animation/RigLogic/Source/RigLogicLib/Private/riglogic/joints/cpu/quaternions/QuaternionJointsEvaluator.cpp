// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/quaternions/QuaternionJointsEvaluator.h"

namespace rl4 {

QuaternionJointsEvaluator::QuaternionJointsEvaluator(CalculationStrategyPointer strategy_,
                                                     Vector<JointGroup>&& jointGroups_,
                                                     JointsOutputInstance::Factory instanceFactory_,
                                                     MemoryResource* /*unused*/) :
    strategy{std::move(strategy_)},
    jointGroups{std::move(jointGroups_)},
    instanceFactory{instanceFactory_} {
}

JointsOutputInstance::Pointer QuaternionJointsEvaluator::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(instanceMemRes);
}

std::uint32_t QuaternionJointsEvaluator::getJointDeltaValueCountForLOD(std::uint16_t lod) const {
    std::uint32_t deltaCount = {};
    for (const auto& group : jointGroups) {
        assert(lod < group.lods.size());
        deltaCount += (group.lods[lod].inputLODs.size * group.lods[lod].outputLODs.size);
    }
    return deltaCount;
}

void QuaternionJointsEvaluator::calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const {
    for (std::size_t i = {}; i < jointGroups.size(); ++i) {
        calculate(inputs, outputs, lod, static_cast<std::uint16_t>(i));
    }
}

void QuaternionJointsEvaluator::calculate(ControlsInputInstance* inputs,
                                          JointsOutputInstance* outputs,
                                          std::uint16_t lod,
                                          std::uint16_t jointGroupIndex) const {
    const auto& jointGroup = jointGroups[jointGroupIndex];
    if (jointGroup.rowCount != 0u) {
        strategy->calculate(jointGroup, inputs->getInputBuffer(), outputs->getOutputBuffer(), lod);
    }
}

void QuaternionJointsEvaluator::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(jointGroups);
}

void QuaternionJointsEvaluator::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(jointGroups);
}

}  // namespace rl4
