// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/joints/cpu/quaternions/JointGroup.h"
#include "riglogic/joints/cpu/quaternions/QuaternionCalculationStrategy.h"

#include <cstdint>

namespace rl4 {

class QuaternionJointsEvaluator : public JointsEvaluator {
public:
    struct Accessor;
    friend Accessor;

private:
    using CalculationStrategyPointer = typename UniqueInstance<JointGroupQuaternionCalculationStrategy>::PointerType;

public:
    QuaternionJointsEvaluator(CalculationStrategyPointer strategy_,
                              Vector<JointGroup>&& jointGroups_,
                              JointsOutputInstance::Factory instanceFactory_,
                              MemoryResource* memRes);

    JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs,
                   JointsOutputInstance* outputs,
                   std::uint16_t lod,
                   std::uint16_t jointGroupIndex) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    CalculationStrategyPointer strategy;
    Vector<JointGroup> jointGroups;
    JointsOutputInstance::Factory instanceFactory;
};

}  // namespace rl4
