// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/system/simd/SIMD.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <tdm/Quat.h>

namespace rl4 {

struct PassthroughAdapter {
    tdm::rot_sign rotationSigns;

    static FORCE_INLINE void forward(const float* quaternions,
                                     std::size_t count,
                                     std::size_t stride,
                                     const std::uint16_t* outputIndices,
                                     ArrayView<float> outputs) {
        // quaternions    [x, x, x, x, x, x, x, x, y, y, y, y, y, y, y, y, z, z, z, z, z, z, z, z, w, w, w, w, w, w, w, w]
        // output indices [x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w]
        for (std::size_t qi = {}; qi < count; ++qi) {
            outputs[outputIndices[qi * 4 + 0]] = quaternions[stride * 0 + qi];  // x
            outputs[outputIndices[qi * 4 + 1]] = quaternions[stride * 1 + qi];  // y
            outputs[outputIndices[qi * 4 + 2]] = quaternions[stride * 2 + qi];  // z
            outputs[outputIndices[qi * 4 + 3]] = quaternions[stride * 3 + qi];  // w
        }
    }

    static FORCE_INLINE void reverse(float* quaternions,
                                     std::size_t count,
                                     std::size_t stride,
                                     const std::uint16_t* outputIndices,
                                     ConstArrayView<float> outputs) {
        // output indices [x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w]
        // quaternions    [x, x, x, x, x, x, x, x, y, y, y, y, y, y, y, y, z, z, z, z, z, z, z, z, w, w, w, w, w, w, w, w]
        for (std::size_t qi = {}; qi < count; ++qi) {
            quaternions[stride * 0 + qi] = outputs[outputIndices[qi * 4 + 0]];  // x
            quaternions[stride * 1 + qi] = outputs[outputIndices[qi * 4 + 1]];  // y
            quaternions[stride * 2 + qi] = outputs[outputIndices[qi * 4 + 2]];  // z
            quaternions[stride * 3 + qi] = outputs[outputIndices[qi * 4 + 3]];  // w
        }
    }
};

template<typename TAngle, tdm::rot_seq Order>
struct QuaternionsToEulerAngles {
    tdm::rot_sign rotationSigns;

    FORCE_INLINE void forward(const float* quaternions,
                              std::size_t count,
                              std::size_t stride,
                              const std::uint16_t* outputIndices,
                              ArrayView<float> outputs) const {
        // quaternions    [x, x, x, x, x, x, x, x, y, y, y, y, y, y, y, y, z, z, z, z, z, z, z, z, w, w, w, w, w, w, w, w]
        // output indices [x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w]
        for (std::size_t qi = {}; qi < count; ++qi) {
            const tdm::fquat q{quaternions[stride * 0 + qi],
                               quaternions[stride * 1 + qi],
                               quaternions[stride * 2 + qi],
                               quaternions[stride * 3 + qi]};
            const tdm::frad3 e = q.euler<Order>(rotationSigns);
            outputs[outputIndices[qi * 4 + 0]] = TAngle{e[0]}.value;
            outputs[outputIndices[qi * 4 + 1]] = TAngle{e[1]}.value;
            outputs[outputIndices[qi * 4 + 2]] = TAngle{e[2]}.value;
            // outputs[outputIndices[qi * 4 + 3]] is padding, always 0, and is just ignored
        }
    }

    FORCE_INLINE void reverse(float* quaternions,
                              std::size_t count,
                              std::size_t stride,
                              const std::uint16_t* outputIndices,
                              ConstArrayView<float> outputs) const {
        // output indices [x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w, x, y, z, w]
        // quaternions    [x, x, x, x, x, x, x, x, y, y, y, y, y, y, y, y, z, z, z, z, z, z, z, z, w, w, w, w, w, w, w, w]
        for (std::size_t qi = {}; qi < count; ++qi) {
            const tdm::frad3 e{tdm::frad{TAngle{outputs[outputIndices[qi * 4 + 0]]}},
                               tdm::frad{TAngle{outputs[outputIndices[qi * 4 + 1]]}},
                               tdm::frad{TAngle{outputs[outputIndices[qi * 4 + 2]]}}};
            // outputs[outputIndices[qi * 4 + 3]] is padding, always 0, and is just ignored
            const tdm::fquat q = tdm::fquat::from_euler<Order>(e, rotationSigns);
            quaternions[stride * 0 + qi] = q.x;
            quaternions[stride * 1 + qi] = q.y;
            quaternions[stride * 2 + qi] = q.z;
            quaternions[stride * 3 + qi] = q.w;
        }
    }
};

}  // namespace rl4
