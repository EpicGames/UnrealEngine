// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/JointsNullEvaluator.h"
#include "riglogic/joints/cpu/quaternions/RotationAdapters.h"
#include "riglogic/joints/cpu/twistswing/TwistSwingJointsEvaluator.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/utils/Extd.h"

namespace rl4 {

template<class TContainer, class UContainer>
static bool containersEqual(const TContainer& lhs, const UContainer& rhs) {
    if (lhs.size() != rhs.size()) {
        return false;
    }

#if __cplusplus >= 201402L || (defined(_MSC_VER) && _MSC_VER >= 1900)
    return std::equal(lhs.begin(), lhs.end(), rhs.begin(), rhs.end());
#else
    return std::equal(lhs.begin(), lhs.end(), rhs.begin());
#endif
}

template<typename TValue, typename TFVec256, typename TFVec128>
class TwistSwingJointsBuilder : public JointsBuilder {
public:
    TwistSwingJointsBuilder(const Configuration& config_, RigMetadata* meta_, MemoryResource* memRes_);

    void computeStorageRequirements() override;
    void computeStorageRequirements(const JointBehaviorFilter& source) override;
    void allocateStorage(const JointBehaviorFilter& source) override;
    void fillStorage(const JointBehaviorFilter& source) override;
    void registerControls(Controls* controls) override;
    JointsEvaluator::Pointer build() override;

private:
    Configuration config;
    RigMetadata* meta;
    MemoryResource* memRes;
    Vector<TwistSwingSetup> setups;
    dna::RotationUnit rotationUnit;
};

template<typename TValue, typename TFVec256, typename TFVec128>
TwistSwingJointsBuilder<TValue, TFVec256, TFVec128>::TwistSwingJointsBuilder(const Configuration& config_,
                                                                             RigMetadata* meta_,
                                                                             MemoryResource* memRes_) :
    config{config_},
    meta{meta_},
    memRes{memRes_},
    setups{memRes_},
    rotationUnit{} {
}

template<typename TValue, typename TFVec256, typename TFVec128>
void TwistSwingJointsBuilder<TValue, TFVec256, TFVec128>::computeStorageRequirements() {
}

template<typename TValue, typename TFVec256, typename TFVec128>
void TwistSwingJointsBuilder<TValue, TFVec256, TFVec128>::computeStorageRequirements(const JointBehaviorFilter& /*unused*/) {
}

template<typename TValue, typename TFVec256, typename TFVec128>
void TwistSwingJointsBuilder<TValue, TFVec256, TFVec128>::allocateStorage(const JointBehaviorFilter& /*unused*/) {
}

template<typename TValue, typename TFVec256, typename TFVec128>
void TwistSwingJointsBuilder<TValue, TFVec256, TFVec128>::fillStorage(const JointBehaviorFilter& source) {
    rotationUnit = source.getRotationUnit();
    const dna::Reader* reader = source.getReader();
    const auto swingCount = reader->getSwingCount();
    const auto twistCount = reader->getTwistCount();

    if (!config.loadTwistSwingBehavior || ((twistCount == 0u) && (swingCount == 0u))) {
        return;
    }

    setups.reserve(std::max(twistCount, swingCount));

    for (std::uint16_t si = {}; si < swingCount; ++si) {
        TwistSwingSetup setup{memRes};

        const auto swingTwistAxis = reader->getSwingSetupTwistAxis(si);
        const auto swingInputIndices = reader->getSwingInputControlIndices(si);
        const auto swingOutputIndices = reader->getSwingOutputJointIndices(si);
        const auto swingBlendWeights = reader->getSwingBlendWeights(si);

        assert(swingInputIndices.size() == 4ul);
        assert(swingBlendWeights.size() == swingOutputIndices.size());

        setup.swingTwistAxis = swingTwistAxis;
        setup.swingInputIndices.assign(swingInputIndices.begin(), swingInputIndices.end());
        setup.swingBlendWeights.assign(swingBlendWeights.begin(), swingBlendWeights.end());
        setup.swingOutputIndices.reserve(swingOutputIndices.size() * 10ul);
        if (config.rotationType == RotationType::EulerAngles) {
            for (const auto jointIndex : swingOutputIndices) {
                const auto absBaseAttrIndex = jointIndex * 9ul;
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 3u));
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 4u));
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 5u));
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(0));
            }
        } else {
            for (const auto jointIndex : swingOutputIndices) {
                const auto absBaseAttrIndex = jointIndex * 10ul;
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 3u));
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 4u));
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 5u));
                setup.swingOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 6u));
            }
        }

        setups.push_back(std::move(setup));
    }

    for (std::uint16_t ti = {}; ti < twistCount; ++ti) {
        const auto twistInputIndices = reader->getTwistInputControlIndices(ti);
        const auto twistOutputIndices = reader->getTwistOutputJointIndices(ti);
        const auto twistBlendWeights = reader->getTwistBlendWeights(ti);
        const auto twistTwistAxis = reader->getTwistSetupTwistAxis(ti);

        assert(twistInputIndices.size() == 4ul);
        assert(twistBlendWeights.size() == twistOutputIndices.size());

        auto fillSetup =
            [this, twistInputIndices, twistOutputIndices, twistBlendWeights, twistTwistAxis](TwistSwingSetup& setup) {
                setup.twistTwistAxis = twistTwistAxis;
                setup.twistInputIndices.assign(twistInputIndices.begin(), twistInputIndices.end());
                setup.twistBlendWeights.assign(twistBlendWeights.begin(), twistBlendWeights.end());
                setup.twistOutputIndices.reserve(twistOutputIndices.size() * 10ul);
                if (config.rotationType == RotationType::EulerAngles) {
                    for (const auto jointIndex : twistOutputIndices) {
                        const auto absBaseAttrIndex = jointIndex * 9ul;
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 3u));
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 4u));
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 5u));
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(0));
                    }
                } else {
                    for (const auto jointIndex : twistOutputIndices) {
                        const auto absBaseAttrIndex = jointIndex * 10ul;
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 3u));
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 4u));
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 5u));
                        setup.twistOutputIndices.push_back(static_cast<std::uint16_t>(absBaseAttrIndex + 6u));
                    }
                }
            };

        auto it = std::find_if(setups.begin(), setups.end(), [twistInputIndices](const TwistSwingSetup& setup) {
            return containersEqual(setup.swingInputIndices, twistInputIndices);
        });
        if (it == setups.end()) {
            TwistSwingSetup setup{memRes};
            fillSetup(setup);
            setups.push_back(std::move(setup));
        } else {
            fillSetup(*it);
        }
    }
}

template<typename TValue, typename TFVec256, typename TFVec128>
void TwistSwingJointsBuilder<TValue, TFVec256, TFVec128>::registerControls(Controls* /*unused*/) {
}

template<typename TValue, typename TFVec256, typename TFVec128>
struct TwistSwingJointsEvaluatorFactory {

    JointsEvaluator::Pointer operator()(RotationType rotationType,
                                        tdm::rot_seq rotationSequence,
                                        tdm::rot_sign rotationSigns,
                                        dna::RotationUnit rotationUnit,
                                        Vector<TwistSwingSetup> setups,
                                        MemoryResource* memRes) {
        if (rotationType == RotationType::Quaternions) {
            using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, PassthroughAdapter>;
            return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                      PassthroughAdapter{},
                                                                                      nullptr,
                                                                                      memRes);
        }

#ifdef RL_BUILD_WITH_XYZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xyz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::xyz>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::xyz>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            }
        }
#endif  // RL_BUILD_WITH_XYZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_XZY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::xzy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::xzy>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::xzy>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            }
        }
#endif  // RL_BUILD_WITH_XZY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YXZ_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yxz) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::yxz>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::yxz>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            }
        }
#endif  // RL_BUILD_WITH_YXZ_ROTATION_ORDER

#ifdef RL_BUILD_WITH_YZX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::yzx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::yzx>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::yzx>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            }
        }
#endif  // RL_BUILD_WITH_YZX_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZXY_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zxy) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::zxy>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zxy>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            }
        }
#endif  // RL_BUILD_WITH_ZXY_ROTATION_ORDER

#ifdef RL_BUILD_WITH_ZYX_ROTATION_ORDER
        if (rotationSequence == tdm::rot_seq::zyx) {
            if (rotationUnit == dna::RotationUnit::degrees) {
                using Q2E = QuaternionsToEulerAngles<tdm::fdeg, tdm::rot_seq::zyx>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            } else {
                using Q2E = QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>;
                using TSJEvaluator = TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, Q2E>;
                return UniqueInstance<TSJEvaluator, JointsEvaluator>::with(memRes).create(std::move(setups),
                                                                                          Q2E{rotationSigns},
                                                                                          nullptr,
                                                                                          memRes);
            }
        }
#endif  // RL_BUILD_WITH_ZYX_ROTATION_ORDER

        return nullptr;
    }
};

template<typename TValue, typename TFVec256, typename TFVec128>
JointsEvaluator::Pointer TwistSwingJointsBuilder<TValue, TFVec256, TFVec128>::build() {
    const EvaluatorType type =
        (meta->initializationMethod == InitializationMethod::Restore) ? meta->popFrontEvaluator() : EvaluatorType::Auto;

    if ((type == EvaluatorType::Null) || ((type == EvaluatorType::Auto) && setups.empty())) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    TwistSwingJointsEvaluatorFactory<TValue, TFVec256, TFVec128> factory;
    auto evaluator =
        factory(config.rotationType, meta->rotationSequence, meta->rotationSigns, rotationUnit, std::move(setups), memRes);

    if (evaluator == nullptr) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<JointsNullEvaluator, JointsEvaluator>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    return evaluator;
}

}  // namespace rl4
