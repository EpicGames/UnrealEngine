// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/twistswing/TwistSwingJointsBuilderFactory.h"

#include "riglogic/joints/cpu/twistswing/TwistSwingJointsBuilder.h"
#include "riglogic/system/simd/Detect.h"

namespace rl4 {

UniqueInstance<JointsBuilder>::PointerType TwistSwingJointsBuilderFactory::create(const Configuration& config,
                                                                                  RigMetadata* meta,
                                                                                  MemoryResource* memRes) {
    // At the moment, only a Scalar implementation exists for twist and swing evaluation, so there is no need to
    // compile any other configuration.
    /*
    auto features = trimd::getCPUFeatures();
    RL_UNUSED(features);
    #ifdef RL_BUILD_WITH_SSE
        #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
        features.SSE2 = true;
        #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
        if (features.SSE2 && ((config.calculationType == CalculationType::SSE) || (config.calculationType ==
    CalculationType::AnyVector))) { #ifdef RL_BUILD_WITH_HALF_FLOATS #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION features.F16C =
    true; #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION if (features.F16C) { using SSETwistSwingJointsBuilder =
    TwistSwingJointsBuilder<std::uint16_t, trimd::sse::F256, trimd::sse::F128>; return UniqueInstance<SSETwistSwingJointsBuilder,
    JointsBuilder>::with(memRes).create(config, meta, memRes);
                }
            #endif  // RL_BUILD_WITH_HALF_FLOATS
            using SSETwistSwingJointsBuilder = TwistSwingJointsBuilder<float, trimd::sse::F256, trimd::sse::F128>;
            return UniqueInstance<SSETwistSwingJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
        }
    #endif  // RL_BUILD_WITH_SSE
    #ifdef RL_BUILD_WITH_AVX
        #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
        features.AVX = true;
        #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
        if (features.AVX && ((config.calculationType == CalculationType::AVX) || (config.calculationType ==
    CalculationType::AnyVector))) { #ifdef RL_BUILD_WITH_HALF_FLOATS #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION features.F16C =
    true; #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION if (features.F16C) { using AVXTwistSwingJointsBuilder =
    TwistSwingJointsBuilder<std::uint16_t, trimd::avx::F256, trimd::sse::F128>; return UniqueInstance<AVXTwistSwingJointsBuilder,
    JointsBuilder>::with(memRes).create(config, meta, memRes);
                }
            #endif  // RL_BUILD_WITH_HALF_FLOATS
            using AVXTwistSwingJointsBuilder = TwistSwingJointsBuilder<float, trimd::avx::F256, trimd::sse::F128>;
            return UniqueInstance<AVXTwistSwingJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
        }
    #endif  // RL_BUILD_WITH_AVX
    #ifdef RL_BUILD_WITH_NEON
        #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
        features.NEON = true;
        #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
        if (features.NEON && ((config.calculationType == CalculationType::NEON) || (config.calculationType ==
    CalculationType::AnyVector))) { #ifdef RL_BUILD_WITH_HALF_FLOATS #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION features.FP16 =
    true; #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION if (features.FP16) { using NEONTwistSwingJointsBuilder =
    TwistSwingJointsBuilder<std::uint16_t, trimd::neon::F256, trimd::neon::F128>; return
    UniqueInstance<NEONTwistSwingJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
                }
            #endif  // RL_BUILD_WITH_HALF_FLOATS
            using NEONTwistSwingJointsBuilder = TwistSwingJointsBuilder<float, trimd::neon::F256, trimd::neon::F128>;
            return UniqueInstance<NEONTwistSwingJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
        }
    #endif  // RL_BUILD_WITH_NEON
    */
    using ScalarTwistSwingJointsBuilder = TwistSwingJointsBuilder<float, trimd::scalar::F256, trimd::scalar::F128>;
    return UniqueInstance<ScalarTwistSwingJointsBuilder, JointsBuilder>::with(memRes).create(config, meta, memRes);
}

}  // namespace rl4
