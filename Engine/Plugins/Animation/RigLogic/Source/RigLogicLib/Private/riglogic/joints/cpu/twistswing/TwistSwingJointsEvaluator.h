// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointsEvaluator.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/joints/cpu/twistswing/TwistSwingSetup.h"

#include <tdm/Computations.h>
#include <tdm/Quat.h>

#include <cstddef>
#include <cstdint>

namespace rl4 {

static inline tdm::fquat separateTwistComponentByAxisX(const tdm::fquat& input) {
    const tdm::fquat twist{input.x, 0.0f, 0.0f, input.w};
    return (twist.length2() == 0.0f) ? tdm::fquat{} : tdm::normalize(twist);
}

static inline tdm::fquat separateTwistComponentByAxisY(const tdm::fquat& input) {
    const tdm::fquat twist{0.0f, input.y, 0.0f, input.w};
    return (twist.length2() == 0.0f) ? tdm::fquat{} : tdm::normalize(twist);
}

static inline tdm::fquat separateTwistComponentByAxisZ(const tdm::fquat& input) {
    const tdm::fquat twist{0.0f, 0.0f, input.z, input.w};
    return (twist.length2() == 0.0f) ? tdm::fquat{} : tdm::normalize(twist);
}

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
class TwistSwingJointsEvaluator : public JointsEvaluator {
public:
    struct Accessor;
    friend Accessor;

public:
    explicit TwistSwingJointsEvaluator(Vector<TwistSwingSetup>&& setups_,
                                       TRotationAdapter&& rotationAdapter_,
                                       JointsOutputInstance::Factory instanceFactory_,
                                       MemoryResource* memRes);

    JointsOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint32_t getJointDeltaValueCountForLOD(std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs, JointsOutputInstance* outputs, std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputs,
                   JointsOutputInstance* outputs,
                   std::uint16_t lod,
                   std::uint16_t jointGroupIndex) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    Vector<TwistSwingSetup> setups;
    TRotationAdapter rotationAdapter;
    JointsOutputInstance::Factory instanceFactory;
};

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, TRotationAdapter>::TwistSwingJointsEvaluator(
    Vector<TwistSwingSetup>&& setups_,
    TRotationAdapter&& rotationAdapter_,
    JointsOutputInstance::Factory instanceFactory_,
    MemoryResource* /*unused*/) :
    setups{std::move(setups_)},
    rotationAdapter{std::move(rotationAdapter_)},
    instanceFactory{std::move(instanceFactory_)} {
}

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
JointsOutputInstance::Pointer
TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, TRotationAdapter>::createInstance(MemoryResource* instanceMemRes) const {

    return instanceFactory(instanceMemRes);
}

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
std::uint32_t TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, TRotationAdapter>::getJointDeltaValueCountForLOD(
    std::uint16_t /*unused*/) const {
    return {};
}

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
void TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, TRotationAdapter>::calculate(ControlsInputInstance* inputs,
                                                                                        JointsOutputInstance* outputs,
                                                                                        std::uint16_t /*unused*/) const {
    const auto inputBuffer = inputs->getInputBuffer();
    auto outputBuffer = outputs->getOutputBuffer();

    const tdm::fquat identity;

    static constexpr decltype(separateTwistComponentByAxisX)* separateTwistComponent[] = {separateTwistComponentByAxisX,
                                                                                          separateTwistComponentByAxisY,
                                                                                          separateTwistComponentByAxisZ};

    for (const auto& setup : setups) {
        tdm::fquat invTwist;

        // A TwistSwingSetup instance will only have both twist and swing portions populated if both of them
        // rely on the same input control indices. Otherwise either the swing or the twist portion will be populated only.
        if (!setup.swingInputIndices.empty()) {
            const tdm::fquat swingInput{inputBuffer[setup.swingInputIndices[0]],
                                        inputBuffer[setup.swingInputIndices[1]],
                                        inputBuffer[setup.swingInputIndices[2]],
                                        inputBuffer[setup.swingInputIndices[3]]};
            const tdm::fquat twist = separateTwistComponent[static_cast<std::size_t>(setup.swingTwistAxis)](swingInput);
            invTwist = tdm::inverse(twist);
            const tdm::fquat swing = swingInput * invTwist;
            const tdm::fquat invSwing = tdm::inverse(swing);

            for (std::size_t si = {}; si < setup.swingBlendWeights.size(); ++si) {
                const float swingBlendWeight = setup.swingBlendWeights[si];
                const tdm::fquat invSwingFraction = tdm::slerp(invSwing, identity, swingBlendWeight);
                const tdm::fquat swingOutput = invTwist * invSwingFraction;
                const std::uint16_t* outputIndices = &setup.swingOutputIndices[si * 4ul];
                const float outbuf[] = {swingOutput.x, swingOutput.y, swingOutput.z, swingOutput.w};
                rotationAdapter.forward(outbuf, 1ul, 1ul, outputIndices, outputBuffer);
            }
        }

        if (!setup.twistInputIndices.empty() && setup.swingInputIndices.empty()) {
            // 1. When swing and twist input indices are the same, both twist and swing setups are merged
            // into a single setup instance. In this case, twist needs no recalculation, since both
            // twist and swing rely on the same input joint.
            // 2. When swing and twist input indices are different, they will be on separate setup instances,
            // in which case the swing portion will remain empty (when twist data exists), so this part needs
            // to be executed. This is the "fromEnd" case calculation.
            const tdm::fquat twistInput{inputBuffer[setup.twistInputIndices[0]],
                                        inputBuffer[setup.twistInputIndices[1]],
                                        inputBuffer[setup.twistInputIndices[2]],
                                        inputBuffer[setup.twistInputIndices[3]]};
            const tdm::fquat twist = separateTwistComponent[static_cast<std::size_t>(setup.twistTwistAxis)](twistInput);
            invTwist = twist;  // No actual inversion is needed here
        }

        assert((setup.twistBlendWeights.size() * 4) == setup.twistOutputIndices.size());
        for (std::size_t ti = {}; ti < setup.twistBlendWeights.size(); ++ti) {
            const float twistBlendWeight = setup.twistBlendWeights[ti];
            const tdm::fquat twistOutput = tdm::slerp(invTwist, identity, twistBlendWeight);
            const float outbuf[] = {twistOutput.x, twistOutput.y, twistOutput.z, twistOutput.w};
            const std::uint16_t* outputIndices = &setup.twistOutputIndices[ti * 4ul];
            rotationAdapter.forward(outbuf, 1ul, 1ul, outputIndices, outputBuffer);
        }
    }
}

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
void TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, TRotationAdapter>::calculate(ControlsInputInstance* /*unused*/,
                                                                                        JointsOutputInstance* /*unused*/,
                                                                                        std::uint16_t /*unused*/,
                                                                                        std::uint16_t /*unused*/) const {
}

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
void TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, TRotationAdapter>::load(
    terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(setups);
}

template<typename TValue, typename TFVec256, typename TFVec128, class TRotationAdapter>
void TwistSwingJointsEvaluator<TValue, TFVec256, TFVec128, TRotationAdapter>::save(
    terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(setups);
}

}  // namespace rl4
