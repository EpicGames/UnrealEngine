// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

namespace rl4 {

struct TwistSwingSetup {
    dna::TwistAxis twistTwistAxis;
    Vector<float> twistBlendWeights;
    Vector<std::uint16_t> twistOutputIndices;
    Vector<std::uint16_t> twistInputIndices;
    dna::TwistAxis swingTwistAxis;
    Vector<float> swingBlendWeights;
    Vector<std::uint16_t> swingOutputIndices;
    Vector<std::uint16_t> swingInputIndices;

    explicit TwistSwingSetup(MemoryResource* memRes) :
        twistTwistAxis{},
        twistBlendWeights{memRes},
        twistOutputIndices{memRes},
        twistInputIndices{memRes},
        swingTwistAxis{},
        swingBlendWeights{memRes},
        swingOutputIndices{memRes},
        swingInputIndices{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(twistTwistAxis);
        archive(twistBlendWeights);
        archive(twistOutputIndices);
        archive(twistInputIndices);
        archive(swingTwistAxis);
        archive(swingBlendWeights);
        archive(swingOutputIndices);
        archive(swingInputIndices);
    }
};

}  // namespace rl4
