// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/joints/cpu/utils/JointGroupOptimizer.h"

#include "riglogic/joints/JointBehaviorFilter.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cmath>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace {

struct ColumnMatch {
    bool found;
    std::uint32_t column;
};

}  // namespace

void JointGroupOptimizer::defragment(const JointBehaviorFilter& source,
                                     std::uint16_t jointGroupIndex,
                                     Vector<float>& values,
                                     Vector<std::uint16_t>& inputIndices,
                                     Vector<std::uint16_t>& outputIndices,
                                     ArrayView<LODRegion> lodRegions,
                                     float translationPruningThreshold,
                                     float rotationPruningThreshold,
                                     float scalePruningThreshold) {

    auto getPruningThreshold = [&](std::uint16_t outputIndex) {
        const std::uint16_t attrIndex = (outputIndex % 9u) / 3u;
        const float thresholds[] = {translationPruningThreshold, rotationPruningThreshold, scalePruningThreshold};
        return thresholds[attrIndex];
    };

    auto hasNonZeroDeltasInColumnUntilRow = [&](std::uint32_t col, std::uint32_t colCount, std::uint32_t rowCount) {
        for (std::uint32_t ri = {}; ri < rowCount; ++ri) {
            if (std::fabs(values[ri * colCount + col]) > getPruningThreshold(outputIndices[ri])) {
                return true;
            }
        }
        return false;
    };

    auto hasNonZeroDeltasInRowUntilColumn = [&](std::uint32_t row, std::uint32_t /*unused*/, std::uint32_t colCount) {
        const float threshold = getPruningThreshold(outputIndices[row]);
        for (std::uint32_t ci = {}; ci < colCount; ++ci) {
            if (std::fabs(values[row * colCount + ci]) > threshold) {
                return true;
            }
        }
        return false;
    };

    auto findNextColumnInReverseOrder =
        [&hasNonZeroDeltasInColumnUntilRow](std::uint32_t startColumn, std::uint32_t columnCount, std::uint32_t rowCountForLOD) {
            for (std::uint32_t column = startColumn; column != static_cast<std::uint32_t>(-1); --column) {
                if (!hasNonZeroDeltasInColumnUntilRow(column, columnCount, rowCountForLOD)) {
                    return ColumnMatch{true, column};
                }
            }
            return ColumnMatch{};
        };

    auto swapColumns = [&values, &inputIndices](std::uint32_t col1, std::uint32_t col2) {
        const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        const auto rowCount = static_cast<std::uint32_t>(values.size() / colCount);
        for (std::uint32_t ri = {}; ri < rowCount; ++ri) {
            std::swap(values[ri * colCount + col1], values[ri * colCount + col2]);
        }
        std::swap(inputIndices[col1], inputIndices[col2]);
    };

    auto deleteColumn = [&values, &inputIndices](std::uint32_t column) {
        const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        const auto rowCount = static_cast<std::uint32_t>(values.size() / colCount);
        std::uint32_t dstIndex = {};
        for (std::uint32_t ri = {}; ri < rowCount; ++ri) {
            for (std::uint32_t ci = {}; ci < colCount; ++ci) {
                if (ci != column) {
                    const auto i = ri * colCount + ci;
                    values[dstIndex++] = values[i];
                }
            }
        }
        values.resize(values.size() - rowCount);
        extd::filter(inputIndices, [column](std::uint16_t /*unused*/, std::size_t index) { return index != column; });
    };

    auto deleteRow = [&values, &inputIndices, &outputIndices](std::uint32_t row) {
        const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        for (std::uint32_t i = (row * colCount); i < (values.size() - colCount); ++i) {
            values[i] = values[i + colCount];
        }
        values.resize(values.size() - colCount);
        extd::filter(outputIndices, [row](std::uint16_t /*unused*/, std::size_t index) { return index != row; });
    };

    auto purgeEmptyColumns = [&]() {
        auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        const auto rowCount = static_cast<std::uint32_t>(values.size() / std::max(colCount, 1u));
        for (std::uint32_t col = (colCount - 1u); col != static_cast<std::uint32_t>(-1); --col) {
            if (!hasNonZeroDeltasInColumnUntilRow(col, colCount, rowCount)) {
                deleteColumn(col);
                --colCount;
            }
        }
    };

    auto purgeEmptyRows = [&]() {
        const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
        auto rowCount = static_cast<std::uint32_t>(values.size() / std::max(colCount, 1u));
        for (std::uint32_t row = (rowCount - 1u); row != static_cast<std::uint32_t>(-1); --row) {
            if (!hasNonZeroDeltasInRowUntilColumn(row, rowCount, colCount)) {
                deleteRow(row);
                --rowCount;
                for (std::uint16_t lod = {}; lod < static_cast<std::uint16_t>(lodRegions.size()); ++lod) {
                    if (row < lodRegions[lod].outputLODs.size) {
                        lodRegions[lod].outputLODs.size -= 1u;
                    }
                }
            }
        }
    };

    const auto lodCount = static_cast<std::uint32_t>(source.getLODCount());
    for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
        const auto rowCountForLOD = source.getRowCountForLOD(jointGroupIndex, lod);
        lodRegions[lod].outputLODs.size = rowCountForLOD;
    }

    purgeEmptyRows();
    purgeEmptyColumns();

    const auto colCount = static_cast<std::uint32_t>(inputIndices.size());
    constexpr auto endColumn = static_cast<std::uint32_t>(-1);
    for (std::uint32_t startColumn = (colCount - 1u), lod = {}; (startColumn != endColumn) && (lod < lodCount);) {
        const auto rowCountForLOD = lodRegions[lod].outputLODs.size;
        const auto match = findNextColumnInReverseOrder(startColumn, colCount, rowCountForLOD);
        if (match.found) {
            if (match.column != startColumn) {
                swapColumns(startColumn, match.column);
            }
            --startColumn;
        } else {
            lodRegions[lod].inputLODs = ColumnLOD{startColumn + 1u};
            ++lod;
        }
    }
}

}  // namespace rl4
