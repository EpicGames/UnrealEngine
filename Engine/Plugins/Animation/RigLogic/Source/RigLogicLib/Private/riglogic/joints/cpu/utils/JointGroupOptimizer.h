// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/utils/LODRegion.h"

namespace rl4 {

class JointBehaviorFilter;

struct JointGroupOptimizer {
    static void defragment(const JointBehaviorFilter& source,
                           std::uint16_t jointGroupIndex,
                           Vector<float>& values,
                           Vector<std::uint16_t>& inputIndices,
                           Vector<std::uint16_t>& outputIndices,
                           ArrayView<LODRegion> lodRegions,
                           float translationPruningThreshold,
                           float rotationPruningThreshold,
                           float scalePruningThreshold);
};

}  // namespace rl4
