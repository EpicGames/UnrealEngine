// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/types/PaddedBlockView.h"

#include <cstdint>

namespace rl4 {

struct ColumnLOD {
    std::uint32_t size;
    std::uint32_t sizeAlignedTo4;
    std::uint32_t sizeAlignedTo8;

    ColumnLOD() = default;
    explicit ColumnLOD(std::uint32_t size_) :
        size{size_},
        sizeAlignedTo4{size - (size % 4u)},
        sizeAlignedTo8{size - (size % 8u)} {
    }

    ColumnLOD(std::uint32_t size_, std::uint32_t sizeAlignedTo4_, std::uint32_t sizeAlignedTo8_) :
        size{size_},
        sizeAlignedTo4{sizeAlignedTo4_},
        sizeAlignedTo8{sizeAlignedTo8_} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(size, sizeAlignedTo4, sizeAlignedTo8);
    }
};

using RowLOD = PaddedBlockView;

struct LODRegion {
    ColumnLOD inputLODs;
    RowLOD outputLODs;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(inputLODs, outputLODs);
    }
};

}  // namespace rl4
