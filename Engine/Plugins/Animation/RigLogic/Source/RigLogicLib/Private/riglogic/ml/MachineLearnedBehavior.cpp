// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

#include "riglogic/ml/MachineLearnedBehavior.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"

#include <cassert>
#include <cstddef>

namespace rl4 {

MachineLearnedBehavior::MachineLearnedBehavior(MachineLearnedBehaviorEvaluator::Pointer evaluator_, MemoryResource* memRes) :
    evaluator{std::move(evaluator_)},
    meshRegionCounts{memRes} {
}

MachineLearnedBehavior::MachineLearnedBehavior(MachineLearnedBehaviorEvaluator::Pointer evaluator_,
                                               Vector<std::uint16_t>&& meshRegionCounts_) :
    evaluator{std::move(evaluator_)},
    meshRegionCounts{std::move(meshRegionCounts_)} {
}

MachineLearnedBehaviorOutputInstance::Pointer MachineLearnedBehavior::createInstance(MemoryResource* instanceMemRes) const {
    return evaluator->createInstance(instanceMemRes);
}

void MachineLearnedBehavior::calculate(ControlsInputInstance* inputs,
                                       MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                                       std::uint16_t lod) const {
    evaluator->calculate(inputs, intermediateOutputs, lod);
}

void MachineLearnedBehavior::calculate(ControlsInputInstance* inputs,
                                       MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                                       std::uint16_t lod,
                                       std::uint16_t mlTypeIndex,
                                       std::uint16_t mlOperationSetIndex,
                                       std::uint16_t mlOperationIndex) const {
    evaluator->calculate(inputs, intermediateOutputs, lod, mlTypeIndex, mlOperationSetIndex, mlOperationIndex);
}

std::uint16_t MachineLearnedBehavior::getMeshCount() const {
    return static_cast<std::uint16_t>(meshRegionCounts.size());
}

std::uint16_t MachineLearnedBehavior::getMeshRegionCount(std::uint16_t meshIndex) const {
    assert(meshIndex < meshRegionCounts.size());
    return meshRegionCounts[meshIndex];
}

std::uint16_t MachineLearnedBehavior::getMLOperationSetCount(std::uint16_t mlTypeIndex) const {
    return evaluator->getMLOperationSetCount(mlTypeIndex);
}

std::uint16_t MachineLearnedBehavior::getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const {
    return evaluator->getMLOperationCount(mlTypeIndex, mlOperationSetIndex);
}

ConstArrayView<std::uint16_t> MachineLearnedBehavior::getMLOperationIndicesForLOD(std::uint16_t lod,
                                                                                  std::uint16_t mlTypeIndex,
                                                                                  std::uint16_t mlOperationSetIndex) const {
    return evaluator->getMLOperationIndicesForLOD(lod, mlTypeIndex, mlOperationSetIndex);
}

}  // namespace rl4
