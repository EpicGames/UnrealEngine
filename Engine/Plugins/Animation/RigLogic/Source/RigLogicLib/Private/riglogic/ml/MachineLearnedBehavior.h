// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/MachineLearnedBehaviorEvaluator.h"
#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <cstddef>

namespace rl4 {

class ControlsInputInstance;

class MachineLearnedBehavior {
public:
    using Pointer = UniqueInstance<MachineLearnedBehavior>::PointerType;

public:
    MachineLearnedBehavior(MachineLearnedBehaviorEvaluator::Pointer evaluator_, MemoryResource* memRes);
    MachineLearnedBehavior(MachineLearnedBehaviorEvaluator::Pointer evaluator_, Vector<std::uint16_t>&& meshRegionCounts_);

    MachineLearnedBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const;
    void calculate(ControlsInputInstance* inputs,
                   MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                   std::uint16_t lod) const;
    void calculate(ControlsInputInstance* inputs,
                   MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                   std::uint16_t lod,
                   std::uint16_t mlTypeIndex,
                   std::uint16_t mlOperationSetIndex,
                   std::uint16_t mlOperationIndex) const;

    template<class Archive>
    void load(Archive& archive) {
        evaluator->load(archive);
        archive >> meshRegionCounts;
    }

    template<class Archive>
    void save(Archive& archive) {
        evaluator->save(archive);
        archive << meshRegionCounts;
    }

    std::uint16_t getMeshCount() const;
    std::uint16_t getMeshRegionCount(std::uint16_t meshIndex) const;
    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const;
    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const;
    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t lod,
                                                              std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex) const;

private:
    MachineLearnedBehaviorEvaluator::Pointer evaluator;
    Vector<std::uint16_t> meshRegionCounts;
};

}  // namespace rl4
