// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"

namespace rl4 {

class ControlsInputInstance;

class MachineLearnedBehaviorEvaluator {
public:
    using Pointer = UniqueInstance<MachineLearnedBehaviorEvaluator>::PointerType;

protected:
    virtual ~MachineLearnedBehaviorEvaluator();

public:
    virtual MachineLearnedBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const = 0;
    virtual std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const = 0;
    virtual std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const = 0;
    virtual ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t lod,
                                                                      std::uint16_t mlTypeIndex,
                                                                      std::uint16_t mlOperationSetIndex) const = 0;
    virtual void calculate(ControlsInputInstance* inputs,
                           MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                           std::uint16_t lod) const = 0;
    virtual void calculate(ControlsInputInstance* inputs,
                           MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                           std::uint16_t lod,
                           std::uint16_t mlTypeIndex,
                           std::uint16_t mlOperationSetIndex,
                           std::uint16_t mlOperationIndex) const = 0;
    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive) = 0;
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) = 0;
};

}  // namespace rl4
