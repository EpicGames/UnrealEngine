// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/ml/MachineLearnedBehaviorFactory.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/MachineLearnedBehaviorEvaluator.h"
#include "riglogic/ml/MachineLearnedBehaviorNullEvaluator.h"
#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorFactory.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/utils/Extd.h"

#include <cstdint>

namespace rl4 {

MachineLearnedBehaviorEvaluator::Pointer createMLEvaluator(const Configuration& config,
                                                           RigMetadata* meta,
                                                           const dna::Reader* reader,
                                                           MemoryResource* memRes) {
    return ml::cpu::Factory::create(config, meta, reader, memRes);
}

MachineLearnedBehavior::Pointer MachineLearnedBehaviorFactory::create(const Configuration& config,
                                                                      RigMetadata* meta,
                                                                      const dna::Reader* reader,
                                                                      MemoryResource* memRes) {
    auto moduleFactory = UniqueInstance<MachineLearnedBehavior>::with(memRes);
    if (!config.loadMachineLearnedBehavior || (meta->lodCount == 0u) || (meta->mlTypeCount == 0u)) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        auto evaluator =
            UniqueInstance<MachineLearnedBehaviorNullEvaluator, MachineLearnedBehaviorEvaluator>::with(memRes).create();
        return moduleFactory.create(std::move(evaluator), memRes);
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);

    Vector<std::uint16_t> meshRegionCount{memRes};
    meshRegionCount.resize(reader->getMeshCount());
    for (std::uint16_t meshIdx = {}; meshIdx < meshRegionCount.size(); ++meshIdx) {
        meshRegionCount[meshIdx] = reader->getMeshRegionCount(meshIdx);
    }

    return moduleFactory.create(createMLEvaluator(config, meta, reader, memRes), std::move(meshRegionCount));
}

MachineLearnedBehavior::Pointer MachineLearnedBehaviorFactory::create(const Configuration& config,
                                                                      RigMetadata* meta,
                                                                      MemoryResource* memRes) {
    auto moduleFactory = UniqueInstance<MachineLearnedBehavior>::with(memRes);
    const EvaluatorType type = meta->popFrontEvaluator();

    if (type == EvaluatorType::Null) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        auto evaluator =
            UniqueInstance<MachineLearnedBehaviorNullEvaluator, MachineLearnedBehaviorEvaluator>::with(memRes).create();
        return moduleFactory.create(std::move(evaluator), memRes);
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    return moduleFactory.create(createMLEvaluator(config, meta, nullptr, memRes), memRes);
}

}  // namespace rl4
