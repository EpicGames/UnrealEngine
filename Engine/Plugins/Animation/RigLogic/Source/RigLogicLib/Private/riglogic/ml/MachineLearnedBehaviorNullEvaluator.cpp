// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/ml/MachineLearnedBehaviorNullEvaluator.h"

#include "riglogic/ml/MachineLearnedBehaviorNullOutputInstance.h"

namespace rl4 {

MachineLearnedBehaviorOutputInstance::Pointer MachineLearnedBehaviorNullEvaluator::createInstance(
    MemoryResource* instanceMemRes) const {
    return UniqueInstance<MachineLearnedBehaviorNullOutputInstance, MachineLearnedBehaviorOutputInstance>::with(instanceMemRes)
        .create();
}

std::uint16_t MachineLearnedBehaviorNullEvaluator::getMLOperationSetCount(std::uint16_t /*unused*/) const {
    return {};
}

std::uint16_t MachineLearnedBehaviorNullEvaluator::getMLOperationCount(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const {
    return {};
}

ConstArrayView<std::uint16_t> MachineLearnedBehaviorNullEvaluator::getMLOperationIndicesForLOD(std::uint16_t /*unused*/,
                                                                                               std::uint16_t /*unused*/,
                                                                                               std::uint16_t /*unused*/) const {
    return {};
}

void MachineLearnedBehaviorNullEvaluator::calculate(ControlsInputInstance* /*unused*/,
                                                    MachineLearnedBehaviorOutputInstance* /*unused*/,
                                                    std::uint16_t /*unused*/) const {
}

void MachineLearnedBehaviorNullEvaluator::calculate(ControlsInputInstance* /*unused*/,
                                                    MachineLearnedBehaviorOutputInstance* /*unused*/,
                                                    std::uint16_t /*unused*/,
                                                    std::uint16_t /*unused*/,
                                                    std::uint16_t /*unused*/,
                                                    std::uint16_t /*unused*/) const {
}

void MachineLearnedBehaviorNullEvaluator::load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) {
}

void MachineLearnedBehaviorNullEvaluator::save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) {
}

}  // namespace rl4
