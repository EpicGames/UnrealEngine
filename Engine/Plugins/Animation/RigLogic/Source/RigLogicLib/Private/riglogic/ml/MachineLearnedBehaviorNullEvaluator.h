// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/MachineLearnedBehaviorEvaluator.h"
#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"

namespace rl4 {

class ControlsInputInstance;

class MachineLearnedBehaviorNullEvaluator : public MachineLearnedBehaviorEvaluator {
public:
    MachineLearnedBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint16_t getMLOperationSetCount(std::uint16_t /*unused*/) const override;
    std::uint16_t getMLOperationCount(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override;
    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t /*unused*/,
                                                              std::uint16_t /*unused*/,
                                                              std::uint16_t /*unused*/) const override;
    void calculate(ControlsInputInstance* /*unused*/,
                   MachineLearnedBehaviorOutputInstance* /*unused*/,
                   std::uint16_t /*unused*/) const override;
    void calculate(ControlsInputInstance* /*unused*/,
                   MachineLearnedBehaviorOutputInstance* /*unused*/,
                   std::uint16_t /*unused*/,
                   std::uint16_t /*unused*/,
                   std::uint16_t /*unused*/,
                   std::uint16_t /*unused*/) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) override;
};

}  // namespace rl4
