// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/ml/MachineLearnedBehaviorNullOutputInstance.h"

namespace rl4 {

ArrayView<float> MachineLearnedBehaviorNullOutputInstance::getMaskBuffer() {
    return {};
}

ConstArrayView<float> MachineLearnedBehaviorNullOutputInstance::getMaskBuffer() const {
    return {};
}

std::uint16_t MachineLearnedBehaviorNullOutputInstance::getMLOperationSetCount(std::uint16_t /*unused*/) const {
    return {};
}

std::uint16_t MachineLearnedBehaviorNullOutputInstance::getMLOperationCount(std::uint16_t /*unused*/,
                                                                            std::uint16_t /*unused*/) const {
    return {};
}

}  // namespace rl4
