// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"

namespace rl4 {

class MachineLearnedBehaviorNullOutputInstance : public MachineLearnedBehaviorOutputInstance {
public:
    ArrayView<float> getMaskBuffer() override;
    ConstArrayView<float> getMaskBuffer() const override;
    std::uint16_t getMLOperationSetCount(std::uint16_t /*unused*/) const override;
    std::uint16_t getMLOperationCount(std::uint16_t /*unused*/, std::uint16_t /*unused*/) const override;
};

}  // namespace rl4
