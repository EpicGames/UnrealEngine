// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

namespace rl4 {

class MachineLearnedBehaviorOutputInstance {
public:
    using Pointer = UniqueInstance<MachineLearnedBehaviorOutputInstance>::PointerType;

public:
    virtual ~MachineLearnedBehaviorOutputInstance();
    virtual ArrayView<float> getMaskBuffer() = 0;
    virtual ConstArrayView<float> getMaskBuffer() const = 0;
    virtual std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const = 0;
    virtual std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const = 0;
};

}  // namespace rl4
