// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorEvaluator.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorOutputInstance.h"
#include "riglogic/ml/cpu/Operation.h"
#include "riglogic/system/simd/Utils.h"
#include "riglogic/types/LODSpec.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace ml {

namespace cpu {

Evaluator::Evaluator(const Configuration& config_,
                     Matrix<LODSpec<std::uint16_t>>&& lods_,
                     Matrix<OperationSet::Pointer>&& mlOperations_,
                     Vector<Matrix<std::uint16_t>>&& bufferSizes_,
                     std::uint32_t meshRegionCount_,
                     OutputInstance::Factory instanceFactory_) :
    config{&config_},
    lods{std::move(lods_)},
    mlOperations{std::move(mlOperations_)},
    bufferSizes{std::move(bufferSizes_)},
    meshRegionCount{meshRegionCount_},
    instanceFactory{instanceFactory_} {
}

MachineLearnedBehaviorOutputInstance::Pointer Evaluator::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(bufferSizes, meshRegionCount, instanceMemRes);
}

std::uint16_t Evaluator::getMLOperationSetCount(std::uint16_t mlTypeIndex) const {
    return static_cast<std::uint16_t>(mlOperations[mlTypeIndex].size());
}

std::uint16_t Evaluator::getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const {
    assert(mlTypeIndex < lods.size());
    assert(mlOperationSetIndex < lods[mlTypeIndex].size());
    return static_cast<std::uint16_t>(lods[mlTypeIndex][mlOperationSetIndex].count);
}

ConstArrayView<std::uint16_t> Evaluator::getMLOperationIndicesForLOD(std::uint16_t lod,
                                                                     std::uint16_t mlTypeIndex,
                                                                     std::uint16_t mlOperationSetIndex) const {
    assert(mlTypeIndex < lods.size());
    assert(mlOperationSetIndex < lods[mlTypeIndex].size());
    assert(lod < lods[mlTypeIndex][mlOperationSetIndex].indicesPerLOD.size());
    return lods[mlTypeIndex][mlOperationSetIndex].indicesPerLOD[lod];
}

void Evaluator::calculate(ControlsInputInstance* inputs,
                          MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                          std::uint16_t lod,
                          std::uint16_t mlTypeIndex,
                          std::uint16_t mlOperationSetIndex,
                          std::uint16_t mlOperationIndex) const {
    auto* out = static_cast<OutputInstance*>(intermediateOutputs);
    const auto masks = out->getMaskBuffer();
    auto inputBuffer = inputs->getInputBuffer();
    const auto allWorkPtrs = out->getWorkBufferPtrs(mlTypeIndex);
    const auto allWorkHalfSizes = out->getWorkBufferHalfSizes(mlTypeIndex);
    const auto setOffsets = out->getWorkBufferOffsetsPerOperationSet(mlTypeIndex);
    const auto opSetWorkBufferOffset = setOffsets[mlOperationSetIndex];
    mlOperations[mlTypeIndex][mlOperationSetIndex]->execute(ConstArrayView<std::uint16_t>(&mlOperationIndex, 1ul),
                                                            lod,
                                                            masks,
                                                            inputBuffer,
                                                            opSetWorkBufferOffset,
                                                            allWorkPtrs,
                                                            allWorkHalfSizes,
                                                            setOffsets);
}

void Evaluator::calculate(ControlsInputInstance* inputs,
                          MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                          std::uint16_t lod) const {
    auto* out = static_cast<OutputInstance*>(intermediateOutputs);
    const auto masks = out->getMaskBuffer();
    auto inputBuffer = inputs->getInputBuffer();
    for (std::uint16_t typeIdx = {}; typeIdx < static_cast<std::uint16_t>(mlOperations.size()); ++typeIdx) {
        const auto& lodsPerType = lods[typeIdx];
        const auto& opsPerType = mlOperations[typeIdx];
        const auto allWorkPtrs = out->getWorkBufferPtrs(typeIdx);
        const auto allWorkHalfSizes = out->getWorkBufferHalfSizes(typeIdx);
        const auto setOffsets = out->getWorkBufferOffsetsPerOperationSet(typeIdx);
        const auto opSetCount = static_cast<std::uint16_t>(mlOperations[typeIdx].size());
        for (std::uint16_t opSetIdx = {}; opSetIdx < opSetCount; ++opSetIdx) {
            const auto& activeOpIndices = lodsPerType[opSetIdx].indicesPerLOD[lod];
            const auto opSetWorkBufferOffset = setOffsets[opSetIdx];
            opsPerType[opSetIdx]->execute(activeOpIndices,
                                          lod,
                                          masks,
                                          inputBuffer,
                                          opSetWorkBufferOffset,
                                          allWorkPtrs,
                                          allWorkHalfSizes,
                                          setOffsets);
        }
    }
}

void Evaluator::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(lods);

    auto memRes = mlOperations.get_allocator().getMemoryResource();

    std::uint32_t mlTypeCount = {};
    archive(mlTypeCount);
    mlOperations.resize(mlTypeCount);

    using BasePointer = UniqueInstance<OperationSet>::PointerType;
    RuntimeTemplateInstantiator instantiator{config};

    for (std::uint32_t typeIdx = {}; typeIdx < mlTypeCount; ++typeIdx) {
        std::uint32_t mlSetCount = {};
        archive(mlSetCount);
        mlOperations[typeIdx].resize(mlSetCount);

        for (std::uint32_t opSetIdx = {}; opSetIdx < mlSetCount; ++opSetIdx) {
            OperationSetType setType = {};
            archive(setType);

            OperationSetData opSetData{memRes};
            opSetData.type = setType;
            auto opSet = instantiator.invoke<OperationSetFactory, BasePointer>(std::move(opSetData), memRes);
            opSet->load(archive);
            mlOperations[typeIdx][opSetIdx] = std::move(opSet);
        }
    }

    archive(bufferSizes);
    archive(meshRegionCount);
}

void Evaluator::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(lods);

    archive(static_cast<std::uint32_t>(mlOperations.size()));
    for (auto& mlTypeSets : mlOperations) {
        archive(static_cast<std::uint32_t>(mlTypeSets.size()));
        for (auto& opSet : mlTypeSets) {
            OperationSetType setType = opSet->getType();
            archive(setType);
            opSet->save(archive);
        }
    }

    archive(bufferSizes);
    archive(meshRegionCount);
}

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
