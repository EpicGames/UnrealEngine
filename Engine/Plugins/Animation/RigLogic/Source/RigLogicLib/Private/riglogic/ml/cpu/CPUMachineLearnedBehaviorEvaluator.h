// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/ml/MachineLearnedBehaviorEvaluator.h"
#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorOutputInstance.h"
#include "riglogic/ml/cpu/Operation.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/types/LODSpec.h"

#include <cstddef>

namespace rl4 {

namespace ml {

namespace cpu {

class Evaluator : public MachineLearnedBehaviorEvaluator {
public:
    struct Accessor;
    friend Accessor;

public:
    Evaluator(const Configuration& config_,
              Matrix<LODSpec<std::uint16_t>>&& lods_,
              Matrix<OperationSet::Pointer>&& mlOperations_,
              Vector<Matrix<std::uint16_t>>&& bufferSizes_,
              std::uint32_t meshRegionCount_,
              OutputInstance::Factory instanceFactory_);
    MachineLearnedBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override;
    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override;
    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t lod,
                                                              std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex) const override;
    void calculate(ControlsInputInstance* inputs,
                   MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                   std::uint16_t lod,
                   std::uint16_t mlTypeIndex,
                   std::uint16_t mlOperationSetIndex,
                   std::uint16_t mlOperationIndex) const override;
    void calculate(ControlsInputInstance* inputs,
                   MachineLearnedBehaviorOutputInstance* intermediateOutputs,
                   std::uint16_t lod) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    const Configuration* config;
    Matrix<LODSpec<std::uint16_t>> lods;
    Matrix<OperationSet::Pointer> mlOperations;
    Vector<Matrix<std::uint16_t>> bufferSizes;
    std::uint32_t meshRegionCount;
    OutputInstance::Factory instanceFactory;
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
