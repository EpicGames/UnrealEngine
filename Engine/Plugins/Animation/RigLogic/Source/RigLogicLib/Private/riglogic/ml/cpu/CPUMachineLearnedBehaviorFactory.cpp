// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorFactory.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/MachineLearnedBehaviorEvaluator.h"
#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorEvaluator.h"
#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorOutputInstance.h"
#include "riglogic/ml/cpu/NeuralNet.h"
#include "riglogic/ml/cpu/Operation.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Utils.h"
#include "riglogic/types/LODSpec.h"
#include "riglogic/types/bpcm/Optimizer.h"
#include "riglogic/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <limits>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace ml {

namespace cpu {

static void remapSetIndices(Vector<Dependency>& deps, ConstArrayView<std::uint16_t> deletedSets) {
    for (std::size_t i = deps.size() - 1; i != static_cast<std::size_t>(-1); --i) {
        if (std::find(deletedSets.begin(), deletedSets.end(), deps[i].opSetIdx) != deletedSets.end()) {
            deps.erase(extd::advanced(deps.begin(), i));
        } else {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ < 7)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wattributes"
#endif
            const auto delta =
                std::count_if(deletedSets.begin(), deletedSets.end(), [&](std::uint16_t d) { return d < deps[i].opSetIdx; });
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ < 7)
    #pragma GCC diagnostic pop
#endif
            deps[i].opSetIdx = static_cast<std::uint16_t>(deps[i].opSetIdx - delta);
        }
    }
}

static Vector<std::uint16_t> getOperationInputIndices(const dna::Reader* reader,
                                                      std::uint16_t mlTypeIndex,
                                                      std::uint16_t mlOperationSetIndex,
                                                      std::uint16_t mlOperationIndex,
                                                      MemoryResource* memRes) {
    Vector<std::uint16_t> inputIndices{memRes};
    const auto dependencyOperationSetIndices =
        reader->getMLOperationDependencyOperationSetIndices(mlTypeIndex, mlOperationSetIndex, mlOperationIndex);
    const auto dependencyOperationIndices =
        reader->getMLOperationDependencyOperationIndices(mlTypeIndex, mlOperationSetIndex, mlOperationIndex);
    assert(dependencyOperationSetIndices.size() == dependencyOperationIndices.size());
    for (std::size_t i = {}; i < dependencyOperationIndices.size(); ++i) {
        const auto type =
            reader->getMLOperationType(mlTypeIndex, dependencyOperationSetIndices[i], dependencyOperationIndices[i]);
        if (type == dna::MachineLearnedBehaviorOperationType::Gather) {
            const auto params =
                reader->getMLOperationParameters(mlTypeIndex, dependencyOperationSetIndices[i], dependencyOperationIndices[i]);
            inputIndices.reserve(inputIndices.size() + params.size());
            for (auto inputIndex : params) {
                inputIndices.push_back(static_cast<std::uint16_t>(inputIndex));
            }
        }
    }
    return inputIndices;
}

static Vector<std::uint16_t> getOperationOutputIndices(const dna::Reader* reader,
                                                       std::uint16_t mlTypeIndex,
                                                       std::uint16_t mlOperationSetIndex,
                                                       std::uint16_t mlOperationIndex,
                                                       MemoryResource* memRes) {
    Vector<std::uint16_t> outputIndices{memRes};
    for (std::uint16_t depOSI = static_cast<std::uint16_t>(mlOperationSetIndex + 1);
         depOSI < reader->getMLOperationSetCount(mlTypeIndex);
         ++depOSI) {
        for (std::uint16_t depOI = {}; depOI < reader->getMLOperationCount(mlTypeIndex, depOSI); ++depOI) {
            const auto type = reader->getMLOperationType(mlTypeIndex, depOSI, depOI);
            if (type == dna::MachineLearnedBehaviorOperationType::Scatter) {
                const auto dependencyOperationSetIndices =
                    reader->getMLOperationDependencyOperationSetIndices(mlTypeIndex, depOSI, depOI);
                const auto dependencyOperationIndices =
                    reader->getMLOperationDependencyOperationIndices(mlTypeIndex, depOSI, depOI);
                assert(dependencyOperationSetIndices.size() == dependencyOperationIndices.size());
                for (std::size_t i = {}; i < dependencyOperationIndices.size(); ++i) {
                    const auto opSetIdx = dependencyOperationSetIndices[i];
                    const auto opIdx = dependencyOperationIndices[i];
                    if ((opSetIdx == mlOperationSetIndex) && (opIdx == mlOperationIndex)) {
                        const auto params = reader->getMLOperationParameters(mlTypeIndex, depOSI, depOI);
                        outputIndices.reserve(outputIndices.size() + params.size());
                        for (auto outputIndex : params) {
                            outputIndices.push_back(static_cast<std::uint16_t>(outputIndex));
                        }
                    }
                }
            }
        }
    }
    return outputIndices;
}

static Vector<float> getDefaultValues(const dna::Reader* reader,
                                      ConstArrayView<std::uint16_t> outputControlIndices,
                                      std::uint16_t outputCount,
                                      MemoryResource* memRes) {
    Vector<float> defaultValues{outputCount, 0.0f, memRes};
    const auto mlJointsInputIndices = reader->getMLJointsInputIndices();
    const auto mlJointsOutputIndices = reader->getMLJointsOutputIndices();

    auto getTransformType = [reader](dna::MachineLearnedBehaviorParameterKey key, std::uint16_t defaultValue) {
        const auto paramKeys = reader->getMLJointsParameterKeys();
        const auto paramValues = reader->getMLJointsParameterValues();
        for (std::size_t i = {}; i < paramKeys.size(); ++i) {
            if (paramKeys[i] == static_cast<std::uint16_t>(key)) {
                if (key == dna::MachineLearnedBehaviorParameterKey::JointTranslationType) {
                    assert(paramValues[i] == static_cast<std::uint16_t>(dna::TranslationRepresentation::Vector));
                    return static_cast<std::uint16_t>(TranslationType::Vector);
                } else if (key == dna::MachineLearnedBehaviorParameterKey::JointRotationType) {
                    const auto rotationType = static_cast<dna::RotationRepresentation>(paramValues[i]);
                    if (rotationType == dna::RotationRepresentation::EulerAngles) {
                        return static_cast<std::uint16_t>(RotationType::EulerAngles);
                    } else if (rotationType == dna::RotationRepresentation::Quaternion) {
                        return static_cast<std::uint16_t>(RotationType::Quaternions);
                    }
                } else if (key == dna::MachineLearnedBehaviorParameterKey::JointScaleType) {
                    assert(paramValues[i] == static_cast<std::uint16_t>(dna::ScaleRepresentation::Vector));
                    return static_cast<std::uint16_t>(ScaleType::Vector);
                }
            }
        }
        return defaultValue;
    };
    const auto translationType = getTransformType(dna::MachineLearnedBehaviorParameterKey::JointTranslationType,
                                                  static_cast<std::uint16_t>(TranslationType::Vector));
    const auto rotationType = getTransformType(dna::MachineLearnedBehaviorParameterKey::JointRotationType,
                                               static_cast<std::uint16_t>(RotationType::EulerAngles));
    const auto scaleType =
        getTransformType(dna::MachineLearnedBehaviorParameterKey::JointScaleType, static_cast<std::uint16_t>(ScaleType::Vector));
    const auto attrCount = static_cast<std::uint16_t>(translationType + rotationType + scaleType);
    const auto qwIndex = static_cast<std::uint16_t>(translationType + rotationType - 1);

    if (static_cast<RotationType>(rotationType) == RotationType::Quaternions) {
        for (std::size_t opIdx = {}; opIdx < outputControlIndices.size(); ++opIdx) {
            for (std::size_t ii = {}; ii < mlJointsInputIndices.size(); ++ii) {
                if (outputControlIndices[opIdx] == mlJointsInputIndices[ii]) {
                    if (mlJointsOutputIndices[ii] % attrCount == qwIndex) {
                        defaultValues[opIdx] = 1.0f;
                    }
                    break;
                }
            }
        }
    }

    return defaultValues;
}

static Vector<LODSpec<std::uint16_t>> populateMLLODs(const dna::Reader* reader,
                                                     std::uint16_t mlTypeIndex,
                                                     MemoryResource* memRes) {
    const auto mlOperationSetCount = reader->getMLOperationSetCount(mlTypeIndex);
    Vector<LODSpec<std::uint16_t>> lods{mlOperationSetCount, LODSpec<std::uint16_t>{memRes}, memRes};
    for (std::uint16_t mlOperationSetIndex = {}; mlOperationSetIndex < mlOperationSetCount; ++mlOperationSetIndex) {
        const auto lodCount = reader->getLODCount();
        lods[mlOperationSetIndex].indicesPerLOD.resize(lodCount);
        for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
            const auto opIndices = reader->getMLOperationIndicesForLOD(mlTypeIndex, mlOperationSetIndex, lod);
            lods[mlOperationSetIndex].indicesPerLOD[lod].assign(opIndices.begin(), opIndices.end());
        }
        lods[mlOperationSetIndex].count = reader->getMLOperationCount(mlTypeIndex, mlOperationSetIndex);
    }
    return lods;
}

static NeuralNetLayer createLayerDescriptor(std::uint32_t inputCount,
                                            std::uint32_t outputCount,
                                            ConstArrayView<std::uint32_t> lods,
                                            dna::ActivationFunction activationFunction,
                                            ConstArrayView<float> activationFunctionParams,
                                            MemoryResource* memRes) {
    NeuralNetLayer layer{memRes};

    layer.weights.original = {outputCount, inputCount};
    const std::uint32_t padding =
        extd::roundUp(layer.weights.original.rows, static_cast<std::uint32_t>(trimd::F128::size())) - layer.weights.original.rows;
    layer.weights.padded = {layer.weights.original.rows + padding, layer.weights.original.cols};
    layer.weights.rows.reserve(lods.size());
    for (std::size_t lod = {}; lod < lods.size(); ++lod) {
        layer.weights.rows.emplace_back(lods[lod],
                                        layer.weights.padded.rows,
                                        static_cast<std::uint32_t>(trimd::F256::size()),
                                        static_cast<std::uint32_t>(trimd::F128::size()));
    }
    layer.weights.cols = PaddedBlockView{layer.weights.padded.cols,
                                         layer.weights.padded.cols - (layer.weights.padded.cols % 4u),
                                         layer.weights.padded.cols - (layer.weights.padded.cols % 8u)};

    layer.activationFunction = activationFunction;
    layer.activationFunctionParameters.assign(activationFunctionParams.begin(), activationFunctionParams.end());
    return layer;
}

template<typename T, typename TF256, typename TF128>
static void createNeuralNet(NeuralNet& neuralNet,
                            const dna::Reader* reader,
                            ConstArrayView<std::uint32_t> params,
                            MemoryResource* memRes) {
    const auto neuralNetIndex = static_cast<std::uint16_t>(params[0]);
    const auto layerCount = reader->getNeuralNetworkLayerCount(neuralNetIndex);
    const auto lodCount = reader->getLODCount();

    auto findMaskIndex = [reader, neuralNetIndex]() {
        std::uint32_t maskIndex = {};
        for (std::uint16_t meshIndex = {}; meshIndex < reader->getMeshCount(); ++meshIndex) {
            for (std::uint16_t regionIndex = {}; regionIndex < reader->getMeshRegionCount(meshIndex);
                 ++regionIndex, ++maskIndex) {
                const auto netIndicesForMeshRegion = reader->getNeuralNetworkIndicesForMeshRegion(meshIndex, regionIndex);
                if (extd::contains(netIndicesForMeshRegion, neuralNetIndex)) {
                    return maskIndex;
                }
            }
        }
        return static_cast<std::uint32_t>(-1);
    };

    neuralNet.maskIndex = findMaskIndex();

    Vector<std::uint32_t> lods{lodCount, {}, memRes};
    if (params.size() > 1ul) {
        params = params.subview(1ul, params.size() - 1ul);
        assert(params.size() == static_cast<std::size_t>(layerCount) * static_cast<std::size_t>(lodCount));
    }

    neuralNet.layers.reserve(layerCount);

    // Build layer descriptors, assign flat-buffer offsets, compute total size.
    std::size_t totalFlatSize = {};
    for (std::uint16_t layerIdx = {}; layerIdx < layerCount; ++layerIdx) {
        const auto biases = reader->getNeuralNetworkLayerBiases(neuralNetIndex, layerIdx);
        const auto weights = reader->getNeuralNetworkLayerWeights(neuralNetIndex, layerIdx);
        const auto activationFunction = reader->getNeuralNetworkLayerActivationFunction(neuralNetIndex, layerIdx);
        const auto activationFunctionParams = reader->getNeuralNetworkLayerActivationFunctionParameters(neuralNetIndex, layerIdx);
        const auto outputCount = static_cast<std::uint16_t>(biases.size());
        const auto inputCount = static_cast<std::uint16_t>(weights.size() / outputCount);
        if (params.size() > 1ul) {
            extd::copy(params.subview(static_cast<std::size_t>(layerIdx) * static_cast<std::size_t>(lodCount), lodCount),
                       ArrayView<std::uint32_t>{lods});
        } else {
            for (auto& rowCount : lods) {
                rowCount = outputCount;
            }
        }
        auto layer = createLayerDescriptor(inputCount, outputCount, lods, activationFunction, activationFunctionParams, memRes);
        layer.weightOffset = totalFlatSize;
        totalFlatSize += layer.weights.padded.size();
        layer.biasOffset = totalFlatSize;
        totalFlatSize += layer.weights.padded.rows;  // bias count = padded output count (same padding as weight rows)
        neuralNet.layers.push_back(std::move(layer));
    }

    // Allocate the flat buffer once - all layers share this single allocation.
    neuralNet.flatData.resize<T>(totalFlatSize);
    T* flat = neuralNet.flatData.template data<T>();

    // Write optimized weight/bias data directly into the flat buffer.
    for (std::uint16_t layerIdx = {}; layerIdx < layerCount; ++layerIdx) {
        const auto& layer = neuralNet.layers[layerIdx];
        const auto weights = reader->getNeuralNetworkLayerWeights(neuralNetIndex, layerIdx);
        const auto biases = reader->getNeuralNetworkLayerBiases(neuralNetIndex, layerIdx);
        bpcm::Optimizer<TF128, TF256::size(), TF128::size(), 1u>::optimize(flat + layer.weightOffset,
                                                                           weights.data(),
                                                                           layer.weights.original);
        bpcm::Optimizer<TF128, TF256::size(), TF128::size(), 1u>::optimize(flat + layer.biasOffset,
                                                                           biases.data(),
                                                                           Extent{layer.weights.original.rows, 1u});
    }
}

template<typename T, typename TF256, typename TF128>
struct MLPOperationSetBuilder {
    OperationSetData operator()(const dna::Reader* reader,
                                std::uint16_t mlTypeIndex,
                                std::uint16_t mlOperationSetIndex,
                                Vector<Matrix<std::uint16_t>>& bufferSizes,
                                Vector<Matrix<std::uint16_t>>& outputCounts,
                                MemoryResource* memRes) {
        OperationSetData data{memRes};
        data.type = OperationSetType::MLP;

        const auto mlOperationCount = reader->getMLOperationCount(mlTypeIndex, mlOperationSetIndex);
        for (std::uint16_t opIdx = {}; opIdx < mlOperationCount; ++opIdx) {
            const auto operationType = reader->getMLOperationType(mlTypeIndex, mlOperationSetIndex, opIdx);
            if (operationType != dna::MachineLearnedBehaviorOperationType::MLP) {
                // Placeholder: keep bufferSizes/outputCounts at 0 for non-MLP ops.
                continue;
            }

            const auto params = reader->getMLOperationParameters(mlTypeIndex, mlOperationSetIndex, opIdx);
            const auto neuralNetIndex = static_cast<std::uint16_t>(params[0]);
            const auto layerCount = reader->getNeuralNetworkLayerCount(neuralNetIndex);
            if (layerCount == 0u) {
                continue;
            }

            MLPOperationData entry{memRes};

            // Cross-set dependencies are stored as packed (opSetIdx, opIdx) dependency pairs.
            const auto depSetIndices =
                reader->getMLOperationDependencyOperationSetIndices(mlTypeIndex, mlOperationSetIndex, opIdx);
            const auto depOpIndices = reader->getMLOperationDependencyOperationIndices(mlTypeIndex, mlOperationSetIndex, opIdx);
            entry.inputDeps.resize(depSetIndices.size());
            for (std::size_t di = {}; di < depSetIndices.size(); ++di) {
                entry.inputDeps[di] = {depSetIndices[di], depOpIndices[di]};
            }

            entry.inputControlIndices = getOperationInputIndices(reader, mlTypeIndex, mlOperationSetIndex, opIdx, memRes);
            entry.outputControlIndices = getOperationOutputIndices(reader, mlTypeIndex, mlOperationSetIndex, opIdx, memRes);

            createNeuralNet<T, TF256, TF128>(entry.neuralNet, reader, params, memRes);

            const auto outputCount = static_cast<std::uint16_t>(entry.neuralNet.layers.back().weights.original.rows);

            // Precompute per-LOD output count from last layer's row sizes (avoids chasing 4-pointer deep structure in the hot
            // loop).
            const auto& lastRows = entry.neuralNet.layers.back().weights.rows;
            entry.outputCountsPerLOD.resize(lastRows.size());
            if (entry.outputControlIndices.empty()) {
                // Only allow operations in the final operation set to specify the output count per LOD. Not doing this would
                // require handling on the inputs as well, e.g. limiting the output count of intermediate MLP would require input
                // count of weighted sum to be limited as well.
                std::fill(entry.outputCountsPerLOD.begin(), entry.outputCountsPerLOD.end(), outputCount);
            } else {
                for (std::size_t li = {}; li < lastRows.size(); ++li) {
                    entry.outputCountsPerLOD[li] = static_cast<std::uint16_t>(lastRows[li].size);
                }
            }

            // Buffer size = 2 x workBufferHalfSize (ping-pong for layer evaluation)
            std::uint32_t maxSize = entry.neuralNet.layers.empty() ? 0u : entry.neuralNet.layers.front().weights.padded.cols;
            for (const auto& layer : entry.neuralNet.layers) {
                maxSize = std::max(maxSize, layer.weights.padded.rows);
            }
            bufferSizes[mlTypeIndex][mlOperationSetIndex][opIdx] = static_cast<std::uint16_t>(maxSize * 2u);
            outputCounts[mlTypeIndex][mlOperationSetIndex][opIdx] = outputCount;

            // Per-dependency output counts (used for gather-from-buffer inputs)
            entry.outputCounts.resize(entry.inputDeps.size());
            for (std::size_t di = {}; di < entry.inputDeps.size(); ++di) {
                entry.outputCounts[di] = outputCounts[mlTypeIndex][entry.inputDeps[di].opSetIdx][entry.inputDeps[di].opIdx];
            }

            entry.defaultValues = getDefaultValues(reader, entry.outputControlIndices, outputCount, memRes);

            data.mlpOps.push_back(std::move(entry));
        }

        if (data.mlpOps.empty()) {
            return data;  // empty sentinel - checked by OperationSetData::empty()
        }

        static constexpr std::uint32_t kNoMask = static_cast<std::uint32_t>(-1);
        for (const auto& op : data.mlpOps) {
            if (op.neuralNet.maskIndex != kNoMask) {
                data.hasMasks = true;
                break;
            }
        }

        return data;
    }
};

struct WeightedSumOperationSetBuilder {
    OperationSetData operator()(const dna::Reader* reader,
                                std::uint16_t mlTypeIndex,
                                std::uint16_t mlOperationSetIndex,
                                Vector<Matrix<std::uint16_t>>& bufferSizes,
                                Vector<Matrix<std::uint16_t>>& outputCounts,
                                MemoryResource* memRes) {
        OperationSetData data{memRes};
        std::uint16_t maxOutputCount = {};

        const auto mlOperationCount = reader->getMLOperationCount(mlTypeIndex, mlOperationSetIndex);
        data.wsOps.reserve(mlOperationCount);
        for (std::uint16_t opIdx = {}; opIdx < mlOperationCount; ++opIdx) {
            const auto params = reader->getMLOperationParameters(mlTypeIndex, mlOperationSetIndex, opIdx);
            const auto outputCount = static_cast<std::uint16_t>(params[0]);
            const auto weightParams = params.subview(1ul, params.size() - 1ul);
            const auto dependencyCount = weightParams.size();

            WeightedSumOperationData entry{memRes};
            entry.outputCount = outputCount;
            entry.weights.resize(dependencyCount, 0.0f);
            std::memcpy(entry.weights.data(), weightParams.data(), dependencyCount * sizeof(std::uint32_t));

            const auto depSetIndices =
                reader->getMLOperationDependencyOperationSetIndices(mlTypeIndex, mlOperationSetIndex, opIdx);
            const auto depOpIndices = reader->getMLOperationDependencyOperationIndices(mlTypeIndex, mlOperationSetIndex, opIdx);
            entry.inputDeps.resize(depSetIndices.size());
            for (std::size_t di = {}; di < depSetIndices.size(); ++di) {
                entry.inputDeps[di] = {depSetIndices[di], depOpIndices[di]};
            }

            bufferSizes[mlTypeIndex][mlOperationSetIndex][opIdx] = outputCount;
            outputCounts[mlTypeIndex][mlOperationSetIndex][opIdx] = outputCount;
            maxOutputCount = std::max(maxOutputCount, outputCount);

            data.wsOps.push_back(std::move(entry));
        }

        // Select BlockSize = floor(maxOutputCount / F256::size()) * F256::size(), capped at 64.
        // Encode into type field so OperationSetFactory selects the right WeightedSumOperationSet<BlockSize>.
        const auto blockSize =
            std::min<std::size_t>(64ul, static_cast<std::size_t>(maxOutputCount) / trimd::F256::size() * trimd::F256::size());
        switch (blockSize) {
        case 8:
            data.type = OperationSetType::WeightedSum8;
            break;
        case 16:
            data.type = OperationSetType::WeightedSum16;
            break;
        case 24:
            data.type = OperationSetType::WeightedSum24;
            break;
        case 32:
            data.type = OperationSetType::WeightedSum32;
            break;
        case 40:
            data.type = OperationSetType::WeightedSum40;
            break;
        case 48:
            data.type = OperationSetType::WeightedSum48;
            break;
        case 56:
            data.type = OperationSetType::WeightedSum56;
            break;
        default:
            data.type = OperationSetType::WeightedSum64;
            break;
        }

        return data;
    }
};

MachineLearnedBehaviorEvaluator::Pointer Factory::create(const Configuration& config,
                                                         RigMetadata* meta,
                                                         const dna::Reader* reader,
                                                         MemoryResource* memRes) {
    RL_UNUSED(meta);
    Matrix<LODSpec<std::uint16_t>> lods{memRes};
    Matrix<OperationSet::Pointer> mlOperations{memRes};
    Vector<Matrix<std::uint16_t>> bufferSizes{memRes};
    Vector<Matrix<std::uint16_t>> outputCounts{memRes};
    auto instanceFactory =
        [](const Vector<Matrix<std::uint16_t>>& bufferSizes_, std::uint32_t meshRegionCount, MemoryResource* instanceMemRes) {
            using OutputInstancePointer = UniqueInstance<OutputInstance, MachineLearnedBehaviorOutputInstance>;
            return OutputInstancePointer::with(instanceMemRes).create(bufferSizes_, meshRegionCount, instanceMemRes);
        };
    auto evalFactory = UniqueInstance<Evaluator, MachineLearnedBehaviorEvaluator>::with(memRes);

    if (reader == nullptr) {
        return evalFactory.create(config, std::move(lods), std::move(mlOperations), std::move(bufferSizes), 0u, instanceFactory);
    }

    const auto mlTypeCount = reader->getMLTypeCount();
    lods.resize(mlTypeCount);
    mlOperations.resize(mlTypeCount);
    bufferSizes.resize(mlTypeCount);
    outputCounts.resize(mlTypeCount);

    RuntimeTemplateInstantiator instantiator{&config};
    using BasePointer = UniqueInstance<OperationSet>::PointerType;

    for (std::uint16_t mlTypeIndex = {}; mlTypeIndex < mlTypeCount; ++mlTypeIndex) {
        lods[mlTypeIndex] = populateMLLODs(reader, mlTypeIndex, memRes);
        const auto mlOperationSetCount = reader->getMLOperationSetCount(mlTypeIndex);

        bufferSizes[mlTypeIndex].resize(mlOperationSetCount);
        outputCounts[mlTypeIndex].resize(mlOperationSetCount);

        // Build OperationSetData for each set (one per mlOperationSetIndex, empty if skipped).
        Vector<OperationSetData> mlTypeOpSets{memRes};
        mlTypeOpSets.reserve(mlOperationSetCount);

        for (std::uint16_t mlOperationSetIndex = {}; mlOperationSetIndex < mlOperationSetCount; ++mlOperationSetIndex) {
            const auto mlOperationCount = lods[mlTypeIndex][mlOperationSetIndex].count;
            bufferSizes[mlTypeIndex][mlOperationSetIndex].resize(mlOperationCount);
            outputCounts[mlTypeIndex][mlOperationSetIndex].resize(mlOperationCount);

            if (mlOperationCount == 0u) {
                mlTypeOpSets.push_back(OperationSetData{memRes});
                continue;
            }

            // Determine set type from the first non-Gather/Scatter operation.
            dna::MachineLearnedBehaviorOperationType firstOpType = {};
            bool found = false;
            for (std::uint16_t opIdx = {}; opIdx < mlOperationCount && !found; ++opIdx) {
                const auto mlOpType = reader->getMLOperationType(mlTypeIndex, mlOperationSetIndex, opIdx);
                if (mlOpType == dna::MachineLearnedBehaviorOperationType::MLP ||
                    mlOpType == dna::MachineLearnedBehaviorOperationType::WeightedSum) {
                    firstOpType = mlOpType;
                    found = true;
                }
            }
            if (!found) {
                mlTypeOpSets.push_back(OperationSetData{memRes});
                continue;
            }

            if (firstOpType == dna::MachineLearnedBehaviorOperationType::MLP) {
                mlTypeOpSets.push_back(instantiator.invoke<MLPOperationSetBuilder, OperationSetData>(reader,
                                                                                                     mlTypeIndex,
                                                                                                     mlOperationSetIndex,
                                                                                                     bufferSizes,
                                                                                                     outputCounts,
                                                                                                     memRes));
            } else {
                mlTypeOpSets.push_back(WeightedSumOperationSetBuilder()(reader,
                                                                        mlTypeIndex,
                                                                        mlOperationSetIndex,
                                                                        bufferSizes,
                                                                        outputCounts,
                                                                        memRes));
            }
        }

        // Remove empty sets and remap cross-set dependency indices.
        Vector<std::uint16_t> deletedOperationSets{memRes};
        if (mlOperationSetCount > 0u) {
            for (std::uint16_t opSetIdx = mlOperationSetCount; opSetIdx-- > 0u;) {
                if (mlTypeOpSets[opSetIdx].empty()) {
                    mlTypeOpSets.erase(extd::advanced(mlTypeOpSets.begin(), opSetIdx));
                    bufferSizes[mlTypeIndex].erase(extd::advanced(bufferSizes[mlTypeIndex].begin(), opSetIdx));
                    outputCounts[mlTypeIndex].erase(extd::advanced(outputCounts[mlTypeIndex].begin(), opSetIdx));
                    lods[mlTypeIndex].erase(extd::advanced(lods[mlTypeIndex].begin(), opSetIdx));
                    deletedOperationSets.push_back(opSetIdx);
                }
            }
        }

        if (!deletedOperationSets.empty()) {
            for (auto& data : mlTypeOpSets) {
                if (data.type == OperationSetType::MLP) {
                    for (auto& op : data.mlpOps) {
                        remapSetIndices(op.inputDeps, deletedOperationSets);
                    }
                } else {
                    for (auto& op : data.wsOps) {
                        remapSetIndices(op.inputDeps, deletedOperationSets);
                    }
                }
            }
        }

        // Construct OperationSet instances from built data.
        mlOperations[mlTypeIndex].reserve(mlTypeOpSets.size());
        for (auto& data : mlTypeOpSets) {
            mlOperations[mlTypeIndex].push_back(instantiator.invoke<OperationSetFactory, BasePointer>(std::move(data), memRes));
        }
    }

    std::uint32_t meshRegionCount = {};
    for (std::uint16_t meshIndex = {}; meshIndex < reader->getMeshCount(); ++meshIndex) {
        meshRegionCount += reader->getMeshRegionCount(meshIndex);
    }

    return evalFactory
        .create(config, std::move(lods), std::move(mlOperations), std::move(bufferSizes), meshRegionCount, instanceFactory);
}

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
