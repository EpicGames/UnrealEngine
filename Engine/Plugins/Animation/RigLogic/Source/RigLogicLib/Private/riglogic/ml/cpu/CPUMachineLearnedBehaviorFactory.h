// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/MachineLearnedBehaviorEvaluator.h"

namespace rl4 {

struct Configuration;
struct RigMetadata;

namespace ml {

namespace cpu {

class Factory {
public:
    static MachineLearnedBehaviorEvaluator::Pointer create(const Configuration& config,
                                                           RigMetadata* meta,
                                                           const dna::Reader* reader,
                                                           MemoryResource* memRes);
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
