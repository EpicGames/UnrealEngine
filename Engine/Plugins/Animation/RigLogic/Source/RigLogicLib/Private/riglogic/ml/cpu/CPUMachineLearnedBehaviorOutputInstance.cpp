// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorOutputInstance.h"

#include "riglogic/utils/Macros.h"

#include <cassert>
#include <cstdint>

namespace rl4 {

namespace ml {

namespace cpu {

OutputInstance::OutputInstance(const Vector<Matrix<std::uint16_t>>& bufferSizes,
                               std::uint32_t meshRegionCount,
                               MemoryResource* memRes) :
    workBuffer{memRes},
    workBufferPtrs{memRes},
    workBufferHalfSizes{memRes},
    workBufferOffsetsPerOperationSet{memRes},
    maskBuffer{memRes} {

    // All ops across all type/set/op dimensions share one contiguous workBuffer allocation.
    // workBufferPtrs[typeIdx][flatIdx] points into it, where flatIdx = workBufferOffsetsPerOperationSet[typeIdx][opSetIdx] +
    // opIdx.
    // workBufferOffsetsPerOperationSet[typeIdx] tracks the total number of operations up to the selected opSetIdx in it.
    //   workBufferOffsetsPerOperationSet[0] has 0 operations
    //   workBufferOffsetsPerOperationSet[1] has operationCountOf(opSet0) operations
    //   workBufferOffsetsPerOperationSet[N] has sum of all operation counts up to N
    // workBufferHalfSizes[typeIdx][flatIdx] = bufferSize/2 for ping-pong layer evaluation within each op's buffer.
    std::uint32_t totalSize = {};
    const auto mlTypeCount = bufferSizes.size();
    for (std::size_t typeIdx = {}; typeIdx < mlTypeCount; ++typeIdx) {
        for (std::size_t opSetIdx = {}; opSetIdx < bufferSizes[typeIdx].size(); ++opSetIdx) {
            for (std::size_t opIdx = {}; opIdx < bufferSizes[typeIdx][opSetIdx].size(); ++opIdx) {
                totalSize += bufferSizes[typeIdx][opSetIdx][opIdx];
            }
        }
    }

    workBuffer.resize(totalSize, 0.0f);
    maskBuffer.resize(meshRegionCount, 1.0f);

    workBufferPtrs.resize(mlTypeCount);
    workBufferHalfSizes.resize(mlTypeCount);
    workBufferOffsetsPerOperationSet.resize(mlTypeCount);
    float* cursor = workBuffer.data();
    for (std::size_t typeIdx = {}; typeIdx < mlTypeCount; ++typeIdx) {
        const auto mlSetCount = bufferSizes[typeIdx].size();

        workBufferOffsetsPerOperationSet[typeIdx].resize(mlSetCount + 1u);
        workBufferOffsetsPerOperationSet[typeIdx][0] = 0u;
        for (std::size_t opSetIdx = {}; opSetIdx < mlSetCount; ++opSetIdx) {
            workBufferOffsetsPerOperationSet[typeIdx][opSetIdx + 1u] = static_cast<std::uint16_t>(
                workBufferOffsetsPerOperationSet[typeIdx][opSetIdx] + bufferSizes[typeIdx][opSetIdx].size());
        }
        const auto totalOps = workBufferOffsetsPerOperationSet[typeIdx][mlSetCount];
        workBufferPtrs[typeIdx].resize(totalOps);
        workBufferHalfSizes[typeIdx].resize(totalOps);

        for (std::size_t opSetIdx = {}; opSetIdx < mlSetCount; ++opSetIdx) {
            const auto opCount = bufferSizes[typeIdx][opSetIdx].size();
            for (std::size_t opIdx = {}; opIdx < opCount; ++opIdx) {
                const auto size = bufferSizes[typeIdx][opSetIdx][opIdx];
                const auto flatIdx = workBufferOffsetsPerOperationSet[typeIdx][opSetIdx] + opIdx;
                workBufferPtrs[typeIdx][flatIdx] = cursor;
                workBufferHalfSizes[typeIdx][flatIdx] = static_cast<std::uint16_t>(size / 2u);
                cursor += size;
            }
        }
    }
}

ArrayView<float> OutputInstance::getOutputBuffer(std::uint16_t mlTypeIndex,
                                                 std::uint16_t mlOperationSetIndex,
                                                 std::uint16_t mlOperationIndex) {
    assert(mlTypeIndex < workBufferOffsetsPerOperationSet.size());
    assert(mlOperationSetIndex + 1u < workBufferOffsetsPerOperationSet[mlTypeIndex].size());
    const auto flatIdx =
        static_cast<std::size_t>(workBufferOffsetsPerOperationSet[mlTypeIndex][mlOperationSetIndex]) + mlOperationIndex;
    return ArrayView<float>{workBufferPtrs[mlTypeIndex][flatIdx],
                            static_cast<std::size_t>(workBufferHalfSizes[mlTypeIndex][flatIdx]) * 2u};
}

ArrayView<float> OutputInstance::getMaskBuffer() {
    return maskBuffer;
}

ConstArrayView<float> OutputInstance::getMaskBuffer() const {
    return maskBuffer;
}

std::uint16_t OutputInstance::getMLOperationSetCount(std::uint16_t mlTypeIndex) const {
    assert(mlTypeIndex < workBufferOffsetsPerOperationSet.size());
    // workBufferOffsetsPerOperationSet has mlSetCount+1 entries (sentinel at end).
    return static_cast<std::uint16_t>(workBufferOffsetsPerOperationSet[mlTypeIndex].size() - 1u);
}

std::uint16_t OutputInstance::getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const {
    assert(mlTypeIndex < workBufferOffsetsPerOperationSet.size());
    assert(mlOperationSetIndex + 1u < workBufferOffsetsPerOperationSet[mlTypeIndex].size());
    return static_cast<std::uint16_t>(workBufferOffsetsPerOperationSet[mlTypeIndex][mlOperationSetIndex + 1u] -
                                      workBufferOffsetsPerOperationSet[mlTypeIndex][mlOperationSetIndex]);
}

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
