// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace ml {

namespace cpu {

class OutputInstance : public MachineLearnedBehaviorOutputInstance {
public:
    using Factory = std::function<Pointer(const Vector<Matrix<std::uint16_t>>&, std::uint32_t, MemoryResource*)>;

public:
    OutputInstance(const Vector<Matrix<std::uint16_t>>& bufferSizes, std::uint32_t meshRegionCount, MemoryResource* memRes);
    ArrayView<float> getMaskBuffer() override;
    ConstArrayView<float> getMaskBuffer() const override;
    ArrayView<float> getOutputBuffer(std::uint16_t mlTypeIndex,
                                     std::uint16_t mlOperationSetIndex,
                                     std::uint16_t mlOperationIndex);

    ConstArrayView<float*> getWorkBufferPtrs(std::uint16_t mlTypeIndex) const {
        return workBufferPtrs[mlTypeIndex];
    }
    ConstArrayView<std::uint16_t> getWorkBufferHalfSizes(std::uint16_t mlTypeIndex) const {
        return workBufferHalfSizes[mlTypeIndex];
    }

    ConstArrayView<std::uint16_t> getWorkBufferOffsetsPerOperationSet(std::uint16_t mlTypeIndex) const {
        return workBufferOffsetsPerOperationSet[mlTypeIndex];
    }

    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override;
    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override;

private:
    AlignedVector<float> workBuffer;
    Matrix<float*> workBufferPtrs;
    Matrix<std::uint16_t> workBufferHalfSizes;
    Matrix<std::uint16_t> workBufferOffsetsPerOperationSet;
    Vector<float> maskBuffer;
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
