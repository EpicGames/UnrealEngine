// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/types/Extent.h"
#include "riglogic/types/PaddedBlockView.h"
#include "riglogic/types/bpcm/FloatArray.h"

namespace rl4 {

namespace ml {

namespace cpu {

// Describes the memory layout of a weight matrix: original dimensions, SIMD-padded dimensions,
// per-LOD output block views, and the input block view. Does not own any data.
struct MatrixLayout {
    Extent original;
    Extent padded;
    Vector<PaddedBlockView> rows;  // per-LOD output block views
    PaddedBlockView cols;          // input block view

    explicit MatrixLayout(MemoryResource* memRes) :
        original{},
        padded{},
        rows{memRes},
        cols{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(original, padded, rows, cols);
    }
};

struct NeuralNetLayer {
    MatrixLayout weights;
    Vector<float> activationFunctionParameters;
    dna::ActivationFunction activationFunction;
    std::size_t weightOffset;  // offset into NeuralNet::flatData
    std::size_t biasOffset;    // offset into NeuralNet::flatData

    explicit NeuralNetLayer(MemoryResource* memRes) :
        weights{memRes},
        activationFunctionParameters{memRes},
        activationFunction{},
        weightOffset{},
        biasOffset{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(weights, activationFunctionParameters, activationFunction, weightOffset, biasOffset);
    }
};

struct NeuralNet {
    Vector<NeuralNetLayer> layers;
    std::uint32_t maskIndex;
    FloatArray flatData;  // contiguous weights+biases for all layers, indexed via per-layer offsets

    NeuralNet(MemoryResource* memRes) :
        layers{memRes},
        maskIndex{},
        flatData{memRes} {
    }

    NeuralNet(Vector<NeuralNetLayer>&& layers_, std::uint32_t maskIndex_, FloatArray&& flatData_) :
        layers{std::move(layers_)},
        maskIndex{maskIndex_},
        flatData{std::move(flatData_)} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(layers, maskIndex, flatData);
    }
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
