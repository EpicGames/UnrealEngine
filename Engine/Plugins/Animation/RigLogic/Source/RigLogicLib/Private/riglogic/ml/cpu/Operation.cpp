// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/ml/cpu/Operation.h"

namespace rl4 {

namespace ml {

namespace cpu {

OperationSet::~OperationSet() = default;

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
