// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/NeuralNet.h"
#include "riglogic/ml/cpu/layers/LayerEvaluator.h"
#include "riglogic/ml/cpu/layers/LeakyReLULayerEvaluator.h"
#include "riglogic/ml/cpu/layers/LinearLayerEvaluator.h"
#include "riglogic/ml/cpu/layers/ReLULayerEvaluator.h"
#include "riglogic/ml/cpu/layers/SigmoidLayerEvaluator.h"
#include "riglogic/ml/cpu/layers/TanHLayerEvaluator.h"
#include "riglogic/utils/Extd.h"
#include "riglogic/utils/Macros.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstring>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace ml {

namespace cpu {

enum class OperationSetType : std::uint16_t {
    MLP = 0,
    WeightedSum8 = 1,
    WeightedSum16 = 2,
    WeightedSum24 = 3,
    WeightedSum32 = 4,
    WeightedSum40 = 5,
    WeightedSum48 = 6,
    WeightedSum56 = 7,
    WeightedSum64 = 8,
    WeightedSum = WeightedSum64  // default alias: largest block size
};

template<typename Archive>
void load(Archive& archive, OperationSetType& value) {
    std::uint16_t tmp = {};
    archive(tmp);
    value = static_cast<OperationSetType>(tmp);
}

template<typename Archive>
void save(Archive& archive, OperationSetType& value) {
    archive(static_cast<std::uint16_t>(value));
}

struct OperationSet {
    using Pointer = UniqueInstance<OperationSet>::PointerType;

    virtual ~OperationSet();

    virtual void execute(ConstArrayView<std::uint16_t> activeOpIndices,
                         std::uint16_t lod,
                         ConstArrayView<float> masks,
                         ArrayView<float> inputBuffer,
                         std::size_t opSetWorkBufferOffset,
                         ConstArrayView<float*> workBufferPtrs,
                         ConstArrayView<std::uint16_t> workBufferHalfSizes,
                         ConstArrayView<std::uint16_t> workBufferOffsetsPerOperationSet) const = 0;

    virtual OperationSetType getType() const = 0;

    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive) = 0;
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) = 0;
};

struct Dependency {
    std::uint16_t opSetIdx;
    std::uint16_t opIdx;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(opSetIdx, opIdx);
    }
};

struct MLPOperationData {
    NeuralNet neuralNet;
    Vector<Dependency> inputDeps;  // cross-operation-set (opSetIdx, opIdx) pairs; empty if inputs come from controls
    Vector<std::uint16_t> inputControlIndices;
    Vector<std::uint16_t> outputControlIndices;
    Vector<std::uint16_t> outputCounts;        // per-dep output count for the gather phase
    Vector<std::uint16_t> outputCountsPerLOD;  // precomputed from layers.back().weights.rows[lod].size
    Vector<float> defaultValues;

    explicit MLPOperationData(MemoryResource* memRes) :
        neuralNet{memRes},
        inputDeps{memRes},
        inputControlIndices{memRes},
        outputControlIndices{memRes},
        outputCounts{memRes},
        outputCountsPerLOD{memRes},
        defaultValues{memRes} {
    }
};

struct WeightedSumOperationData {
    Vector<Dependency> inputDeps;  // cross-operation-set (opSetIdx, opIdx) pairs
    Vector<float> weights;
    std::uint16_t outputCount;

    explicit WeightedSumOperationData(MemoryResource* memRes) :
        inputDeps{memRes},
        weights{memRes},
        outputCount{} {
    }
};

struct OperationSetData {
    OperationSetType type;
    Vector<WeightedSumOperationData> wsOps;
    Vector<MLPOperationData> mlpOps;
    bool hasMasks;

    explicit OperationSetData(MemoryResource* memRes) :
        type{},
        wsOps{memRes},
        mlpOps{memRes},
        hasMasks{false} {
    }

    bool empty() const {
        return mlpOps.empty() && wsOps.empty();
    }
};

template<typename T, typename TF256, typename TF128>
struct MLPOperationSet : OperationSet {
    Vector<MLPOperationData> ops;
    Vector<const T*> cachedLayerDataPtrs;
    bool hasMasks;

    explicit MLPOperationSet(MemoryResource* memRes) :
        ops{memRes},
        cachedLayerDataPtrs{memRes},
        hasMasks{false} {
    }

    explicit MLPOperationSet(Vector<MLPOperationData>&& ops_, bool hasMasks_, MemoryResource* memRes) :
        ops{std::move(ops_)},
        cachedLayerDataPtrs{memRes},
        hasMasks{hasMasks_} {
        cachedLayerDataPtrs.reserve(ops.size());
        for (const auto& op : ops) {
            cachedLayerDataPtrs.push_back(op.neuralNet.flatData.template data<T>());
        }
    }

    void execute(ConstArrayView<std::uint16_t> activeOpIndices,
                 std::uint16_t lod,
                 ConstArrayView<float> masks,
                 ArrayView<float> inputBuffer,
                 std::size_t opSetWorkBufferOffset,
                 ConstArrayView<float*> workBufferPtrs,
                 ConstArrayView<std::uint16_t> workBufferHalfSizes,
                 ConstArrayView<std::uint16_t> workBufferOffsetsPerOperationSet) const override {

        // workBufferPtrs[opSetWorkBufferOffset + opIdx] gives the ping-pong work buffer for op opIdx in this op-set.
        // workBufferHalfSizes[opSetWorkBufferOffset + opIdx] = bufferSize/2; the buffer splits into buf1=[ptr,half) and
        // buf2=[ptr+half,2*half) for ping-pong evaluation across layers - swap after each layer; result = buf1.data() after the
        // loop. Cross-set deps resolve via workBufferPtrs[workBufferOffsetsPerOperationSet[dep.opSetIdx] + dep.opIdx].
        float* pInput = inputBuffer.data();
        for (std::size_t k = {}; k < activeOpIndices.size(); ++k) {
            const auto opIdx = activeOpIndices[k];
            const auto& op = ops[opIdx];

            // Prefetch next op's weight blob before mask+gather so the full iteration hides the miss.
            if (k + 1u < activeOpIndices.size()) {
                TF256::prefetchT0(cachedLayerDataPtrs[activeOpIndices[k + 1u]]);
            }

            float* pBuf = workBufferPtrs[opSetWorkBufferOffset + opIdx];

            if (hasMasks) {
                static constexpr std::uint32_t kNoMask = static_cast<std::uint32_t>(-1);
                const float weight = (op.neuralNet.maskIndex != kNoMask) ? masks[op.neuralNet.maskIndex] : 1.0f;
                if (weight == 0.0f) {
                    if (op.outputControlIndices.empty()) {
                        std::memcpy(pBuf, op.defaultValues.data(), op.defaultValues.size() * sizeof(float));
                    } else {
                        const auto* outIndices = op.outputControlIndices.data();
                        const auto limit = std::min(op.defaultValues.size(), op.outputControlIndices.size());
                        for (std::size_t i = {}; i < limit; ++i) {
                            pInput[outIndices[i]] = op.defaultValues[i];
                        }
                    }
                    continue;
                }
            }

            // gather
            if (!op.inputControlIndices.empty()) {
                const auto count = op.inputControlIndices.size();
                const auto* inIndices = op.inputControlIndices.data();
                for (std::size_t i = {}; i < count; ++i) {
                    pBuf[i] = pInput[inIndices[i]];
                }
            } else {
                // Populate MLP (neural net) inputs from the outputs of all nodes that are dependencies of the MLP
                for (std::size_t i = {}, offset = {}; i < op.inputDeps.size(); ++i) {
                    const auto dep = op.inputDeps[i];
                    const float* depData =
                        workBufferPtrs[static_cast<std::size_t>(workBufferOffsetsPerOperationSet[dep.opSetIdx]) + dep.opIdx];
                    const auto count = op.outputCounts[i];
                    std::memcpy(pBuf + offset, depData, count * sizeof(float));
                    offset += count;
                }
            }

            // evaluate layers
            const float* result = pBuf;
            if (!op.neuralNet.layers.empty()) {
                const auto halfSize = static_cast<std::size_t>(workBufferHalfSizes[opSetWorkBufferOffset + opIdx]);
                auto buf1 = ArrayView<float>{pBuf, halfSize};
                auto buf2 = ArrayView<float>{pBuf + halfSize, halfSize};
                const T* flatPtr = op.neuralNet.flatData.template data<T>();
                for (const auto& layer : op.neuralNet.layers) {
                    const T* weights = flatPtr + layer.weightOffset;
                    const T* biases = flatPtr + layer.biasOffset;
                    const float* activationParams = layer.activationFunctionParameters.data();
                    switch (layer.activationFunction) {
                    case dna::ActivationFunction::linear:
                        calculateBlock4<T, TF256, TF128, LinearActivationFunction>(weights,
                                                                                   biases,
                                                                                   activationParams,
                                                                                   layer.weights.cols,
                                                                                   layer.weights.rows[lod],
                                                                                   buf1,
                                                                                   buf2);
                        break;
                    case dna::ActivationFunction::relu:
                        calculateBlock4<T, TF256, TF128, ReLUActivationFunction>(weights,
                                                                                 biases,
                                                                                 activationParams,
                                                                                 layer.weights.cols,
                                                                                 layer.weights.rows[lod],
                                                                                 buf1,
                                                                                 buf2);
                        break;
                    case dna::ActivationFunction::leakyrelu:
                        calculateBlock4<T, TF256, TF128, LeakyReLUActivationFunction>(weights,
                                                                                      biases,
                                                                                      activationParams,
                                                                                      layer.weights.cols,
                                                                                      layer.weights.rows[lod],
                                                                                      buf1,
                                                                                      buf2);
                        break;
                    case dna::ActivationFunction::tanh:
                        calculateBlock4<T, TF256, TF128, TanHActivationFunction>(weights,
                                                                                 biases,
                                                                                 activationParams,
                                                                                 layer.weights.cols,
                                                                                 layer.weights.rows[lod],
                                                                                 buf1,
                                                                                 buf2);
                        break;
                    case dna::ActivationFunction::sigmoid:
                        calculateBlock4<T, TF256, TF128, SigmoidActivationFunction>(weights,
                                                                                    biases,
                                                                                    activationParams,
                                                                                    layer.weights.cols,
                                                                                    layer.weights.rows[lod],
                                                                                    buf1,
                                                                                    buf2);
                        break;
                    }
                    std::swap(buf1, buf2);
                }
                result = buf1.data();
            }

            // scatter
            if (!op.outputControlIndices.empty()) {
                const auto outputCount = static_cast<std::uint32_t>(op.outputCountsPerLOD[lod]);
                const auto outIndicesCount = static_cast<std::uint32_t>(op.outputControlIndices.size());
                const auto limit = std::min(outputCount, outIndicesCount);
                const auto* outIndices = op.outputControlIndices.data();
                for (std::uint32_t i = {}; i < limit; ++i) {
                    pInput[outIndices[i]] = result[i];
                }
            } else if (result != pBuf) {
                // Intermediate dep MLP with an odd layer count: the ping-pong evaluation left the result
                // in the second half of the work buffer (pBuf+halfSize) rather than pBuf.
                // Downstream dep gathers always read from workBufferPtrs[dep] = pBuf, so normalize here.
                const auto outputCount = static_cast<std::size_t>(op.outputCountsPerLOD[lod]);
                std::memcpy(pBuf, result, outputCount * sizeof(float));
            }
        }
    }

    OperationSetType getType() const override {
        return OperationSetType::MLP;
    }

    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override {
        std::uint32_t count = {};
        archive(count);
        auto memRes = ops.get_allocator().getMemoryResource();
        ops.reserve(count);
        hasMasks = false;
        for (std::uint32_t i = {}; i < count; ++i) {
            MLPOperationData entry{memRes};
            archive(entry.neuralNet,
                    entry.inputDeps,
                    entry.inputControlIndices,
                    entry.outputControlIndices,
                    entry.outputCounts,
                    entry.defaultValues);
            if (!entry.neuralNet.layers.empty()) {
                const auto& lastRows = entry.neuralNet.layers.back().weights.rows;
                entry.outputCountsPerLOD.resize(lastRows.size());
                // Mirror the factory rule: intermediate MLPs (no scatter output) use a fixed count across all LODs so
                // that dep-gathers and weighted sum reads are consistent with the normalization-fix memcpy.
                if (entry.outputControlIndices.empty()) {
                    std::fill(entry.outputCountsPerLOD.begin(),
                              entry.outputCountsPerLOD.end(),
                              static_cast<std::uint16_t>(lastRows[0].size));
                } else {
                    for (std::size_t li = {}; li < lastRows.size(); ++li) {
                        entry.outputCountsPerLOD[li] = static_cast<std::uint16_t>(lastRows[li].size);
                    }
                }
            }
            static constexpr std::uint32_t kNoMask = static_cast<std::uint32_t>(-1);
            if (entry.neuralNet.maskIndex != kNoMask) {
                hasMasks = true;
            }
            ops.push_back(std::move(entry));
            cachedLayerDataPtrs.push_back(ops.back().neuralNet.flatData.template data<T>());
        }
    }

    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override {
        archive(static_cast<std::uint32_t>(ops.size()));
        for (auto& op : ops) {
            archive(op.neuralNet,
                    op.inputDeps,
                    op.inputControlIndices,
                    op.outputControlIndices,
                    op.outputCounts,
                    op.defaultValues);
        }
    }
};

template<std::size_t Count, std::size_t Index = 0>
struct InvokeN {
    template<typename Fn>
    void operator()(Fn func) {
        func(Index);
        InvokeN<Count, Index + 1>()(func);
    }
};

template<std::size_t Count>
struct InvokeN<Count, Count> {
    template<typename Fn>
    void operator()(Fn func) {
        RL_UNUSED(func);
    }
};

template<std::size_t BlockSize, typename TF256, typename TF128>
struct WeightedSumOperationSet : OperationSet {
    Vector<WeightedSumOperationData> ops;

    explicit WeightedSumOperationSet(MemoryResource* memRes) :
        ops{memRes} {
    }

    explicit WeightedSumOperationSet(Vector<WeightedSumOperationData>&& ops_, MemoryResource* /*memRes*/) :
        ops{std::move(ops_)} {
    }

    void execute(ConstArrayView<std::uint16_t> activeOpIndices,
                 std::uint16_t lod,
                 ConstArrayView<float> masks,
                 ArrayView<float> inputBuffer,
                 std::size_t opSetWorkBufferOffset,
                 ConstArrayView<float*> workBufferPtrs,
                 ConstArrayView<std::uint16_t> workBufferHalfSizes,
                 ConstArrayView<std::uint16_t> workBufferOffsetsPerOperationSet) const override {

        // workBufferPtrs[opSetWorkBufferOffset + opIdx] is the destination work slot for this op.
        // Cross-set dep pointers resolve via workBufferPtrs[workBufferOffsetsPerOperationSet[dep.opSetIdx] + dep.opIdx].
        // Outputs are processed in BlockSize-wide aligned chunks: blockCount = BlockSize / TF256::size() SIMD registers
        // accumulate weighted dep contributions per chunk; a scalar remainder loop handles the trailing elements.
        RL_UNUSED(lod);
        RL_UNUSED(masks);
        RL_UNUSED(inputBuffer);
        RL_UNUSED(workBufferHalfSizes);

        constexpr std::size_t blockCount = BlockSize / TF256::size();

        for (const auto opIdx : activeOpIndices) {
            const auto& op = ops[opIdx];
            if (op.inputDeps.empty()) {
                continue;
            }

            float* pBuf = workBufferPtrs[opSetWorkBufferOffset + opIdx];
            const auto outputCount = static_cast<std::size_t>(op.outputCount);
            const auto alignedCount = outputCount - (outputCount % BlockSize);

            for (std::size_t elem = {}; elem < alignedCount; elem += BlockSize) {
                TF256 sums[blockCount] = {};
                for (std::size_t di = {}; di < op.inputDeps.size(); ++di) {
                    // Populate weighted sum inputs from the buffers of the nodes that are its dependencies
                    const auto depBufOffset = workBufferOffsetsPerOperationSet[op.inputDeps[di].opSetIdx];
                    const float* pDepBuf = workBufferPtrs[static_cast<std::size_t>(depBufOffset) + op.inputDeps[di].opIdx];
                    TF256 blocks[blockCount];
                    InvokeN<blockCount>()(
                        [&](std::size_t bi) { blocks[bi] = TF256::fromUnalignedSource(pDepBuf + elem + TF256::size() * bi); });
                    const TF256 weight{op.weights[di]};
                    InvokeN<blockCount>()([&](std::size_t bi) { sums[bi] += blocks[bi] * weight; });
                }
                InvokeN<blockCount>()([&](std::size_t bi) { sums[bi].unalignedStore(pBuf + elem + TF256::size() * bi); });
            }

            if (alignedCount != outputCount) {
                std::fill(pBuf + alignedCount, pBuf + outputCount, 0.0f);
                for (std::size_t di = {}; di < op.inputDeps.size(); ++di) {
                    const float* pDep =
                        workBufferPtrs[static_cast<std::size_t>(workBufferOffsetsPerOperationSet[op.inputDeps[di].opSetIdx]) +
                                       op.inputDeps[di].opIdx];
                    const float w = op.weights[di];
                    for (std::size_t elem = alignedCount; elem < outputCount; ++elem) {
                        pBuf[elem] += w * pDep[elem];
                    }
                }
            }
        }
    }

    OperationSetType getType() const override {
        switch (BlockSize) {
        case 8:
            return OperationSetType::WeightedSum8;
        case 16:
            return OperationSetType::WeightedSum16;
        case 24:
            return OperationSetType::WeightedSum24;
        case 32:
            return OperationSetType::WeightedSum32;
        case 40:
            return OperationSetType::WeightedSum40;
        case 48:
            return OperationSetType::WeightedSum48;
        case 56:
            return OperationSetType::WeightedSum56;
        default:
            return OperationSetType::WeightedSum64;
        }
    }

    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override {
        std::uint32_t count = {};
        archive(count);
        auto memRes = ops.get_allocator().getMemoryResource();
        ops.reserve(count);
        for (std::uint32_t i = {}; i < count; ++i) {
            WeightedSumOperationData entry{memRes};
            archive(entry.inputDeps, entry.weights, entry.outputCount);
            ops.push_back(std::move(entry));
        }
    }

    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override {
        archive(static_cast<std::uint32_t>(ops.size()));
        for (auto& op : ops) {
            archive(op.inputDeps, op.weights, op.outputCount);
        }
    }
};

template<typename T, typename TF256, typename TF128>
struct OperationSetFactory {
    using BasePointer = UniqueInstance<OperationSet>::PointerType;

    BasePointer operator()(OperationSetData&& data, MemoryResource* memRes) {
        switch (data.type) {
        case (OperationSetType::MLP): {

            using OpSet = MLPOperationSet<T, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.mlpOps), data.hasMasks, memRes);
        }
        case OperationSetType::WeightedSum8: {
            using OpSet = WeightedSumOperationSet<8, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        case OperationSetType::WeightedSum16: {
            using OpSet = WeightedSumOperationSet<16, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        case OperationSetType::WeightedSum24: {
            using OpSet = WeightedSumOperationSet<24, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        case OperationSetType::WeightedSum32: {
            using OpSet = WeightedSumOperationSet<32, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        case OperationSetType::WeightedSum40: {
            using OpSet = WeightedSumOperationSet<40, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        case OperationSetType::WeightedSum48: {
            using OpSet = WeightedSumOperationSet<48, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        case OperationSetType::WeightedSum56: {
            using OpSet = WeightedSumOperationSet<56, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        case OperationSetType::WeightedSum64:
        default: {
            using OpSet = WeightedSumOperationSet<64, TF256, TF128>;
            return UniqueInstance<OpSet, OperationSet>::with(memRes).create(std::move(data.wsOps), memRes);
        }
        }
    }
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
