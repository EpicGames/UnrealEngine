// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/NeuralNet.h"
#include "riglogic/utils/Macros.h"

namespace rl4 {

namespace ml {

namespace cpu {

template<typename TF128, typename T>
static FORCE_INLINE void
processBlocks8x1(const float* inputVectorEndAlignedTo4, const float* inputVectorEnd, const T* weights, TF128& sum1, TF128& sum2) {
    TF128 remainder1{};
    TF128 remainder2{};
    for (const float* inputVector = inputVectorEndAlignedTo4; inputVector < inputVectorEnd;
         ++inputVector, weights += TF128::size() * 2) {
        const TF128 input{*inputVector};
        const TF128 blk1 = TF128::fromAlignedSource(weights);
        const TF128 blk2 = TF128::fromAlignedSource(weights + TF128::size());
        remainder1 += (blk1 * input);
        remainder2 += (blk2 * input);
    }
    sum1 += remainder1;
    sum2 += remainder2;
}

template<typename TF128, typename TActivationFunction, typename T>
static FORCE_INLINE void processBlocks8x4(const float* inputVectorStart,
                                          const float* inputVectorEndAlignedTo4,
                                          const float* inputVectorEnd,
                                          const T* weights,
                                          const T* biases,
                                          const float* activationParams,
                                          float* outbuf) {
    TF128 sum1{};
    TF128 sum2{};
    TF128 sum3{};
    TF128 sum4{};
    TF128 sum5{};
    TF128 sum6{};
    TF128 sum7{};
    TF128 sum8{};
    for (const float* inputVector = inputVectorStart; inputVector < inputVectorEndAlignedTo4;
         inputVector += 4ul, weights += (TF128::size() * 8ul)) {
        const TF128 input1{inputVector[0]};
        const TF128 input2{inputVector[1]};
        const TF128 input3{inputVector[2]};
        const TF128 input4{inputVector[3]};
        const TF128 blk1 = TF128::fromAlignedSource(weights + TF128::size() * 0);
        const TF128 blk2 = TF128::fromAlignedSource(weights + TF128::size() * 1);
        const TF128 blk3 = TF128::fromAlignedSource(weights + TF128::size() * 2);
        const TF128 blk4 = TF128::fromAlignedSource(weights + TF128::size() * 3);
        const TF128 blk5 = TF128::fromAlignedSource(weights + TF128::size() * 4);
        const TF128 blk6 = TF128::fromAlignedSource(weights + TF128::size() * 5);
        const TF128 blk7 = TF128::fromAlignedSource(weights + TF128::size() * 6);
        const TF128 blk8 = TF128::fromAlignedSource(weights + TF128::size() * 7);
        sum1 += (blk1 * input1);
        sum2 += (blk2 * input1);
        sum3 += (blk3 * input2);
        sum4 += (blk4 * input2);
        sum5 += (blk5 * input3);
        sum6 += (blk6 * input3);
        sum7 += (blk7 * input4);
        sum8 += (blk8 * input4);
    }
    // Process 8x1 horizontal remainder portion after 8x4 blocks are consumed
    processBlocks8x1(inputVectorEndAlignedTo4, inputVectorEnd, weights, sum1, sum2);

    const TF128 bias1 = TF128::fromAlignedSource(biases + TF128::size() * 0);
    const TF128 bias2 = TF128::fromAlignedSource(biases + TF128::size() * 1);

    sum1 += sum3;
    sum2 += sum4;
    sum5 += sum7;
    sum6 += sum8;
    sum1 += sum5;
    sum2 += sum6;

    sum1 += bias1;
    sum2 += bias2;

    TActivationFunction{}(sum1, sum2, activationParams);

    sum1.alignedStore(outbuf);
    sum2.alignedStore(outbuf + TF128::size());
}

template<typename TF128, typename T>
static FORCE_INLINE void processBlocks4x1(const float* inputVectorEndAlignedTo8,
                                          const float* inputVectorEnd,
                                          const T* weights,
                                          TF128& sum1) {
    TF128 remainder{};
    for (const float* inputVector = inputVectorEndAlignedTo8; inputVector < inputVectorEnd;
         ++inputVector, weights += TF128::size()) {
        const TF128 input{*inputVector};
        const TF128 blk = TF128::fromAlignedSource(weights);
        remainder += (blk * input);
    }
    sum1 += remainder;
}

template<typename TF128, typename TActivationFunction, typename T>
static FORCE_INLINE void processBlocks4x8(const float* inputVectorStart,
                                          const float* inputVectorEndAlignedTo8,
                                          const float* inputVectorEnd,
                                          const T* weights,
                                          const T* biases,
                                          const float* activationParams,
                                          float* outbuf) {
    TF128 sum1{};
    TF128 sum2{};
    TF128 sum3{};
    TF128 sum4{};
    TF128 sum5{};
    TF128 sum6{};
    TF128 sum7{};
    TF128 sum8{};
    for (const float* inputVector = inputVectorStart; inputVector < inputVectorEndAlignedTo8;
         inputVector += 8ul, weights += (TF128::size() * 8ul)) {
        const TF128 input1{inputVector[0]};
        const TF128 input2{inputVector[1]};
        const TF128 input3{inputVector[2]};
        const TF128 input4{inputVector[3]};
        const TF128 input5{inputVector[4]};
        const TF128 input6{inputVector[5]};
        const TF128 input7{inputVector[6]};
        const TF128 input8{inputVector[7]};
        const TF128 blk1 = TF128::fromAlignedSource(weights);
        const TF128 blk2 = TF128::fromAlignedSource(weights + TF128::size());
        const TF128 blk3 = TF128::fromAlignedSource(weights + TF128::size() * 2);
        const TF128 blk4 = TF128::fromAlignedSource(weights + TF128::size() * 3);
        const TF128 blk5 = TF128::fromAlignedSource(weights + TF128::size() * 4);
        const TF128 blk6 = TF128::fromAlignedSource(weights + TF128::size() * 5);
        const TF128 blk7 = TF128::fromAlignedSource(weights + TF128::size() * 6);
        const TF128 blk8 = TF128::fromAlignedSource(weights + TF128::size() * 7);
        sum1 += (blk1 * input1);
        sum2 += (blk2 * input2);
        sum3 += (blk3 * input3);
        sum4 += (blk4 * input4);
        sum5 += (blk5 * input5);
        sum6 += (blk6 * input6);
        sum7 += (blk7 * input7);
        sum8 += (blk8 * input8);
    }
    // Process 4x1 horizontal remainder portion after 4x8 blocks are consumed
    processBlocks4x1(inputVectorEndAlignedTo8, inputVectorEnd, weights, sum1);

    const TF128 bias1 = TF128::fromAlignedSource(biases);

    sum1 += sum2;
    sum3 += sum4;
    sum5 += sum6;
    sum7 += sum8;
    sum1 += sum3;
    sum5 += sum7;
    sum1 += sum5;

    sum1 += bias1;

    TActivationFunction{}(sum1, activationParams);

    sum1.alignedStore(outbuf);
}

// Raw-pointer overload: used by the set-based evaluator with pre-resolved pointers.
template<typename T, typename TF256, typename TF128, template<class...> class TActivationFunction>
static FORCE_INLINE void calculateBlock4(const T* weights,
                                         const T* biases,
                                         const float* activationParams,
                                         const PaddedBlockView& cols,
                                         const PaddedBlockView& rows,
                                         ConstArrayView<float> inputs,
                                         ArrayView<float> outputs) {
    float* outputVector = outputs.data();
    const float* outputVectorEndPaddedToLastFullBlock = outputVector + rows.sizePaddedToLastFullBlock;
    float* outputVectorEnd = outputVector + rows.size;

    const float* inputVector = inputs.data();
    const float* inputVectorEndPaddedTo4 = inputVector + cols.sizePaddedToLastFullBlock;
    const float* inputVectorEndPaddedTo8 = inputVector + cols.sizePaddedToSecondLastFullBlock;
    const float* inputVectorEnd = inputVector + cols.size;

    const std::size_t fullBlockSize = cols.size * TF256::size();
    const std::size_t halfBlockSize = cols.size * TF128::size();

    for (; outputVector < outputVectorEndPaddedToLastFullBlock;
         outputVector += TF256::size(), biases += TF256::size(), weights += fullBlockSize) {
        alignas(TF128::alignment()) float outbuf[TF128::size() * 2];
        processBlocks8x4<TF128, TActivationFunction<TF128>>(inputVector,
                                                            inputVectorEndPaddedTo4,
                                                            inputVectorEnd,
                                                            weights,
                                                            biases,
                                                            activationParams,
                                                            outbuf);
        std::memcpy(outputVector, outbuf, sizeof(outbuf));
    }

    for (; outputVector < outputVectorEnd; outputVector += TF128::size(), biases += TF128::size(), weights += halfBlockSize) {
        alignas(TF128::alignment()) float outbuf[TF128::size()];
        processBlocks4x8<TF128, TActivationFunction<TF128>>(inputVector,
                                                            inputVectorEndPaddedTo8,
                                                            inputVectorEnd,
                                                            weights,
                                                            biases,
                                                            activationParams,
                                                            outbuf);
        std::memcpy(outputVector, outbuf, sizeof(outbuf));
    }
}

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
