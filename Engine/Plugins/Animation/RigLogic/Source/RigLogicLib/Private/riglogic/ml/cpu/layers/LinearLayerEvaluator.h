// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/layers/LayerEvaluator.h"
#include "riglogic/utils/Macros.h"

namespace rl4 {

namespace ml {

namespace cpu {

template<typename TFVec>
struct LinearActivationFunction {

    void operator()(TFVec& /*unused*/, const float* /*unused*/) {
    }

    void operator()(TFVec& /*unused*/, TFVec& /*unused*/, const float* /*unused*/) {
    }
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
