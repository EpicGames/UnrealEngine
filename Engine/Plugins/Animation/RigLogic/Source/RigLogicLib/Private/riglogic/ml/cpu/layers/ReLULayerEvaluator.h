// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/layers/LayerEvaluator.h"
#include "riglogic/system/simd/SIMD.h"
#include "riglogic/utils/Macros.h"

namespace rl4 {

namespace ml {

namespace cpu {

template<typename TFVec>
struct ReLUActivationFunction {

    void operator()(TFVec& sum, const float* /*unused*/) {
        const TFVec zero{};
        const TFVec mask = (sum < zero);
        sum = trimd::andnot(mask, sum);
    }

    void operator()(TFVec& sum1, TFVec& sum2, const float* /*unused*/) {
        const TFVec zero{};
        const TFVec mask1 = (sum1 < zero);
        const TFVec mask2 = (sum2 < zero);
        sum1 = trimd::andnot(mask1, sum1);
        sum2 = trimd::andnot(mask2, sum2);
    }
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
