// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/layers/LayerEvaluator.h"
#include "riglogic/ml/cpu/layers/Utils.h"
#include "riglogic/utils/Macros.h"

#include <cmath>
#include <type_traits>

namespace rl4 {

namespace ml {

namespace cpu {

template<typename TFVec, typename Enable = void>
struct TanHActivationFunction;

template<typename TF128>
struct TanHActivationFunction<TF128, typename std::enable_if<HasSize<TF128, 4ul>::value>::type> {

    void operator()(TF128& sum, const float* /*unused*/) {
        // FIXME: dangerously inefficient code
        alignas(TF128::alignment()) float buf[TF128::size()];

        sum.alignedStore(buf);

        buf[0] = std::tanh(buf[0]);
        buf[1] = std::tanh(buf[1]);
        buf[2] = std::tanh(buf[2]);
        buf[3] = std::tanh(buf[3]);

        sum.alignedLoad(buf);
    }

    void operator()(TF128& sum1, TF128& sum2, const float* /*unused*/) {
        // FIXME: dangerously inefficient code
        alignas(TF128::alignment()) float buf[TF128::size() * 2];

        sum1.alignedStore(buf);
        sum2.alignedStore(buf + TF128::size());

        buf[0] = std::tanh(buf[0]);
        buf[1] = std::tanh(buf[1]);
        buf[2] = std::tanh(buf[2]);
        buf[3] = std::tanh(buf[3]);
        buf[4] = std::tanh(buf[4]);
        buf[5] = std::tanh(buf[5]);
        buf[6] = std::tanh(buf[6]);
        buf[7] = std::tanh(buf[7]);

        sum1.alignedLoad(buf);
        sum2.alignedLoad(buf + TF128::size());
    }
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
