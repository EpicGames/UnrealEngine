// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace rl4 {

namespace ml {

namespace cpu {

template<typename TFVec, std::size_t Size>
struct HasSize {
    static constexpr bool value = (TFVec::size() == Size);
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4
