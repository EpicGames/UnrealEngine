// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/psdnet/PSDNetOutputInstance.h"

#include <cstdint>

namespace rl4 {

class ControlsInputInstance;

class PSDNet {
public:
    using Pointer = UniqueInstance<PSDNet>::PointerType;

protected:
    virtual ~PSDNet();

public:
    virtual PSDNetOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const = 0;
    virtual std::uint16_t getPSDCount() const = 0;
    virtual ConstArrayView<std::uint16_t> getPSDInputIndicesForLOD(std::uint16_t lod) const = 0;
    virtual ConstArrayView<std::uint16_t> getPSDOutputIndicesForLOD(std::uint16_t lod) const = 0;
    virtual void calculate(ControlsInputInstance* inputInstance,
                           PSDNetOutputInstance* outputInstance,
                           std::uint16_t lod) const = 0;
    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive) = 0;
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) = 0;
};

}  // namespace rl4
