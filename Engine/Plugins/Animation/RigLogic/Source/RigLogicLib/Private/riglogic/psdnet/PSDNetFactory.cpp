// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/psdnet/PSDNetFactory.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/psdnet/PSDNetImpl.h"
#include "riglogic/psdnet/PSDNetImplOutputInstance.h"
#include "riglogic/psdnet/PSDNetNull.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/utils/Extd.h"
#include "riglogic/utils/Macros.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

PSDNet::Pointer PSDNetFactory::create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes) {
    RL_UNUSED(config);
    const EvaluatorType type = meta->popFrontEvaluator();

    if (type == EvaluatorType::Null) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<PSDNetNull, PSDNet>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    const auto controlCount = static_cast<std::size_t>(meta->rawControlCount) + static_cast<std::size_t>(meta->psdControlCount) +
                              static_cast<std::size_t>(meta->mlControlCount) + static_cast<std::size_t>(meta->rbfControlCount);
    auto instanceFactory = [controlCount](MemoryResource* instanceMemRes) {
        return UniqueInstance<PSDNetImplOutputInstance, PSDNetOutputInstance>::with(instanceMemRes)
            .create(static_cast<std::uint16_t>(controlCount), instanceMemRes);
    };
    return UniqueInstance<PSDNetImpl, PSDNet>::with(memRes).create(instanceFactory, memRes);
}

PSDNet::Pointer PSDNetFactory::create(const Configuration& config,
                                      RigMetadata* meta,
                                      const dna::Reader* reader,
                                      Controls* controls,
                                      MemoryResource* memRes) {
    RL_UNUSED(config);
    if ((meta->psdControlCount == 0u) || (meta->lodCount == 0u)) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        return UniqueInstance<PSDNetNull, PSDNet>::with(memRes).create();
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);

    Matrix<std::uint16_t> inputLODs{memRes};
    Matrix<std::uint16_t> outputLODs{memRes};
    Vector<PSD> psds{memRes};
    Vector<std::uint16_t> inputIndicesPerPSD{memRes};

    const auto lodCount = reader->getLODCount();
    const auto rawControlCount = static_cast<std::size_t>(reader->getRawControlCount());
    const auto psdControlCount = static_cast<std::size_t>(reader->getPSDCount());
    const auto mlControlCount = static_cast<std::size_t>(reader->getMLControlCount());
    const auto rbfControlCount = static_cast<std::size_t>(reader->getRBFPoseControlCount());
    const auto controlCount = rawControlCount + psdControlCount + mlControlCount + rbfControlCount;
    const auto psdRows = reader->getPSDRowIndices();
    const auto psdCols = reader->getPSDColumnIndices();
    const auto psdWeights = reader->getPSDValues();
    auto minPSD = rawControlCount;
    auto maxPSD = static_cast<std::size_t>(0);
    if (psdControlCount > 0u) {
        maxPSD = minPSD + psdControlCount - 1ul;
    }

    psds.resize(psdControlCount);
    inputIndicesPerPSD.reserve(psdCols.size());
    for (std::size_t start = {}; start < psdRows.size(); ++start) {
        const auto psdOutputIndex = psdRows[start];
        PSD& psd = psds[static_cast<std::size_t>(psdOutputIndex) - rawControlCount];
        if (psd.size != 0ul) {
            continue;
        }

        psd.weight = 1.0f;
        psd.offset = inputIndicesPerPSD.size();
        for (std::size_t i = start; i < psdRows.size(); ++i) {
            if (psdRows[i] == psdOutputIndex) {
                const auto psdInputIndex = psdCols[i];
                inputIndicesPerPSD.push_back(psdInputIndex);
                psd.weight *= psdWeights[i];
            }
        }
        psd.size = inputIndicesPerPSD.size() - psd.offset;
    }

    inputLODs.resize(lodCount);
    outputLODs.resize(lodCount);

    for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
        const auto controlIndices = controls->getRegisteredControls(lod);
        if (controlIndices.size() == 0ul) {
            continue;
        }

        auto& inputIndices = inputLODs[lod];
        auto& outputIndices = outputLODs[lod];
        inputIndices.reserve(rawControlCount);
        outputIndices.reserve(psdControlCount);

        for (auto controlIndex : controlIndices) {
            if ((controlIndex >= minPSD) && (controlIndex <= maxPSD)) {
                outputIndices.push_back(controlIndex);
                const PSD& psd = psds[static_cast<std::size_t>(controlIndex) - minPSD];
                for (std::size_t i = psd.offset; i < psd.offset + psd.size; ++i) {
                    inputIndices.push_back(inputIndicesPerPSD[i]);
                }
            }
        }

        std::sort(inputIndices.begin(), inputIndices.end());
        std::sort(outputIndices.begin(), outputIndices.end());
        inputIndices.erase(std::unique(inputIndices.begin(), inputIndices.end()), inputIndices.end());
        outputIndices.erase(std::unique(outputIndices.begin(), outputIndices.end()), outputIndices.end());
    }

    auto instanceFactory = [controlCount](MemoryResource* instanceMemRes) {
        return UniqueInstance<PSDNetImplOutputInstance, PSDNetOutputInstance>::with(instanceMemRes)
            .create(static_cast<std::uint16_t>(controlCount), instanceMemRes);
    };

    return UniqueInstance<PSDNetImpl, PSDNet>::with(memRes).create(std::move(inputLODs),
                                                                   std::move(outputLODs),
                                                                   std::move(inputIndicesPerPSD),
                                                                   std::move(psds),
                                                                   static_cast<std::uint16_t>(minPSD),
                                                                   static_cast<std::uint16_t>(maxPSD),
                                                                   std::move(instanceFactory));
}

}  // namespace rl4
