// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/psdnet/PSDNet.h"

namespace rl4 {

struct Configuration;
struct RigMetadata;
class Controls;

struct PSDNetFactory {
    static PSDNet::Pointer
    create(const Configuration& config, RigMetadata* meta, const dna::Reader* reader, Controls* controls, MemoryResource* memRes);
    static PSDNet::Pointer create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes);
};

}  // namespace rl4
