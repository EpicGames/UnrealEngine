// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/psdnet/PSDNetImpl.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/utils/Extd.h"
#include "riglogic/utils/Macros.h"

#include <cstddef>

namespace rl4 {

namespace {

constexpr float minPSDValue = 0.0f;
constexpr float maxPSDValue = 1.0f;

}  // namespace

PSDNetImpl::PSDNetImpl(PSDNetOutputInstance::Factory instanceFactory_, MemoryResource* memRes) :
    inputLODs{memRes},
    outputLODs{memRes},
    inputIndicesPerPSD{memRes},
    psds{memRes},
    psdMinIndex{},
    psdMaxIndex{},
    instanceFactory{instanceFactory_} {
}

PSDNetImpl::PSDNetImpl(Matrix<std::uint16_t>&& inputLODs_,
                       Matrix<std::uint16_t>&& outputLODs_,
                       Vector<std::uint16_t>&& inputIndicesPerPSD_,
                       Vector<PSD>&& psds_,
                       std::uint16_t psdMinIndex_,
                       std::uint16_t psdMaxIndex_,
                       PSDNetOutputInstance::Factory instanceFactory_) :
    inputLODs{std::move(inputLODs_)},
    outputLODs{std::move(outputLODs_)},
    inputIndicesPerPSD{std::move(inputIndicesPerPSD_)},
    psds{std::move(psds_)},
    psdMinIndex{psdMinIndex_},
    psdMaxIndex{psdMaxIndex_},
    instanceFactory{instanceFactory_} {
}

PSDNetOutputInstance::Pointer PSDNetImpl::createInstance(MemoryResource* instanceMemRes) const {
    return instanceFactory(instanceMemRes);
}

std::uint16_t PSDNetImpl::getPSDCount() const {
    return static_cast<std::uint16_t>(psds.size());
}

ConstArrayView<std::uint16_t> PSDNetImpl::getPSDInputIndicesForLOD(std::uint16_t lod) const {
    assert(lod < inputLODs.size());
    return inputLODs[lod];
}

ConstArrayView<std::uint16_t> PSDNetImpl::getPSDOutputIndicesForLOD(std::uint16_t lod) const {
    assert(lod < outputLODs.size());
    return outputLODs[lod];
}

void PSDNetImpl::calculate(ControlsInputInstance* inputInstance, PSDNetOutputInstance* outputInstance, std::uint16_t lod) const {
    auto inputs = inputInstance->getInputBuffer();
    auto clampBuffer = outputInstance->getClampBuffer();

    ConstArrayView<std::uint16_t> inputIndices = inputLODs[lod];
    ConstArrayView<std::uint16_t> outputIndices = outputLODs[lod];

    for (auto inputIndex : inputIndices) {
        clampBuffer[inputIndex] = extd::clamp(inputs[inputIndex], minPSDValue, maxPSDValue);
    }

    for (auto outputIndex : outputIndices) {
        const PSD psd = psds[static_cast<std::size_t>(outputIndex) - static_cast<std::size_t>(psdMinIndex)];
        float psdOutput = psd.weight;
        for (std::size_t i = psd.offset; i < psd.offset + psd.size; ++i) {
            const float input = clampBuffer[inputIndicesPerPSD[i]];
            psdOutput *= input;
        }
        inputs[outputIndex] = std::min(maxPSDValue, psdOutput);
    }
}

void PSDNetImpl::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(inputLODs, outputLODs, inputIndicesPerPSD, psds, psdMinIndex, psdMaxIndex);
}

void PSDNetImpl::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(inputLODs, outputLODs, inputIndicesPerPSD, psds, psdMinIndex, psdMaxIndex);
}

}  // namespace rl4
