// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/psdnet/PSDNet.h"
#include "riglogic/psdnet/PSDNetOutputInstance.h"

#include <cstddef>

namespace rl4 {

class ControlsInputInstance;

struct PSD {
    std::size_t offset;
    std::size_t size;
    float weight;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(offset, size, weight);
    }
};

class PSDNetImpl : public PSDNet {
public:
    PSDNetImpl(PSDNetOutputInstance::Factory instanceFactory_, MemoryResource* memRes);
    PSDNetImpl(Matrix<std::uint16_t>&& inputLODs_,
               Matrix<std::uint16_t>&& outputLODs_,
               Vector<std::uint16_t>&& inputIndicesPerPSD_,
               Vector<PSD>&& psds_,
               std::uint16_t psdMinIndex_,
               std::uint16_t psdMaxIndex_,
               PSDNetOutputInstance::Factory instanceFactory_);

    PSDNetOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint16_t getPSDCount() const override;
    ConstArrayView<std::uint16_t> getPSDInputIndicesForLOD(std::uint16_t lod) const override;
    ConstArrayView<std::uint16_t> getPSDOutputIndicesForLOD(std::uint16_t lod) const override;
    void calculate(ControlsInputInstance* inputInstance, PSDNetOutputInstance* outputInstance, std::uint16_t lod) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

private:
    Matrix<std::uint16_t> inputLODs;
    Matrix<std::uint16_t> outputLODs;
    Vector<std::uint16_t> inputIndicesPerPSD;
    Vector<PSD> psds;
    std::uint16_t psdMinIndex;
    std::uint16_t psdMaxIndex;
    PSDNetOutputInstance::Factory instanceFactory;
};

}  // namespace rl4
