// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/psdnet/PSDNetImplOutputInstance.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

PSDNetImplOutputInstance::PSDNetImplOutputInstance(std::uint16_t controlCount, MemoryResource* memRes) :
    clampBuffer{controlCount, {}, memRes} {
}

ArrayView<float> PSDNetImplOutputInstance::getClampBuffer() {
    return clampBuffer;
}

void PSDNetImplOutputInstance::resetClampBuffer() {
    std::fill(clampBuffer.begin(), clampBuffer.end(), 0.0f);
}

}  // namespace rl4
