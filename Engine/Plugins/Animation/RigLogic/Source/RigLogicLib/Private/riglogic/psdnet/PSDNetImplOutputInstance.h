// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/psdnet/PSDNetOutputInstance.h"

#include <cstdint>

namespace rl4 {

class PSDNetImplOutputInstance : public PSDNetOutputInstance {
public:
    PSDNetImplOutputInstance(std::uint16_t controlCount, MemoryResource* memRes);
    void resetClampBuffer() override;
    ArrayView<float> getClampBuffer() override;

private:
    Vector<float> clampBuffer;
};

}  // namespace rl4
