// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/psdnet/PSDNetNull.h"

#include "riglogic/psdnet/PSDNetNullOutputInstance.h"

namespace rl4 {

PSDNetOutputInstance::Pointer PSDNetNull::createInstance(MemoryResource* instanceMemRes) const {
    return UniqueInstance<PSDNetNullOutputInstance, PSDNetOutputInstance>::with(instanceMemRes).create();
}

std::uint16_t PSDNetNull::getPSDCount() const {
    return {};
}

ConstArrayView<std::uint16_t> PSDNetNull::getPSDInputIndicesForLOD(std::uint16_t /*unused*/) const {
    return {};
}

ConstArrayView<std::uint16_t> PSDNetNull::getPSDOutputIndicesForLOD(std::uint16_t /*unused*/) const {
    return {};
}

void PSDNetNull::calculate(ControlsInputInstance* /*unused*/, PSDNetOutputInstance* /*unused*/, std::uint16_t /*unused*/) const {
}

void PSDNetNull::load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) {
}

void PSDNetNull::save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) {
}

}  // namespace rl4
