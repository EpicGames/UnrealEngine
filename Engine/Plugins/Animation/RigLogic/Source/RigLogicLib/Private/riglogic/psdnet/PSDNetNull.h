// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/psdnet/PSDNet.h"
#include "riglogic/psdnet/PSDNetOutputInstance.h"

#include <cstdint>

namespace rl4 {

class PSDNetNull : public PSDNet {
public:
    PSDNetOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    std::uint16_t getPSDCount() const override;
    ConstArrayView<std::uint16_t> getPSDInputIndicesForLOD(std::uint16_t /*unused*/) const override;
    ConstArrayView<std::uint16_t> getPSDOutputIndicesForLOD(std::uint16_t /*unused*/) const override;
    void calculate(ControlsInputInstance* /*unused*/, PSDNetOutputInstance* /*unused*/, std::uint16_t /*unused*/) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) override;
};

}  // namespace rl4
