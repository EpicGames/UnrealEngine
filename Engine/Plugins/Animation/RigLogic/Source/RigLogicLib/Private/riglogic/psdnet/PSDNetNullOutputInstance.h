// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/psdnet/PSDNetOutputInstance.h"

#include <cstdint>

namespace rl4 {

class PSDNetNullOutputInstance : public PSDNetOutputInstance {
public:
    void resetClampBuffer() override;
    ArrayView<float> getClampBuffer() override;
};

}  // namespace rl4
