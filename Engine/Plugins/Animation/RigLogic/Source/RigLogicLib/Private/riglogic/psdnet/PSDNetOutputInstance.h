// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

namespace rl4 {

class PSDNetOutputInstance {
public:
    using Pointer = UniqueInstance<PSDNetOutputInstance>::PointerType;
    using Factory = std::function<Pointer(MemoryResource*)>;

public:
    virtual ~PSDNetOutputInstance();
    virtual void resetClampBuffer() = 0;
    virtual ArrayView<float> getClampBuffer() = 0;
};

}  // namespace rl4
