// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/RBFBehavior.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/rbf/RBFBehaviorOutputInstance.h"

#include <cassert>
#include <cstddef>

namespace rl4 {

RBFBehavior::RBFBehavior(RBFBehaviorEvaluator::Pointer evaluator_) :
    evaluator{std::move(evaluator_)} {
}

RBFBehaviorOutputInstance::Pointer RBFBehavior::createInstance(MemoryResource* instanceMemRes) const {
    return evaluator->createInstance(instanceMemRes);
}

ConstArrayView<std::uint16_t> RBFBehavior::getSolverIndicesForLOD(std::uint16_t lod) const {
    return evaluator->getSolverIndicesForLOD(lod);
}

void RBFBehavior::calculate(ControlsInputInstance* inputs,
                            RBFBehaviorOutputInstance* intermediateOutputs,
                            std::uint16_t lod) const {
    evaluator->calculate(inputs, intermediateOutputs, lod);
}

void RBFBehavior::calculate(ControlsInputInstance* inputs,
                            RBFBehaviorOutputInstance* intermediateOutputs,
                            std::uint16_t lod,
                            std::uint16_t solverIndex) const {
    evaluator->calculate(inputs, intermediateOutputs, lod, solverIndex);
}

}  // namespace rl4
