// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/RBFBehaviorEvaluator.h"
#include "riglogic/rbf/RBFBehaviorOutputInstance.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Macros.h"

#include <cstddef>

namespace rl4 {

class ControlsInputInstance;

class RBFBehavior {
public:
    using Pointer = UniqueInstance<RBFBehavior>::PointerType;

public:
    explicit RBFBehavior(RBFBehaviorEvaluator::Pointer evaluator_);

    RBFBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const;
    ConstArrayView<std::uint16_t> getSolverIndicesForLOD(std::uint16_t lod) const;
    void calculate(ControlsInputInstance* inputs, RBFBehaviorOutputInstance* intermediateOutputs, std::uint16_t lod) const;
    void calculate(ControlsInputInstance* inputs,
                   RBFBehaviorOutputInstance* intermediateOutputs,
                   std::uint16_t lod,
                   std::uint16_t solverIndex) const;

    template<class Archive>
    void load(Archive& archive) {
        evaluator->load(archive);
    }

    template<class Archive>
    void save(Archive& archive) {
        evaluator->save(archive);
    }

private:
    RBFBehaviorEvaluator::Pointer evaluator;
};

}  // namespace rl4
