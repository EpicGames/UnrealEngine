// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/RBFBehaviorOutputInstance.h"

namespace rl4 {

class ControlsInputInstance;

class RBFBehaviorEvaluator {
public:
    using Pointer = UniqueInstance<RBFBehaviorEvaluator>::PointerType;

protected:
    virtual ~RBFBehaviorEvaluator();

public:
    virtual RBFBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const = 0;
    virtual ConstArrayView<std::uint16_t> getSolverIndicesForLOD(std::uint16_t lod) const = 0;
    virtual void calculate(ControlsInputInstance* inputs,
                           RBFBehaviorOutputInstance* intermediateOutputs,
                           std::uint16_t lod) const = 0;
    virtual void calculate(ControlsInputInstance* inputs,
                           RBFBehaviorOutputInstance* intermediateOutputs,
                           std::uint16_t lod,
                           std::uint16_t solverIndex) const = 0;
    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive) = 0;
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) = 0;
};

}  // namespace rl4
