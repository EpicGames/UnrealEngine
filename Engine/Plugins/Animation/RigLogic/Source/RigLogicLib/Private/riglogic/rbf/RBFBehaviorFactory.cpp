// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/RBFBehaviorFactory.h"

#include "riglogic/rbf/RBFBehaviorEvaluator.h"
#include "riglogic/rbf/RBFBehaviorNullEvaluator.h"
#include "riglogic/rbf/cpu/CPURBFBehaviorFactory.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/SIMD.h"

#include <cstdint>

namespace rl4 {

RBFBehaviorEvaluator::Pointer createRBFEvaluator(const Configuration& config,
                                                 RigMetadata* meta,
                                                 const dna::Reader* reader,
                                                 MemoryResource* memRes) {
#if defined(__clang__)
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wunused-local-typedef"
#endif
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wunused-local-typedefs"
#endif
    using StorageValueType = float;
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
#if defined(__clang__)
    #pragma clang diagnostic pop
#endif

    // Work around unused parameter warning when building without SSE and AVX
    static_cast<void>(config);
#ifdef RL_BUILD_WITH_SSE
    if ((config.calculationType == CalculationType::SSE) || (config.calculationType == CalculationType::AnyVector)) {
        return rbf::cpu::Factory<StorageValueType, trimd::sse::F256, trimd::sse::F128>::create(meta, reader, memRes);
    }
#endif  // RL_BUILD_WITH_SSE
#ifdef RL_BUILD_WITH_AVX
    if ((config.calculationType == CalculationType::AVX) || (config.calculationType == CalculationType::AnyVector)) {
        // Use 256-bit AVX registers and whatever 128-bit width type is available
        return rbf::cpu::Factory<StorageValueType, trimd::avx::F256, trimd::sse::F128>::create(meta, reader, memRes);
    }
#endif  // RL_BUILD_WITH_AVX
#ifdef RL_BUILD_WITH_NEON
    if ((config.calculationType == CalculationType::NEON) || (config.calculationType == CalculationType::AnyVector)) {
        return rbf::cpu::Factory<StorageValueType, trimd::neon::F256, trimd::neon::F128>::create(meta, reader, memRes);
    }
#endif  // RL_BUILD_WITH_NEON
    return rbf::cpu::Factory<float, trimd::scalar::F256, trimd::scalar::F128>::create(meta, reader, memRes);
}

RBFBehavior::Pointer RBFBehaviorFactory::create(const Configuration& config,
                                                RigMetadata* meta,
                                                const dna::Reader* reader,
                                                MemoryResource* memRes) {
    auto moduleFactory = UniqueInstance<RBFBehavior>::with(memRes);
    if (!config.loadRBFBehavior || (meta->lodCount == 0u) || (meta->rbfSolverCount == 0u) || (meta->rbfControlCount == 0u)) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        auto evaluator = UniqueInstance<RBFBehaviorNullEvaluator, RBFBehaviorEvaluator>::with(memRes).create();
        return moduleFactory.create(std::move(evaluator));
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    return moduleFactory.create(createRBFEvaluator(config, meta, reader, memRes));
}

RBFBehavior::Pointer RBFBehaviorFactory::create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes) {
    auto moduleFactory = UniqueInstance<RBFBehavior>::with(memRes);
    const EvaluatorType type = meta->popFrontEvaluator();

    if (type == EvaluatorType::Null) {
        meta->pushBackEvaluator(EvaluatorType::Null);
        auto evaluator = UniqueInstance<RBFBehaviorNullEvaluator, RBFBehaviorEvaluator>::with(memRes).create();
        return moduleFactory.create(std::move(evaluator));
    }

    meta->pushBackEvaluator(EvaluatorType::Concrete);
    return moduleFactory.create(createRBFEvaluator(config, meta, nullptr, memRes));
}

}  // namespace rl4
