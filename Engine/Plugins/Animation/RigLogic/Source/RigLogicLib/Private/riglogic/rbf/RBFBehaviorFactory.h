// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/RBFBehavior.h"

namespace rl4 {

struct Configuration;
struct RigMetadata;

struct RBFBehaviorFactory {
    static RBFBehavior::Pointer create(const Configuration& config,
                                       RigMetadata* meta,
                                       const dna::Reader* reader,
                                       MemoryResource* memRes);
    static RBFBehavior::Pointer create(const Configuration& config, RigMetadata* meta, MemoryResource* memRes);
};

}  // namespace rl4
