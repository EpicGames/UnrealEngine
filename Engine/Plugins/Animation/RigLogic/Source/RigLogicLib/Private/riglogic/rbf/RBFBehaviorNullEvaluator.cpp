// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/RBFBehaviorNullEvaluator.h"

#include "riglogic/rbf/RBFBehaviorNullOutputInstance.h"

namespace rl4 {

RBFBehaviorOutputInstance::Pointer RBFBehaviorNullEvaluator::createInstance(MemoryResource* instanceMemRes) const {
    return UniqueInstance<RBFBehaviorNullOutputInstance, RBFBehaviorOutputInstance>::with(instanceMemRes).create();
}

ConstArrayView<std::uint16_t> RBFBehaviorNullEvaluator::getSolverIndicesForLOD(std::uint16_t /*unused*/) const {
    return {};
}

void RBFBehaviorNullEvaluator::calculate(ControlsInputInstance* /*unused*/,
                                         RBFBehaviorOutputInstance* /*unused*/,
                                         std::uint16_t /*unused*/) const {
}

void RBFBehaviorNullEvaluator::calculate(ControlsInputInstance* /*unused*/,
                                         RBFBehaviorOutputInstance* /*unused*/,
                                         std::uint16_t /*unused*/,
                                         std::uint16_t /*unused*/) const {
}

void RBFBehaviorNullEvaluator::load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) {
}

void RBFBehaviorNullEvaluator::save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) {
}

}  // namespace rl4
