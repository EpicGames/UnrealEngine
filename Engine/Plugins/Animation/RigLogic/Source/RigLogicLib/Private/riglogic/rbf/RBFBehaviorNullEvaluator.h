// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/RBFBehaviorEvaluator.h"
#include "riglogic/rbf/RBFBehaviorOutputInstance.h"

namespace rl4 {

class ControlsInputInstance;

class RBFBehaviorNullEvaluator : public RBFBehaviorEvaluator {
public:
    RBFBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override;
    ConstArrayView<std::uint16_t> getSolverIndicesForLOD(std::uint16_t /*unused*/) const override;
    void calculate(ControlsInputInstance* /*unused*/,
                   RBFBehaviorOutputInstance* /*unused*/,
                   std::uint16_t /*unused*/) const override;
    void calculate(ControlsInputInstance* /*unused*/,
                   RBFBehaviorOutputInstance* /*unused*/,
                   std::uint16_t /*unused*/,
                   std::uint16_t /*unused*/) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& /*unused*/) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& /*unused*/) override;
};

}  // namespace rl4
