// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/RBFBehaviorNullOutputInstance.h"

namespace rl4 {

ArrayView<float> RBFBehaviorNullOutputInstance::getInputBuffer(std::uint16_t /*unused*/) {
    return {};
}

ArrayView<float> RBFBehaviorNullOutputInstance::getIntermediateWeightsBuffer(std::uint16_t /*unused*/) {
    return {};
}

ArrayView<float> RBFBehaviorNullOutputInstance::getOutputWeightsBuffer(std::uint16_t /*unused*/) {
    return {};
}

}  // namespace rl4
