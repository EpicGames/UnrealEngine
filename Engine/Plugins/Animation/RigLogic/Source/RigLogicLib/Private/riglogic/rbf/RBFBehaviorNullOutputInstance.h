// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/rbf/RBFBehaviorOutputInstance.h"

namespace rl4 {

class RBFBehaviorNullOutputInstance : public RBFBehaviorOutputInstance {
public:
    ArrayView<float> getInputBuffer(std::uint16_t /*unused*/) override;
    ArrayView<float> getIntermediateWeightsBuffer(std::uint16_t /*unused*/) override;
    ArrayView<float> getOutputWeightsBuffer(std::uint16_t /*unused*/) override;
};

}  // namespace rl4
