// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

namespace rl4 {

class RBFBehaviorOutputInstance {
public:
    using Pointer = UniqueInstance<RBFBehaviorOutputInstance>::PointerType;

public:
    virtual ~RBFBehaviorOutputInstance();
    virtual ArrayView<float> getInputBuffer(std::uint16_t solverIndex) = 0;
    virtual ArrayView<float> getIntermediateWeightsBuffer(std::uint16_t solverIndex) = 0;
    virtual ArrayView<float> getOutputWeightsBuffer(std::uint16_t solverIndex) = 0;
};

}  // namespace rl4
