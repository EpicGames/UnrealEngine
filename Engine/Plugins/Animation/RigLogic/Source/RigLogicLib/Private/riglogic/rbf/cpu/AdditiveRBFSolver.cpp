// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/cpu/AdditiveRBFSolver.h"

namespace rl4 {

AdditiveRBFSolver::AdditiveRBFSolver(MemoryResource* memRes) :
    RBFSolver(memRes) {
}

AdditiveRBFSolver::AdditiveRBFSolver(const RBFSolverRecipe& recipe, MemoryResource* memRes) :
    RBFSolver(recipe, memRes) {
}

RBFSolverType AdditiveRBFSolver::getSolverType() const {
    return RBFSolverType::Additive;
}

void AdditiveRBFSolver::solve(ArrayView<float> input, ArrayView<float> /*unused*/, ArrayView<float> outputWeights) const {
    convertInput(input);
    getDistanceWeight(targets, input, outputWeights, radius);

    normalizeAndCutOff(outputWeights);
}

}  // namespace rl4
