// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/rbf/cpu/RBFSolver.h"

namespace rl4 {

class AdditiveRBFSolver : public RBFSolver {
public:
    explicit AdditiveRBFSolver(MemoryResource* memRes);
    AdditiveRBFSolver(const RBFSolverRecipe& recipe, MemoryResource* memRes);

    RBFSolverType getSolverType() const override;
    void solve(ArrayView<float> input, ArrayView<float> intermediateWeights, ArrayView<float> outputWeights) const override;
};

}  // namespace rl4
