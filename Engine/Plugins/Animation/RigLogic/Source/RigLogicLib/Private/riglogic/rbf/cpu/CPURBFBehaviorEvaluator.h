// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/rbf/RBFBehaviorEvaluator.h"
#include "riglogic/rbf/RBFBehaviorOutputInstance.h"
#include "riglogic/rbf/cpu/AdditiveRBFSolver.h"
#include "riglogic/rbf/cpu/CPURBFBehaviorOutputInstance.h"
#include "riglogic/rbf/cpu/InterpolativeRBFSolver.h"
#include "riglogic/rbf/cpu/RBFSolver.h"
#include "riglogic/types/LODSpec.h"
#include "riglogic/utils/Macros.h"

#include <cstddef>
#include <cstdint>

namespace rl4 {

namespace rbf {

namespace cpu {

template<typename T, typename TF256, typename TF128>
class Evaluator : public RBFBehaviorEvaluator {
public:
    using SolverVectorType = Vector<RBFSolver::Pointer>;

    struct Accessor;
    friend Accessor;

public:
    Evaluator(LODSpec<std::uint16_t>&& lods_,
              SolverVectorType&& solvers_,
              Matrix<std::uint16_t>&& solverRawControlInputIndices_,
              Matrix<std::uint16_t>&& solverRawControlOutputIndices_,
              Matrix<std::uint16_t>&& solverPoseIndices_,
              Matrix<std::uint16_t>&& poseInputControlIndices_,
              Matrix<std::uint16_t>&& poseOutputControlIndices_,
              Matrix<float>&& poseOutputControlWeights_,
              Vector<std::uint16_t>&& inputCountPerSolver_,
              Vector<std::uint16_t>&& targetCountPerSolver_,
              OutputInstance::Factory&& instanceFactory_) :
        lods{std::move(lods_)},
        solvers{std::move(solvers_)},
        solverRawControlInputIndices{std::move(solverRawControlInputIndices_)},
        solverRawControlOutputIndices{std::move(solverRawControlOutputIndices_)},
        solverPoseIndices{std::move(solverPoseIndices_)},
        poseInputControlIndices{std::move(poseInputControlIndices_)},
        poseOutputControlIndices{std::move(poseOutputControlIndices_)},
        poseOutputControlWeights{std::move(poseOutputControlWeights_)},
        inputCountPerSolver{std::move(inputCountPerSolver_)},
        targetCountPerSolver{std::move(targetCountPerSolver_)},
        instanceFactory{std::move(instanceFactory_)} {
    }

    RBFBehaviorOutputInstance::Pointer createInstance(MemoryResource* instanceMemRes) const override {
        return instanceFactory(inputCountPerSolver, targetCountPerSolver, instanceMemRes);
    }

    ConstArrayView<std::uint16_t> getSolverIndicesForLOD(std::uint16_t lod) const override {
        assert(lod < lods.indicesPerLOD.size());
        return lods.indicesPerLOD[lod];
    }

    void calculate(std::uint16_t solverIndex,
                   ArrayView<float> rawControls,
                   ArrayView<float> inputBuffer,
                   ArrayView<float> intermediateWeightsBuffer,
                   ArrayView<float> outputWeightsBuffer) const {

        assert(solverIndex < solvers.size());
        const auto& rawControlInputIndices = solverRawControlInputIndices[solverIndex];
        const auto& rawControlOutputIndices = solverRawControlOutputIndices[solverIndex];
        const auto& poseIndices = solverPoseIndices[solverIndex];
        const auto poseCount = poseIndices.size();
        const auto rawControlInputCount = rawControlInputIndices.size();

        for (const auto outputIndex : rawControlOutputIndices) {
            rawControls[outputIndex] = 0.0f;
        }

        for (std::uint16_t i = {}; i < rawControlInputCount; ++i) {
            const std::uint16_t ri = rawControlInputIndices[i];
            inputBuffer[i] = rawControls[ri];
        }

        solvers[solverIndex]->solve(inputBuffer, intermediateWeightsBuffer, outputWeightsBuffer);

        for (std::uint16_t i = {}; i < poseCount; ++i) {
            const std::uint16_t pi = poseIndices[i];
            const auto& inputControlIndices = poseInputControlIndices[pi];
            const auto& outputControlIndices = poseOutputControlIndices[pi];
            const auto& outputControlWeights = poseOutputControlWeights[pi];
            assert(outputControlIndices.size() == outputControlWeights.size());

            float inputWeight = 1.0f;
            for (const auto inputControlIndex : inputControlIndices) {
                inputWeight *= rawControls[inputControlIndex];
            }

            for (std::uint16_t ci = 0u; ci < outputControlIndices.size(); ci++) {
                rawControls[outputControlIndices[ci]] += outputControlWeights[ci] * outputWeightsBuffer[i] * inputWeight;
            }
        }
    }

    void calculate(ControlsInputInstance* inputs,
                   RBFBehaviorOutputInstance* intermediateOutputs,
                   std::uint16_t lod) const override {
        assert(lod < lods.indicesPerLOD.size());
        const auto& solverIndices = lods.indicesPerLOD[lod];
        auto rawControls = inputs->getInputBuffer();
        auto outputInstance = static_cast<OutputInstance*>(intermediateOutputs);
        for (const auto solverIndex : solverIndices) {
            assert(solverIndex < solvers.size());
            auto inputBuffer = outputInstance->getInputBuffer(solverIndex);
            auto intermediateWeightsBuffer = outputInstance->getIntermediateWeightsBuffer(solverIndex);
            auto outputWeightsBuffer = outputInstance->getOutputWeightsBuffer(solverIndex);
            calculate(solverIndex, rawControls, inputBuffer, intermediateWeightsBuffer, outputWeightsBuffer);
        }
    }

    void calculate(ControlsInputInstance* inputs,
                   RBFBehaviorOutputInstance* intermediateOutputs,
                   std::uint16_t lod,
                   std::uint16_t solverIndex) const override {
        assert(lod < lods.indicesPerLOD.size());
        static_cast<void>(lod);
        assert(solverIndex < solvers.size());
        auto rawControls = inputs->getInputBuffer();
        auto outputInstance = static_cast<OutputInstance*>(intermediateOutputs);
        auto inputBuffer = outputInstance->getInputBuffer(solverIndex);
        auto intermediateWeightsBuffer = outputInstance->getIntermediateWeightsBuffer(solverIndex);
        auto outputWeightsBuffer = outputInstance->getOutputWeightsBuffer(solverIndex);
        calculate(solverIndex, rawControls, inputBuffer, intermediateWeightsBuffer, outputWeightsBuffer);
    }

    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override {
        archive(lods);
        std::size_t solverCount;
        archive(solverCount);
        solvers.reserve(solverCount);

        auto memRes = solvers.get_allocator().getMemoryResource();

        for (std::size_t i = 0u; i < solverCount; ++i) {
            dna::RBFSolverType solverType;
            archive(solverType);
            if (solverType == dna::RBFSolverType::Additive) {
                solvers.emplace_back(UniqueInstance<AdditiveRBFSolver, RBFSolver>::with(memRes).create(memRes));
            } else if (solverType == dna::RBFSolverType::Interpolative) {
                solvers.push_back(UniqueInstance<InterpolativeRBFSolver, RBFSolver>::with(memRes).create(memRes));
            }
            solvers[i]->load(archive);
        }
        archive(solverRawControlInputIndices);
        archive(solverRawControlOutputIndices);
        archive(solverPoseIndices);
        archive(poseInputControlIndices);
        archive(poseOutputControlIndices);
        archive(poseOutputControlWeights);
        archive(inputCountPerSolver);
        archive(targetCountPerSolver);
    }

    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override {
        archive(lods);
        std::size_t solverCount = solvers.size();
        archive(solverCount);
        for (auto& solver : solvers) {
            dna::RBFSolverType solverType = solver->getSolverType();
            archive(solverType);
            solver->save(archive);
        }
        archive(solverRawControlInputIndices);
        archive(solverRawControlOutputIndices);
        archive(solverPoseIndices);
        archive(poseInputControlIndices);
        archive(poseOutputControlIndices);
        archive(poseOutputControlWeights);
        archive(inputCountPerSolver);
        archive(targetCountPerSolver);
    }

private:
    LODSpec<std::uint16_t> lods;
    SolverVectorType solvers;
    Matrix<std::uint16_t> solverRawControlInputIndices;
    Matrix<std::uint16_t> solverRawControlOutputIndices;
    Matrix<std::uint16_t> solverPoseIndices;
    Matrix<std::uint16_t> poseInputControlIndices;
    Matrix<std::uint16_t> poseOutputControlIndices;
    Matrix<float> poseOutputControlWeights;
    Vector<std::uint16_t> inputCountPerSolver;
    Vector<std::uint16_t> targetCountPerSolver;
    OutputInstance::Factory instanceFactory;
};

}  // namespace cpu

}  // namespace rbf

}  // namespace rl4
