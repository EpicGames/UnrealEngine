// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/RBFBehaviorEvaluator.h"
#include "riglogic/rbf/cpu/CPURBFBehaviorEvaluator.h"
#include "riglogic/rbf/cpu/CPURBFBehaviorOutputInstance.h"
#include "riglogic/rbf/cpu/RBFSolver.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/types/LODSpec.h"
#include "riglogic/types/bpcm/Optimizer.h"
#include "riglogic/utils/Extd.h"

#include <tdm/Ang.h>

namespace rl4 {

namespace rbf {

namespace cpu {

template<typename T, typename TF256, typename TF128>
class Factory {
public:
    static RBFBehaviorEvaluator::Pointer create(RigMetadata* meta, const dna::Reader* reader, MemoryResource* memRes) {
        RL_UNUSED(meta);
        Vector<RBFSolver::Pointer> solvers{memRes};
        Vector<std::uint16_t> inputCountPerSolver{memRes};
        Vector<std::uint16_t> targetCountPerSolver{memRes};
        Matrix<std::uint16_t> solverRawControlInputIndices{memRes};
        Matrix<std::uint16_t> solverRawControlOutputIndices{memRes};
        Matrix<std::uint16_t> solverPoseIndices{memRes};
        Matrix<std::uint16_t> poseInputControlIndices{memRes};
        Matrix<std::uint16_t> poseOutputControlIndices{memRes};
        Matrix<float> poseOutputControlWeights{memRes};

        auto instanceFactory = [](ConstArrayView<std::uint16_t> inputCounts,
                                  ConstArrayView<std::uint16_t> targetCounts,
                                  MemoryResource* instanceMemRes) {
            using OutputInstancePointer = UniqueInstance<OutputInstance, RBFBehaviorOutputInstance>;
            return OutputInstancePointer::with(instanceMemRes).create(inputCounts, targetCounts, instanceMemRes);
        };
        auto factory = UniqueInstance<Evaluator<T, TF256, TF128>, RBFBehaviorEvaluator>::with(memRes);

        if (reader == nullptr) {
            return factory.create(LODSpec<std::uint16_t>{memRes},
                                  std::move(solvers),
                                  std::move(solverRawControlInputIndices),
                                  std::move(solverRawControlOutputIndices),
                                  std::move(solverPoseIndices),
                                  std::move(poseInputControlIndices),
                                  std::move(poseOutputControlIndices),
                                  std::move(poseOutputControlWeights),
                                  std::move(inputCountPerSolver),
                                  std::move(targetCountPerSolver),
                                  std::move(instanceFactory));
        }

        auto lods = computeLODs(reader, memRes);
        solvers.reserve(lods.count);
        inputCountPerSolver.reserve(lods.count);
        targetCountPerSolver.reserve(lods.count);
        solverRawControlInputIndices.resize(lods.count);
        solverRawControlOutputIndices.resize(lods.count);
        solverPoseIndices.resize(lods.count);

        RBFSolverRecipe recipe{};
        Vector<float> targetScales{memRes};
        const auto rotationUnit = reader->getRotationUnit();
        std::function<float(float)> angConv;
        if (rotationUnit == dna::RotationUnit::degrees) {
            angConv = [](float angle) { return tdm::frad{tdm::fdeg{angle}}.value; };
        } else {
            angConv = [](float angle) { return tdm::frad{angle}.value; };
        }
        for (std::uint16_t solverIndex = {}; solverIndex < lods.count; ++solverIndex) {
            recipe.solverType = reader->getRBFSolverType(solverIndex);
            recipe.distanceMethod = reader->getRBFSolverDistanceMethod(solverIndex);
            recipe.weightFunction = reader->getRBFSolverFunctionType(solverIndex);
            recipe.normalizeMethod = reader->getRBFSolverNormalizeMethod(solverIndex);
            recipe.twistAxis = reader->getRBFSolverTwistAxis(solverIndex);
            recipe.isAutomaticRadius = reader->getRBFSolverAutomaticRadius(solverIndex) == dna::AutomaticRadius::On;
            recipe.radius = angConv(reader->getRBFSolverRadius(solverIndex));
            recipe.weightThreshold = reader->getRBFSolverWeightThreshold(solverIndex);

            const auto rawControlIndices = reader->getRBFSolverRawControlIndices(solverIndex);
            solverRawControlInputIndices[solverIndex].assign(rawControlIndices.begin(), rawControlIndices.end());
            const auto solverInputCount = static_cast<std::uint16_t>(rawControlIndices.size());
            inputCountPerSolver.push_back(solverInputCount);
            recipe.rawControlCount = solverInputCount;

            const auto poseIndices = reader->getRBFSolverPoseIndices(solverIndex);
            solverPoseIndices[solverIndex].assign(poseIndices.begin(), poseIndices.end());

            auto& solverOutputIndices = solverRawControlOutputIndices[solverIndex];
            solverOutputIndices.reserve(poseIndices.size());
            for (const auto poseIndex : poseIndices) {
                const auto poseOutputIndices = reader->getRBFPoseOutputControlIndices(poseIndex);
                solverOutputIndices.insert(solverOutputIndices.end(), poseOutputIndices.begin(), poseOutputIndices.end());
            }
            std::sort(solverOutputIndices.begin(), solverOutputIndices.end());
            solverOutputIndices.erase(std::unique(solverOutputIndices.begin(), solverOutputIndices.end()),
                                      solverOutputIndices.end());

            const auto targetCount = static_cast<std::uint16_t>(poseIndices.size());
            targetCountPerSolver.push_back(targetCount);

            targetScales.resize(targetCount);
            for (std::uint16_t i = 0u; i < targetCount; ++i) {
                targetScales[i] = reader->getRBFPoseScale(poseIndices[i]);
            }
            recipe.targetValues = reader->getRBFSolverRawControlValues(solverIndex);
            recipe.targetScales = targetScales;

            auto solver = RBFSolver::create(recipe, memRes);
            solvers.emplace_back(std::move(solver));
        }
        const auto poseCount = reader->getRBFPoseCount();
        poseInputControlIndices.resize(poseCount);
        poseOutputControlIndices.resize(poseCount);
        poseOutputControlWeights.resize(poseCount);

        for (std::uint16_t poseIndex = {}; poseIndex < poseCount; ++poseIndex) {
            const auto inputControlIndices = reader->getRBFPoseInputControlIndices(poseIndex);
            const auto outputControlIndices = reader->getRBFPoseOutputControlIndices(poseIndex);
            const auto outputControlWeights = reader->getRBFPoseOutputControlWeights(poseIndex);
            poseInputControlIndices[poseIndex].assign(inputControlIndices.begin(), inputControlIndices.end());
            poseOutputControlIndices[poseIndex].assign(outputControlIndices.begin(), outputControlIndices.end());
            poseOutputControlWeights[poseIndex].assign(outputControlWeights.begin(), outputControlWeights.end());
        }

        return factory.create(std::move(lods),
                              std::move(solvers),
                              std::move(solverRawControlInputIndices),
                              std::move(solverRawControlOutputIndices),
                              std::move(solverPoseIndices),
                              std::move(poseInputControlIndices),
                              std::move(poseOutputControlIndices),
                              std::move(poseOutputControlWeights),
                              std::move(inputCountPerSolver),
                              std::move(targetCountPerSolver),
                              std::move(instanceFactory));
    }

private:
    static LODSpec<std::uint16_t> computeLODs(const dna::RBFBehaviorReader* reader, MemoryResource* memRes) {
        LODSpec<std::uint16_t> lods{memRes};
        const auto lodCount = reader->getLODCount();
        lods.indicesPerLOD.resize(lodCount);
        for (std::uint16_t lod = {}; lod < lodCount; ++lod) {
            const auto solverIndices = reader->getRBFSolverIndicesForLOD(lod);
            lods.indicesPerLOD[lod].assign(solverIndices.begin(), solverIndices.end());
        }
        lods.count = reader->getRBFSolverCount();
        return lods;
    }
};

}  // namespace cpu

}  // namespace rbf

}  // namespace rl4
