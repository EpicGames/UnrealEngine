// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/cpu/CPURBFBehaviorOutputInstance.h"

#include "riglogic/utils/Macros.h"

namespace rl4 {

namespace rbf {

namespace cpu {

OutputInstance::OutputInstance(ConstArrayView<std::uint16_t> inputCountPerSolver,
                               ConstArrayView<std::uint16_t> targetCountPerSolver,
                               MemoryResource* memRes) :
    inputBuffers{memRes},
    intermediateWeightsBuffers{memRes},
    outputWeightsBuffers{memRes} {

    assert(inputCountPerSolver.size() == targetCountPerSolver.size());
    const auto solverCount = static_cast<std::uint16_t>(targetCountPerSolver.size());
    inputBuffers.resize(solverCount);
    intermediateWeightsBuffers.resize(solverCount);
    outputWeightsBuffers.resize(solverCount);
    for (std::uint16_t solverIndex = {}; solverIndex < solverCount; ++solverIndex) {
        inputBuffers[solverIndex].resize(inputCountPerSolver[solverIndex]);
        intermediateWeightsBuffers[solverIndex].resize(targetCountPerSolver[solverIndex]);
        outputWeightsBuffers[solverIndex].resize(targetCountPerSolver[solverIndex]);
    }
}

ArrayView<float> OutputInstance::getInputBuffer(std::uint16_t solverIndex) {
    return inputBuffers[solverIndex];
}

ArrayView<float> OutputInstance::getIntermediateWeightsBuffer(std::uint16_t solverIndex) {
    return intermediateWeightsBuffers[solverIndex];
}

ArrayView<float> OutputInstance::getOutputWeightsBuffer(std::uint16_t solverIndex) {
    return outputWeightsBuffers[solverIndex];
}

}  // namespace cpu

}  // namespace rbf

}  // namespace rl4
