// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/RBFBehaviorOutputInstance.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace rbf {

namespace cpu {

class OutputInstance : public RBFBehaviorOutputInstance {
public:
    using Factory = std::function<Pointer(ConstArrayView<std::uint16_t>, ConstArrayView<std::uint16_t>, MemoryResource*)>;

public:
    OutputInstance(ConstArrayView<std::uint16_t> inputCountPerSolver,
                   ConstArrayView<std::uint16_t> targetCountPerSolver,
                   MemoryResource* memRes);
    ArrayView<float> getInputBuffer(std::uint16_t solverIndex) override;
    ArrayView<float> getIntermediateWeightsBuffer(std::uint16_t solverIndex) override;
    ArrayView<float> getOutputWeightsBuffer(std::uint16_t solverIndex) override;

private:
    Vector<AlignedVector<float>> inputBuffers;
    Vector<AlignedVector<float>> intermediateWeightsBuffers;
    Vector<AlignedVector<float>> outputWeightsBuffers;
};

}  // namespace cpu

}  // namespace rbf

}  // namespace rl4
