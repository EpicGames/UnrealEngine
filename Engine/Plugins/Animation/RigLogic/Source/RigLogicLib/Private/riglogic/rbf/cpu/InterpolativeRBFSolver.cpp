// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/cpu/InterpolativeRBFSolver.h"

#include "riglogic/rbf/cpu/RBFSolver.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Extd.h"

#include <tdm/Computations.h>
#include <tdm/Mat.h>
#include <tdm/Types.h>

namespace rl4 {

namespace {

inline bool decompose(Matrix<float>& a, Vector<std::size_t>& permute) {
    constexpr float absmin = 1.0e-20f;

    Vector<float> scale{a.size(), {}, permute.get_allocator()};

    for (std::size_t i = {}; i < a.size(); ++i) {
        float rowMax = {};
        for (std::size_t j = {}; j < a.size(); ++j) {
            rowMax = std::max(rowMax, std::abs(a[i][j]));
        }

        if (rowMax == 0.0f) {
            return false;
        }
        scale[i] = 1.0f / rowMax;
    }

    for (std::size_t j = {}; j < a.size(); ++j) {
        for (std::size_t i = {}; i < j; ++i) {
            float sum = a[i][j];
            for (std::size_t k = {}; k < i; ++k) {
                sum -= a[i][k] * a[k][j];
            }
            a[i][j] = sum;
        }

        std::size_t iMax = {};
        float colMax = {};
        for (std::size_t i = j; i < a.size(); ++i) {
            float sum = a[i][j];
            for (std::size_t k = {}; k < j; ++k) {
                sum -= a[i][k] * a[k][j];
            }
            a[i][j] = sum;

            const float temp = scale[i] * std::abs(sum);
            if (temp >= colMax) {
                colMax = temp;
                iMax = i;
            }
        }

        if (j != iMax) {
            for (std::size_t k = {}; k < a.size(); ++k) {
                std::swap(a[iMax][k], a[j][k]);
            }
            scale[iMax] = scale[j];
        }

        permute[j] = iMax;

        if (a[j][j] == 0.0f) {
            a[j][j] = absmin;
        }

        if (j != (a.size() - 1)) {
            const float temp = 1.0f / a[j][j];
            for (std::size_t i = j + 1; i < a.size(); ++i) {
                a[i][j] *= temp;
            }
        }
    }

    return true;
}

inline void substitute(const Matrix<float>& a, const Vector<std::size_t>& permute, Vector<float>& b) {
    std::size_t ii = {};
    for (std::size_t i = {}; i < a.size(); ++i) {
        const std::size_t ip = permute[i];
        float sum = b[ip];
        b[ip] = b[i];
        if (ii) {
            for (std::size_t j = (ii - 1); j < i; ++j) {
                sum -= a[i][j] * b[j];
            }
        } else if (sum != 0.0f) {
            ii = i + 1;
        }
        b[i] = sum;
    }

    for (std::size_t ipo = a.size(); ipo > 0; --ipo) {
        const std::size_t i = ipo - 1;
        float sum = b[i];
        for (std::size_t j = i + 1; j < a.size(); ++j) {
            sum -= a[i][j] * b[j];
        }
        b[i] = sum / a[i][i];
    }
}

inline void invert(Matrix<float>& m) {
    Vector<std::size_t> permute{m.size(), {}, m.get_allocator()};
    if (!decompose(m, permute)) {
        return;
    }

    Matrix<float> inv{m.size(), Vector<float>{m.size(), {}, m.get_allocator()}, m.get_allocator()};
    for (std::size_t j = {}; j < m.size(); ++j) {
        Vector<float> col{m.size(), {}, m.get_allocator()};
        col[j] = 1.0f;
        substitute(m, permute, col);
        for (std::size_t i = {}; i < m.size(); ++i) {
            inv[i][j] = col[i];
        }
    }
    m = std::move(inv);
}

}  // namespace

InterpolativeRBFSolver::InterpolativeRBFSolver(MemoryResource* memRes) :
    RBFSolver(memRes),
    coefficients{memRes} {
}

InterpolativeRBFSolver::InterpolativeRBFSolver(const RBFSolverRecipe& recipe, MemoryResource* memRes) :
    RBFSolver(recipe, memRes),
    coefficients{memRes} {
    const auto targetCount = targets.size();
    coefficients = Matrix<float>{targetCount, Vector<float>{targetCount, 0.0f, memRes}, memRes};
    // This can also be optimized, we do not need the actual matrix what we are looking for the inverse matrix
    // We need to include the diagonal itself, since we can't guarantee that the weight
    // function returns 1.0 for nodes of the same coordinates.

    for (std::size_t i = {}; i < targetCount; ++i) {
        // matrix is symmetrical so we only need to calculate right side of the matrix and we ll copy it onto the left
        ArrayView<float> outputWeights{coefficients[i].data() + i, coefficients[i].size() - i};
        const auto& curTarget = targets[i];
        ConstArrayView<AlignedVector<float>> targetView{targets.data() + i, targets.size() - i};
        if (targetView.size() > 0ul) {
            getDistanceWeight(targetView, curTarget, outputWeights, radius);
        }
        // copying
        for (std::size_t j = i + 1ul; j < targetCount; ++j) {
            coefficients[j][i] = coefficients[i][j];
        }
    }
    // there are optimized ways of getting inverse of symmetrical matrix
    // but since this is not in a hot path rather one time call i am not sure if it is
    // worth it
    invert(coefficients);
}

RBFSolverType InterpolativeRBFSolver::getSolverType() const {
    return RBFSolverType::Interpolative;
}

void InterpolativeRBFSolver::solve(ArrayView<float> input,
                                   ArrayView<float> intermediateWeights,
                                   ArrayView<float> outputWeights) const {
    convertInput(input);
    const std::size_t targetSize = targets.size();
    getDistanceWeight(targets, input, intermediateWeights, radius);

    for (std::size_t i = {}; i < targetSize; ++i) {
        float weight = 0.0f;
        const auto& targetCoeff = coefficients[i];
        for (std::size_t j = {}; j < targetSize; ++j) {
            weight += targetCoeff[j] * intermediateWeights[j];  // Vectorize TODO
        }
        outputWeights[i] = extd::clamp(weight, 0.0f, 1.0f);
    }
    normalizeAndCutOff(outputWeights);
}

void InterpolativeRBFSolver::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    RBFSolver::load(archive);
    archive(coefficients);
}

void InterpolativeRBFSolver::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    RBFSolver::save(archive);
    archive(coefficients);
}

const Matrix<float>& InterpolativeRBFSolver::getCoefficients() const {
    return coefficients;
}

}  // namespace rl4
