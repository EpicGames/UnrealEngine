// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/rbf/cpu/RBFSolver.h"

namespace rl4 {

class InterpolativeRBFSolver : public RBFSolver {
public:
    explicit InterpolativeRBFSolver(const RBFSolverRecipe& recipe, MemoryResource* memRes);
    explicit InterpolativeRBFSolver(MemoryResource* memRes);

    RBFSolverType getSolverType() const override;
    void solve(ArrayView<float> input, ArrayView<float> intermediateWeights, ArrayView<float> outputWeights) const override;
    void load(terse::BinaryInputArchive<BoundedIOStream>& archive) override;
    void save(terse::BinaryOutputArchive<BoundedIOStream>& archive) override;

    const Matrix<float>& getCoefficients() const;

private:
    Matrix<float> coefficients;
};

}  // namespace rl4
