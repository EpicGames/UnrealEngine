// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/rbf/cpu/RBFSolver.h"

#include "riglogic/rbf/cpu/AdditiveRBFSolver.h"
#include "riglogic/rbf/cpu/InterpolativeRBFSolver.h"
#include "riglogic/types/Aliases.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cmath>
#include <cstddef>
#include <cstdint>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace {

template<std::size_t Exponent>
float pow(float base);

template<>
inline float pow<1u>(float base) {
    return base;
}

template<std::size_t Exponent>
inline float pow(float base) {
    return base * pow<Exponent - 1u>(base);
}

inline float dotProduct(ConstArrayView<float> qA, ConstArrayView<float> qB) {
    return qA[0] * qB[0] + qA[1] * qB[1] + qA[2] * qB[2] + qA[3] * qB[3];
}

inline float getArcLength(ConstArrayView<float> qA, ConstArrayView<float> qB) {
    assert(qA.size() == qB.size() && qB.size() == 4u);
    float dotSqr = pow<2>(dotProduct(qA, qB));
    if (dotSqr > 1.0f) {
        return 0.0f;
    }
    return std::acos((2 * dotSqr) - 1.0f);
}

template<TwistAxis TTwistAxis>
inline void getSwing(ArrayView<float> q);

template<>
inline void getSwing<TwistAxis::X>(ArrayView<float> q) {
    const float x = q[0];
    const float y = q[1];
    const float z = q[2];
    const float w = q[3];
    q[3] = -std::sqrt(x * x + w * w);
    q[2] = (w * z + x * y) / q[3];
    q[1] = (w * y - x * z) / q[3];
    q[0] = 0.0f;
}

template<>
inline void getSwing<TwistAxis::Y>(ArrayView<float> q) {
    const float x = q[0];
    const float y = q[1];
    const float z = q[2];
    const float w = q[3];
    q[3] = -std::sqrt(y * y + w * w);
    q[2] = (w * z - y * x) / q[3];
    q[1] = 0.0f;
    q[0] = (w * x + y * z) / q[3];
}

template<>
inline void getSwing<TwistAxis::Z>(ArrayView<float> q) {
    const float x = q[0];
    const float y = q[1];
    const float z = q[2];
    const float w = q[3];
    q[3] = -std::sqrt(z * z + w * w);
    q[2] = 0.0f;
    q[1] = (w * y + z * x) / q[3];
    q[0] = (w * x - z * y) / q[3];
}

template<TwistAxis TTwistAxis>
inline void getTwist(ArrayView<float> q) {
    constexpr auto twistIndex = static_cast<std::uint16_t>(TTwistAxis);
    constexpr auto notTwistIndex0 = (twistIndex + 1u) % 3u;
    constexpr auto notTwistIndex1 = (twistIndex + 2u) % 3u;
    q[notTwistIndex0] = 0.0f;
    q[notTwistIndex1] = 0.0f;
    // normalize
    float magnitude = std::sqrt(q[twistIndex] * q[twistIndex] + q[3] * q[3]);
    q[twistIndex] /= magnitude;
    q[3] /= magnitude;
}

template<RBFDistanceMethod TDistanceMethod>
struct DistanceMethodFunctor;

template<>
struct DistanceMethodFunctor<RBFDistanceMethod::Euclidean> {
    static inline void getDistance(ConstArrayView<AlignedVector<float>> targets,
                                   ConstArrayView<float> rawControlValues,
                                   ArrayView<float> intermediateWeights) {
        for (std::size_t ti = 0u; ti < targets.size(); ++ti) {
            auto target = ConstArrayView<float>{targets[ti]};
            float sumSquaredDiff = 0.0f;
            for (std::size_t i = 0u; i < rawControlValues.size(); ++i) {
                sumSquaredDiff += pow<2u>(target[i] - rawControlValues[i]);
            }
            intermediateWeights[ti] = std::sqrt(sumSquaredDiff);
        }
    }
};

template<>
struct DistanceMethodFunctor<RBFDistanceMethod::Quaternion> {
    static inline void getDistance(ConstArrayView<AlignedVector<float>> targets,
                                   ConstArrayView<float> rawControlValues,
                                   ArrayView<float> intermediateWeights) {

        auto quaternionCount = static_cast<std::uint16_t>(rawControlValues.size() / 4u);
        for (std::size_t ti = 0u; ti < targets.size(); ++ti) {
            auto target = ConstArrayView<float>{targets[ti]};
            assert(target.size() % 4u == 0);
            float distance = 0.0f;
            // quaternion count is same for each iteration TODO
            for (std::size_t i = 0u; i < quaternionCount; ++i) {
                auto aQ = target.subview(i * 4u, 4u);
                auto bQ = rawControlValues.subview(i * 4u, 4u);
                // quaternions need to be normalized for this
                distance += pow<2u>(getArcLength(aQ, bQ));
            }
            intermediateWeights[ti] = std::sqrt(distance);
        }
    }
};

template<RBFFunctionType TFunctionType>
struct WeightMethodFunctor;

template<>
struct WeightMethodFunctor<RBFFunctionType::Linear> {
    static inline float getWeight(float value, float kernelWidth) {
        value = 1.0f - (value / kernelWidth);
        return value > 0.0f ? value : 0.0f;
    }
};

template<>
struct WeightMethodFunctor<RBFFunctionType::Cubic> {
    static inline float getWeight(float value, float kernelWidth) {
        value = 1.0f - pow<3>(value / kernelWidth);
        return value > 0.0f ? value : 0.0f;
    }
};

template<>
struct WeightMethodFunctor<RBFFunctionType::Quintic> {
    static inline float getWeight(float value, float kernelWidth) {
        value = 1.0f - pow<5>(value / kernelWidth);
        return value > 0.0f ? value : 0.0f;
    }
};

template<>
struct WeightMethodFunctor<RBFFunctionType::Gaussian> {
    static inline float getWeight(float value, float kernelWidth) {
        return std::exp(-value / pow<2>(kernelWidth));
    }
};

template<>
struct WeightMethodFunctor<RBFFunctionType::Exponential> {
    static inline float getWeight(float value, float kernelWidth) {
        return std::exp(-2.0f * value / kernelWidth);
    }
};

template<RBFDistanceMethod TDistanceMethod>
using D = DistanceMethodFunctor<TDistanceMethod>;

template<RBFFunctionType TFunctionType>
using W = WeightMethodFunctor<TFunctionType>;

using DistanceWeightFun = RBFSolver::DistanceWeightFun;
using InputConvertFun = RBFSolver::InputConvertFun;
using DistanceFun = decltype(&D<RBFDistanceMethod::Euclidean>::getDistance);

DistanceFun getDistanceFun(RBFDistanceMethod distanceMethod) {
    if (distanceMethod == RBFDistanceMethod::Euclidean) {
        return D<RBFDistanceMethod::Euclidean>::getDistance;
    }
    return D<RBFDistanceMethod::Quaternion>::getDistance;
}

template<typename TDistanceFunctionProvider, typename TWeightFunctionProvider>
struct DistanceWeightFunctor {
    void inline operator()(ConstArrayView<AlignedVector<float>> targets,
                           ConstArrayView<float> input,
                           ArrayView<float> intermediateWeights,
                           float kernelWidth) {
        TDistanceFunctionProvider::getDistance(targets, input, intermediateWeights);
        for (std::size_t ti = 0u; ti < targets.size(); ++ti) {
            intermediateWeights[ti] = TWeightFunctionProvider::getWeight(intermediateWeights[ti], kernelWidth);
        }
    }
};

DistanceWeightFun getDistanceWeightFun(RBFFunctionType weightMethod, RBFDistanceMethod distanceMethod) {
    constexpr auto Gaussian = RBFFunctionType::Gaussian;
    constexpr auto Cubic = RBFFunctionType::Cubic;
    constexpr auto Exponential = RBFFunctionType::Exponential;
    constexpr auto Linear = RBFFunctionType::Linear;
    constexpr auto Quintic = RBFFunctionType::Quintic;

    constexpr auto Euclidean = RBFDistanceMethod::Euclidean;
    constexpr auto Quaternion = RBFDistanceMethod::Quaternion;

    if (distanceMethod == Euclidean) {
        switch (weightMethod) {
        case Gaussian:
            return DistanceWeightFunctor<D<Euclidean>, W<Gaussian>>();
        case Cubic:
            return DistanceWeightFunctor<D<Euclidean>, W<Cubic>>();
        case Exponential:
            return DistanceWeightFunctor<D<Euclidean>, W<Exponential>>();
        case Linear:
            return DistanceWeightFunctor<D<Euclidean>, W<Linear>>();
        case Quintic:
            return DistanceWeightFunctor<D<Euclidean>, W<Quintic>>();
        }
    }

    switch (weightMethod) {
    case Gaussian:
        return DistanceWeightFunctor<D<Quaternion>, W<Gaussian>>();
    case Cubic:
        return DistanceWeightFunctor<D<Quaternion>, W<Cubic>>();
    case Exponential:
        return DistanceWeightFunctor<D<Quaternion>, W<Exponential>>();
    case Linear:
        return DistanceWeightFunctor<D<Quaternion>, W<Linear>>();
    case Quintic:
        return DistanceWeightFunctor<D<Quaternion>, W<Quintic>>();
    }

    assert(false);  // Should not reach this
    return nullptr;
}

template<TwistAxis TTwistAxis>
InputConvertFun getInputConvertFun(RBFDistanceMethod distanceMethod) {
    switch (distanceMethod) {
    case RBFDistanceMethod::Quaternion:
    case RBFDistanceMethod::Euclidean:
        return [](ArrayView<float> input) { return input; };
    case RBFDistanceMethod::TwistAngle:
        return [](ArrayView<float> input) {
            assert(input.size() % 4 == 0);
            for (std::size_t qi = {}; qi < input.size(); qi += 4ul) {
                getTwist<TTwistAxis>(input.subview(qi, 4ul));
            }
        };
    default:
    case RBFDistanceMethod::SwingAngle:
        return [](ArrayView<float> input) {
            assert(input.size() % 4 == 0);
            for (std::size_t qi = {}; qi < input.size(); qi += 4ul) {
                getSwing<TTwistAxis>(input.subview(qi, 4ul));
            }
        };
    }
}

InputConvertFun getInputConvertFun(RBFDistanceMethod distanceMethod, TwistAxis axis) {
    switch (axis) {
    default:
    case TwistAxis::X:
        return getInputConvertFun<TwistAxis::X>(distanceMethod);
    case TwistAxis::Y:
        return getInputConvertFun<TwistAxis::Y>(distanceMethod);
    case TwistAxis::Z:
        return getInputConvertFun<TwistAxis::Z>(distanceMethod);
    }
}

}  // namespace

RBFSolver::RBFSolver(MemoryResource* memRes) :
    targets{memRes},
    targetScale{memRes},
    getDistanceWeight{},
    convertInput{},
    radius{},
    weightThreshold{},
    distanceMethod{},
    weightFunction{},
    normalizeMethod{},
    twistAxis{} {
}

RBFSolver::RBFSolver(const RBFSolverRecipe& recipe, MemoryResource* memRes) :
    targets{memRes},
    targetScale{recipe.targetScales.begin(), recipe.targetScales.end(), memRes},
    getDistanceWeight{getDistanceWeightFun(recipe.weightFunction, recipe.distanceMethod)},
    convertInput{getInputConvertFun(recipe.distanceMethod, recipe.twistAxis)},
    radius{recipe.radius},
    weightThreshold{recipe.weightThreshold},
    distanceMethod{recipe.distanceMethod},
    weightFunction{recipe.weightFunction},
    normalizeMethod{recipe.normalizeMethod},
    twistAxis{recipe.twistAxis} {

    assert(recipe.targetValues.size() % recipe.rawControlCount == 0u);
    const auto targetCount = static_cast<std::uint16_t>(recipe.targetValues.size() / recipe.rawControlCount);
    targets.resize(targetCount);

    for (std::uint16_t ti = 0u; ti < targetCount; ti++) {
        const auto offset = static_cast<std::size_t>(ti) * static_cast<std::size_t>(recipe.rawControlCount);
        const auto targetValues = recipe.targetValues.subview(offset, recipe.rawControlCount);
        targets[ti].assign(targetValues.begin(), targetValues.end());
        convertInput(targets[ti]);
    }

    if (recipe.isAutomaticRadius) {
        auto getDistance = getDistanceFun(recipe.distanceMethod);
        ConstArrayView<AlignedVector<float>> targetsView{targets};
        Vector<float> bufferVec{targets.size(), 0.0f, memRes};

        float sumDistance = 0.0f;
        for (std::size_t i = {}; i < targetCount; ++i) {
            const std::size_t offset = i + 1ul;
            const std::size_t count = targetCount - offset;
            ArrayView<float> buffer{bufferVec.data() + offset, count};
            getDistance(targetsView.subview(offset, count), targets[i], buffer);
            sumDistance = std::accumulate(buffer.begin(), buffer.end(), sumDistance);
        }
        const float distancesCount = static_cast<float>(targetCount) * static_cast<float>(targetCount - 1ul) / 2.0f;
        radius = sumDistance / distancesCount;
    }
}

void RBFSolver::load(terse::BinaryInputArchive<BoundedIOStream>& archive) {
    archive(targets);
    archive(targetScale);
    archive(radius);
    archive(weightThreshold);
    archive(distanceMethod);
    archive(weightFunction);
    archive(normalizeMethod);
    archive(twistAxis);
    getDistanceWeight = getDistanceWeightFun(weightFunction, distanceMethod);
    convertInput = getInputConvertFun(distanceMethod, twistAxis);
}

void RBFSolver::save(terse::BinaryOutputArchive<BoundedIOStream>& archive) {
    archive(targets);
    archive(targetScale);
    archive(radius);
    archive(weightThreshold);
    archive(distanceMethod);
    archive(weightFunction);
    archive(normalizeMethod);
    archive(twistAxis);
}

void RBFSolver::normalizeAndCutOff(ArrayView<float> outputWeights) const {
    float sumWeight = 0.0f;
    for (float weight : outputWeights) {
        sumWeight += weight;
    }
    float normalizationRatio = 1.0f;
    if ((sumWeight > 1.0f) || (normalizeMethod == RBFNormalizeMethod::AlwaysNormalize)) {
        normalizationRatio = 1.0f / sumWeight;
    }
    // vectorize TODO
    for (std::uint16_t i = 0u; i < outputWeights.size(); i++) {
        float weight = outputWeights[i] * normalizationRatio * targetScale[i];
        if (weight > weightThreshold) {
            outputWeights[i] = weight;
        } else {
            outputWeights[i] = 0.0f;
        }
    }
}

ConstArrayView<AlignedVector<float>> RBFSolver::getTargets() const {
    return targets;
}

ConstArrayView<float> RBFSolver::getTargetScales() const {
    return targetScale;
}

float RBFSolver::getRadius() const {
    return radius;
}

float RBFSolver::getWeightThreshold() const {
    return weightThreshold;
}

RBFDistanceMethod RBFSolver::getDistanceMethod() const {
    return distanceMethod;
}

RBFFunctionType RBFSolver::getWeightFunction() const {
    return weightFunction;
}

RBFNormalizeMethod RBFSolver::getNormalizeMethod() const {
    return normalizeMethod;
}

TwistAxis RBFSolver::getTwistAxis() const {
    return twistAxis;
}

RBFSolver::~RBFSolver() = default;

RBFSolver::Pointer RBFSolver::create(RBFSolverRecipe recipe, MemoryResource* memRes) {
    const auto solverType = recipe.solverType;

    if (solverType == RBFSolverType::Interpolative) {
        return UniqueInstance<InterpolativeRBFSolver, RBFSolver>::with(memRes).create(recipe, memRes);
    }
    return UniqueInstance<AdditiveRBFSolver, RBFSolver>::with(memRes).create(recipe, memRes);
}

}  // namespace rl4
