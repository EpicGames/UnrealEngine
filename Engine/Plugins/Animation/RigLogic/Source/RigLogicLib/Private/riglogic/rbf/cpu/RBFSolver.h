// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

using dna::RBFDistanceMethod;
using dna::RBFFunctionType;
using dna::RBFNormalizeMethod;
using dna::RBFSolverType;
using dna::TwistAxis;

struct RBFSolverRecipe {
    RBFSolverType solverType;
    RBFDistanceMethod distanceMethod;
    RBFFunctionType weightFunction;
    RBFNormalizeMethod normalizeMethod;
    TwistAxis twistAxis;
    bool isAutomaticRadius;
    float radius;
    float weightThreshold;
    std::uint16_t rawControlCount;
    ConstArrayView<float> targetValues;
    ConstArrayView<float> targetScales;
};

class RBFSolver {
public:
    using Pointer = UniqueInstance<RBFSolver>::PointerType;
    using DistanceWeightFun =
        std::function<void(ConstArrayView<AlignedVector<float>>, ConstArrayView<float>, ArrayView<float>, float)>;
    using InputConvertFun = std::function<void(ArrayView<float>)>;

public:
    static Pointer create(RBFSolverRecipe recipe, MemoryResource* memRes);

public:
    explicit RBFSolver(MemoryResource* memRes);
    RBFSolver(const RBFSolverRecipe& recipe, MemoryResource* memRes);

    virtual ~RBFSolver();

    virtual RBFSolverType getSolverType() const = 0;
    virtual void solve(ArrayView<float> input, ArrayView<float> intermediateWeights, ArrayView<float> outputWeights) const = 0;

    virtual void load(terse::BinaryInputArchive<BoundedIOStream>& archive);
    virtual void save(terse::BinaryOutputArchive<BoundedIOStream>& archive);

    ConstArrayView<AlignedVector<float>> getTargets() const;
    ConstArrayView<float> getTargetScales() const;
    float getRadius() const;
    float getWeightThreshold() const;
    RBFDistanceMethod getDistanceMethod() const;
    RBFFunctionType getWeightFunction() const;
    RBFNormalizeMethod getNormalizeMethod() const;
    TwistAxis getTwistAxis() const;

protected:
    void normalizeAndCutOff(ArrayView<float> outputWeights) const;

protected:
    Vector<AlignedVector<float>> targets;
    Vector<float> targetScale;
    DistanceWeightFun getDistanceWeight;
    InputConvertFun convertInput;
    float radius;
    float weightThreshold;
    RBFDistanceMethod distanceMethod;
    RBFFunctionType weightFunction;
    RBFNormalizeMethod normalizeMethod;
    TwistAxis twistAxis;
};

}  // namespace rl4
