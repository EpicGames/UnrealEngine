// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/riglogic/Configuration.h"

namespace rl4 {

template<class Archive>
void serialize(Archive& archive, Configuration& config) {
    archive(config.calculationType,
            config.loadJoints,
            config.loadBlendShapes,
            config.loadAnimatedMaps,
            config.loadMachineLearnedBehavior,
            config.loadRBFBehavior,
            config.loadTwistSwingBehavior,
            config.translationType,
            config.rotationType,
            config.scaleType,
            config.translationPruningThreshold,
            config.rotationPruningThreshold,
            config.scalePruningThreshold);
}

}  // namespace rl4
