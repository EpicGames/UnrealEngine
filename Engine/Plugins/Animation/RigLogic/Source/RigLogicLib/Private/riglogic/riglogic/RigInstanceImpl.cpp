// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/riglogic/RigInstanceImpl.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/riglogic/RigLogicImpl.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/types/Aliases.h"
#include "riglogic/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstdint>
#include <numeric>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

namespace {

inline std::uint16_t getMaxLODLevel(std::uint16_t lodCount) {
    return (lodCount > 0u ? static_cast<std::uint16_t>(lodCount - 1) : static_cast<std::uint16_t>(0));
}

}  // namespace

RigInstance* RigInstance::create(RigLogic* rigLogic, MemoryResource* memRes) {
    auto pRigLogic = static_cast<RigLogicImpl*>(rigLogic);
    PolyAllocator<RigInstanceImpl> alloc{memRes};
    return alloc.newObject(pRigLogic->getRigMetadata(), pRigLogic, memRes);
}

void RigInstance::destroy(RigInstance* instance) {
    auto ptr = static_cast<RigInstanceImpl*>(instance);
    PolyAllocator<RigInstanceImpl> alloc{ptr->getMemoryResource()};
    alloc.deleteObject(ptr);
}

RigInstance::~RigInstance() = default;

RigInstanceImpl::RigInstanceImpl(RigMetadata* meta, RigLogicImpl* rigLogic, MemoryResource* memRes_) :
    memRes{memRes_},
    lodMaxLevel{getMaxLODLevel(meta->lodCount)},
    lodLevel{},
    guiControlCount{meta->guiControlCount},
    rawControlCount{meta->rawControlCount},
    psdControlCount{meta->psdControlCount},
    mlControlCount{meta->mlControlCount},
    rbfControlCount{meta->rbfControlCount},
    mlTypeCount{meta->mlTypeCount},
    controlsInstance{rigLogic->createControlsInstance(memRes)},
    machineLearnedBehaviorInstance{rigLogic->createMachineLearnedBehaviorInstance(memRes)},
    rbfBehaviorInstance{rigLogic->createRBFBehaviorInstance(memRes)},
    psdNetInstance{rigLogic->createPSDNetInstance(memRes)},
    jointsInstance{rigLogic->createJointsInstance(memRes)},
    blendShapesInstance{rigLogic->createBlendShapesInstance(memRes)},
    animatedMapsInstance{rigLogic->createAnimatedMapsInstance(memRes)} {
}

std::uint16_t RigInstanceImpl::getGUIControlCount() const {
    return guiControlCount;
}

float RigInstanceImpl::getGUIControl(std::uint16_t index) const {
    return getGUIControlValues()[index];
}

void RigInstanceImpl::setGUIControl(std::uint16_t index, float value) {
    auto guiControlBuffer = controlsInstance->getGUIControlBuffer();
    guiControlBuffer[index] = value;
}

ArrayView<float> RigInstanceImpl::getGUIControlValues() {
    return controlsInstance->getGUIControlBuffer();
}

ConstArrayView<float> RigInstanceImpl::getGUIControlValues() const {
    return controlsInstance->getGUIControlBuffer();
}

std::uint16_t RigInstanceImpl::getRawControlCount() const {
    return rawControlCount;
}

float RigInstanceImpl::getRawControl(std::uint16_t index) const {
    return getRawControlValues()[index];
}

void RigInstanceImpl::setRawControl(std::uint16_t index, float value) {
    auto inputBuffer = controlsInstance->getInputBuffer();
    inputBuffer[index] = value;
}

ArrayView<float> RigInstanceImpl::getRawControlValues() {
    return controlsInstance->getInputBuffer().subview(0ul, rawControlCount);
}

ConstArrayView<float> RigInstanceImpl::getRawControlValues() const {
    return controlsInstance->getInputBuffer().subview(0ul, rawControlCount);
}

std::uint16_t RigInstanceImpl::getPSDControlCount() const {
    return psdControlCount;
}

float RigInstanceImpl::getPSDControl(std::uint16_t index) const {
    return getPSDControlValues()[index];
}

ConstArrayView<float> RigInstanceImpl::getPSDControlValues() const {
    return controlsInstance->getInputBuffer().subview(rawControlCount, psdControlCount);
}

std::uint16_t RigInstanceImpl::getMLControlCount() const {
    return mlControlCount;
}

float RigInstanceImpl::getMLControl(std::uint16_t index) const {
    return getMLControlValues()[index];
}

ConstArrayView<float> RigInstanceImpl::getMLControlValues() const {
    const auto mlControlsOffset = static_cast<std::size_t>(rawControlCount) + static_cast<std::size_t>(psdControlCount);
    return controlsInstance->getInputBuffer().subview(mlControlsOffset, mlControlCount);
}

std::uint16_t RigInstanceImpl::getRBFControlCount() const {
    return rbfControlCount;
}

float RigInstanceImpl::getRBFControl(std::uint16_t index) const {
    return getRBFControlValues()[index];
}

ConstArrayView<float> RigInstanceImpl::getRBFControlValues() const {
    const auto rbfControlsOffset = static_cast<std::size_t>(rawControlCount) + static_cast<std::size_t>(psdControlCount) +
                                   static_cast<std::size_t>(mlControlCount);
    return controlsInstance->getInputBuffer().subview(rbfControlsOffset, rbfControlCount);
}

std::uint16_t RigInstanceImpl::getMLTypeCount() const {
    return mlTypeCount;
}

std::uint16_t RigInstanceImpl::getMLOperationSetCount(std::uint16_t mlTypeIndex) const {
    return machineLearnedBehaviorInstance->getMLOperationSetCount(mlTypeIndex);
}

std::uint16_t RigInstanceImpl::getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const {
    return machineLearnedBehaviorInstance->getMLOperationCount(mlTypeIndex, mlOperationSetIndex);
}

ConstArrayView<float> RigInstanceImpl::getMLMaskValues() const {
    return machineLearnedBehaviorInstance->getMaskBuffer();
}

ArrayView<float> RigInstanceImpl::getMLMaskValues() {
    return machineLearnedBehaviorInstance->getMaskBuffer();
}

std::uint16_t RigInstanceImpl::getLOD() const {
    return lodLevel;
}

void RigInstanceImpl::setLOD(std::uint16_t level) {
    if (level != lodLevel) {
        controlsInstance->resetInputBuffer();
        psdNetInstance->resetClampBuffer();
        jointsInstance->resetOutputBuffer();
        blendShapesInstance->resetOutputBuffer();
        animatedMapsInstance->resetOutputBuffer();
    }
    lodLevel = extd::clamp(level, static_cast<std::uint16_t>(0), lodMaxLevel);
}

ConstArrayView<float> RigInstanceImpl::getJointOutputs() const {
    return jointsInstance->getOutputBuffer();
}

ConstArrayView<float> RigInstanceImpl::getBlendShapeOutputs() const {
    return blendShapesInstance->getOutputBuffer();
}

ConstArrayView<float> RigInstanceImpl::getAnimatedMapOutputs() const {
    return animatedMapsInstance->getOutputBuffer();
}

ControlsInputInstance* RigInstanceImpl::getControlsInputInstance() {
    return controlsInstance.get();
}

MachineLearnedBehaviorOutputInstance* RigInstanceImpl::getMachineLearnedBehaviorOutputInstance() {
    return machineLearnedBehaviorInstance.get();
}

RBFBehaviorOutputInstance* RigInstanceImpl::getRBFBehaviorOutputInstance() {
    return rbfBehaviorInstance.get();
}

PSDNetOutputInstance* RigInstanceImpl::getPSDNetOutputInstance() {
    return psdNetInstance.get();
}

JointsOutputInstance* RigInstanceImpl::getJointsOutputInstance() {
    return jointsInstance.get();
}

BlendShapesOutputInstance* RigInstanceImpl::getBlendShapesOutputInstance() {
    return blendShapesInstance.get();
}

AnimatedMapsOutputInstance* RigInstanceImpl::getAnimatedMapOutputInstance() {
    return animatedMapsInstance.get();
}

MemoryResource* RigInstanceImpl::getMemoryResource() {
    return memRes;
}

}  // namespace rl4
