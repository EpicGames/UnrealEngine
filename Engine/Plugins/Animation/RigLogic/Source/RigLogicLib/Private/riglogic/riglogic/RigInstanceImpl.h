// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"
#include "riglogic/psdnet/PSDNetOutputInstance.h"
#include "riglogic/rbf/RBFBehaviorOutputInstance.h"
#include "riglogic/riglogic/RigInstance.h"
#include "riglogic/riglogic/RigMetadata.h"

#include <cstdint>

namespace rl4 {

class RigLogicImpl;

class RigInstanceImpl : public RigInstance {
public:
    RigInstanceImpl(RigMetadata* meta, RigLogicImpl* rigLogic, MemoryResource* memRes_);

    std::uint16_t getGUIControlCount() const override;
    float getGUIControl(std::uint16_t index) const override;
    void setGUIControl(std::uint16_t index, float value) override;
    ArrayView<float> getGUIControlValues() override;
    ConstArrayView<float> getGUIControlValues() const override;

    std::uint16_t getRawControlCount() const override;
    float getRawControl(std::uint16_t index) const override;
    void setRawControl(std::uint16_t index, float value) override;
    ArrayView<float> getRawControlValues() override;
    ConstArrayView<float> getRawControlValues() const override;

    std::uint16_t getPSDControlCount() const override;
    float getPSDControl(std::uint16_t index) const override;
    ConstArrayView<float> getPSDControlValues() const override;

    std::uint16_t getMLControlCount() const override;
    float getMLControl(std::uint16_t index) const override;
    ConstArrayView<float> getMLControlValues() const override;

    std::uint16_t getMLTypeCount() const override;
    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override;
    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override;
    ConstArrayView<float> getMLMaskValues() const override;
    ArrayView<float> getMLMaskValues() override;

    std::uint16_t getRBFControlCount() const override;
    float getRBFControl(std::uint16_t index) const override;
    ConstArrayView<float> getRBFControlValues() const override;

    std::uint16_t getLOD() const override;
    void setLOD(std::uint16_t level) override;

    ConstArrayView<float> getJointOutputs() const override;
    ConstArrayView<float> getBlendShapeOutputs() const override;
    ConstArrayView<float> getAnimatedMapOutputs() const override;

    ControlsInputInstance* getControlsInputInstance();
    MachineLearnedBehaviorOutputInstance* getMachineLearnedBehaviorOutputInstance();
    RBFBehaviorOutputInstance* getRBFBehaviorOutputInstance();
    PSDNetOutputInstance* getPSDNetOutputInstance();
    JointsOutputInstance* getJointsOutputInstance();
    BlendShapesOutputInstance* getBlendShapesOutputInstance();
    AnimatedMapsOutputInstance* getAnimatedMapOutputInstance();

    MemoryResource* getMemoryResource();

private:
    MemoryResource* memRes;

    std::uint16_t lodMaxLevel;
    std::uint16_t lodLevel;
    std::uint16_t guiControlCount;
    std::uint16_t rawControlCount;
    std::uint16_t psdControlCount;
    std::uint16_t mlControlCount;
    std::uint16_t rbfControlCount;
    std::uint16_t mlTypeCount;

    ControlsInputInstance::Pointer controlsInstance;
    MachineLearnedBehaviorOutputInstance::Pointer machineLearnedBehaviorInstance;
    RBFBehaviorOutputInstance::Pointer rbfBehaviorInstance;
    PSDNetOutputInstance::Pointer psdNetInstance;
    JointsOutputInstance::Pointer jointsInstance;
    BlendShapesOutputInstance::Pointer blendShapesInstance;
    AnimatedMapsOutputInstance::Pointer animatedMapsInstance;
};

}  // namespace rl4
