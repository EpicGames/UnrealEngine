// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

#include "riglogic/riglogic/RigLogicImpl.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsFactory.h"
#include "riglogic/blendshapes/BlendShapesFactory.h"
#include "riglogic/controls/ControlsFactory.h"
#include "riglogic/joints/JointsFactory.h"
#include "riglogic/ml/MachineLearnedBehaviorFactory.h"
#include "riglogic/psdnet/PSDNetFactory.h"
#include "riglogic/rbf/RBFBehaviorFactory.h"
#include "riglogic/riglogic/ConfigurationSerializer.h"
#include "riglogic/riglogic/RigInstanceImpl.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/riglogic/Stats.h"
#include "riglogic/system/simd/Utils.h"
#include "riglogic/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <memory>
#include <numeric>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rl4 {

static RigInstanceImpl* castInstance(RigInstance* instance) {
    return static_cast<RigInstanceImpl*>(instance);
}

RigLogic::~RigLogic() = default;

RigLogic* RigLogic::create(const dna::Reader* reader, const Configuration& config, MemoryResource* memRes) {
    const ActiveFeatures activeFeatures = getActiveFeatures(config);
    auto meta = RigMetadata::create(config, reader, memRes, InitializationMethod::Create);
    auto controls = ControlsFactory::create(config, meta.get(), reader, memRes);
    auto mlBehavior = MachineLearnedBehaviorFactory::create(config, meta.get(), reader, memRes);
    auto rbfBehavior = RBFBehaviorFactory::create(config, meta.get(), reader, memRes);
    auto joints = JointsFactory::create(config, meta.get(), reader, controls.get(), memRes);
    auto blendShapes = BlendShapesFactory::create(config, meta.get(), reader, controls.get(), memRes);
    auto animatedMaps = AnimatedMapsFactory::create(config, meta.get(), reader, controls.get(), memRes);
    // Must be created after controls are registered by the earlier systems
    auto psds = PSDNetFactory::create(config, meta.get(), reader, controls.get(), memRes);

    PolyAllocator<RigLogicImpl> alloc{memRes};
    return alloc.newObject(config,
                           activeFeatures,
                           std::move(meta),
                           std::move(controls),
                           std::move(mlBehavior),
                           std::move(rbfBehavior),
                           std::move(psds),
                           std::move(joints),
                           std::move(blendShapes),
                           std::move(animatedMaps),
                           memRes);
}

void RigLogic::destroy(RigLogic* instance) {
    auto ptr = static_cast<RigLogicImpl*>(instance);
    PolyAllocator<RigLogicImpl> alloc{ptr->getMemoryResource()};
    alloc.deleteObject(ptr);
}

RigLogic* RigLogic::restore(BoundedIOStream* source, MemoryResource* memRes) {
    PolyAllocator<RigLogicImpl> alloc{memRes};

    terse::BinaryInputArchive<BoundedIOStream> archive{source};

    Configuration config;
    archive >> config;
    archive.setUserData(&config);

    const ActiveFeatures activeFeatures = getActiveFeatures(config);

    RigMetadata::Pointer meta = UniqueInstance<RigMetadata>::with(memRes).create(memRes, InitializationMethod::Restore);
    archive >> *meta;

    auto controls = ControlsFactory::create(config, meta.get(), memRes);
    auto mlBehavior = MachineLearnedBehaviorFactory::create(config, meta.get(), memRes);
    auto rbfBehavior = RBFBehaviorFactory::create(config, meta.get(), memRes);
    auto joints = JointsFactory::create(config, meta.get(), memRes);
    auto blendShapes = BlendShapesFactory::create(config, meta.get(), memRes);
    auto animatedMaps = AnimatedMapsFactory::create(config, meta.get(), memRes);
    auto psds = PSDNetFactory::create(config, meta.get(), memRes);

    terse::VirtualSerializerProxy<AnimatedMaps> animatedMapsProxy{animatedMaps.get()};
    terse::VirtualSerializerProxy<BlendShapes> blendShapesProxy{blendShapes.get()};
    terse::VirtualSerializerProxy<PSDNet> psdNetProxy{psds.get()};

    archive >> *controls >> *mlBehavior >> *rbfBehavior >> psdNetProxy >> *joints >> blendShapesProxy >> animatedMapsProxy;

    return alloc.newObject(config,
                           activeFeatures,
                           std::move(meta),
                           std::move(controls),
                           std::move(mlBehavior),
                           std::move(rbfBehavior),
                           std::move(psds),
                           std::move(joints),
                           std::move(blendShapes),
                           std::move(animatedMaps),
                           memRes);
}

RigLogicImpl::RigLogicImpl(const Configuration& config_,
                           ActiveFeatures activeFeatures_,
                           RigMetadata::Pointer meta_,
                           Controls::Pointer controls_,
                           MachineLearnedBehavior::Pointer machineLearnedBehavior_,
                           RBFBehavior::Pointer rbfBehavior_,
                           PSDNet::Pointer psds_,
                           Joints::Pointer joints_,
                           BlendShapes::Pointer blendShapes_,
                           AnimatedMaps::Pointer animatedMaps_,
                           MemoryResource* memRes_) :
    memRes{memRes_},
    config{config_},
    activeFeatures{activeFeatures_},
    meta{std::move(meta_)},
    controls{std::move(controls_)},
    machineLearnedBehavior{std::move(machineLearnedBehavior_)},
    rbfBehavior{std::move(rbfBehavior_)},
    psds{std::move(psds_)},
    joints{std::move(joints_)},
    blendShapes{std::move(blendShapes_)},
    animatedMaps{std::move(animatedMaps_)} {
}

void RigLogicImpl::dump(BoundedIOStream* destination) const {
    terse::BinaryOutputArchive<BoundedIOStream> archive{destination};
    terse::VirtualSerializerProxy<AnimatedMaps> animatedMapsProxy{animatedMaps.get()};
    terse::VirtualSerializerProxy<BlendShapes> blendShapesProxy{blendShapes.get()};
    terse::VirtualSerializerProxy<PSDNet> psdNetProxy{psds.get()};
    archive.setUserData(const_cast<Configuration*>(&config));
    // *INDENT-OFF*
    archive << config << *meta << *controls << *machineLearnedBehavior << *rbfBehavior << psdNetProxy << *joints
            << blendShapesProxy << animatedMapsProxy;
    // *INDENT-ON*
}

const Configuration& RigLogicImpl::getConfiguration() const {
    return config;
}

RigMetadata* RigLogicImpl::getRigMetadata() const {
    return meta.get();
}

std::uint16_t RigLogicImpl::getLODCount() const {
    return meta->lodCount;
}

ConstArrayView<std::uint16_t> RigLogicImpl::getRBFSolverIndicesForLOD(std::uint16_t lod) const {
    return rbfBehavior->getSolverIndicesForLOD(lod);
}

ConstArrayView<std::uint16_t> RigLogicImpl::getMLOperationIndicesForLOD(std::uint16_t lod,
                                                                        std::uint16_t mlTypeIndex,
                                                                        std::uint16_t mlOperationSetIndex) const {
    return machineLearnedBehavior->getMLOperationIndicesForLOD(lod, mlTypeIndex, mlOperationSetIndex);
}

ConstArrayView<std::uint16_t> RigLogicImpl::getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const {
    return blendShapes->getBlendShapeChannelIndicesForLOD(lod);
}

ConstArrayView<std::uint16_t> RigLogicImpl::getAnimatedMapIndicesForLOD(std::uint16_t lod) const {
    return animatedMaps->getAnimatedMapIndicesForLOD(lod);
}

ConstArrayView<std::uint16_t> RigLogicImpl::getJointIndicesForLOD(std::uint16_t lod) const {
    return joints->getJointIndicesForLOD(lod);
}

ConstArrayView<float> RigLogicImpl::getNeutralJointValues() const {
    return joints->getNeutralValues();
}

ConstArrayView<std::uint16_t> RigLogicImpl::getJointVariableAttributeIndices(std::uint16_t lod) const {
    return joints->getVariableAttributeIndices(lod);
}

std::uint16_t RigLogicImpl::getJointGroupCount() const {
    return joints->getJointGroupCount();
}

std::uint16_t RigLogicImpl::getRBFSolverCount() const {
    return meta->rbfSolverCount;
}

std::uint16_t RigLogicImpl::getTwistCount() const {
    return meta->twistCount;
}

std::uint16_t RigLogicImpl::getSwingCount() const {
    return meta->swingCount;
}

std::uint16_t RigLogicImpl::getMeshCount() const {
    return machineLearnedBehavior->getMeshCount();
}

std::uint16_t RigLogicImpl::getMeshRegionCount(std::uint16_t meshIndex) const {
    return machineLearnedBehavior->getMeshRegionCount(meshIndex);
}

std::uint16_t RigLogicImpl::getMLTypeCount() const {
    return meta->mlTypeCount;
}

std::uint16_t RigLogicImpl::getMLOperationSetCount(std::uint16_t mlTypeIndex) const {
    return machineLearnedBehavior->getMLOperationSetCount(mlTypeIndex);
}

std::uint16_t RigLogicImpl::getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const {
    return machineLearnedBehavior->getMLOperationCount(mlTypeIndex, mlOperationSetIndex);
}

ControlsInputInstance::Pointer RigLogicImpl::createControlsInstance(MemoryResource* instanceMemRes) const {
    return controls->createInstance(instanceMemRes);
}

MachineLearnedBehaviorOutputInstance::Pointer RigLogicImpl::createMachineLearnedBehaviorInstance(
    MemoryResource* instanceMemRes) const {
    return machineLearnedBehavior->createInstance(instanceMemRes);
}

RBFBehaviorOutputInstance::Pointer RigLogicImpl::createRBFBehaviorInstance(MemoryResource* instanceMemRes) const {
    return rbfBehavior->createInstance(instanceMemRes);
}

PSDNetOutputInstance::Pointer RigLogicImpl::createPSDNetInstance(MemoryResource* instanceMemRes) const {
    return psds->createInstance(instanceMemRes);
}

JointsOutputInstance::Pointer RigLogicImpl::createJointsInstance(MemoryResource* instanceMemRes) const {
    return joints->createInstance(instanceMemRes);
}

BlendShapesOutputInstance::Pointer RigLogicImpl::createBlendShapesInstance(MemoryResource* instanceMemRes) const {
    return blendShapes->createInstance(instanceMemRes);
}

AnimatedMapsOutputInstance::Pointer RigLogicImpl::createAnimatedMapsInstance(MemoryResource* instanceMemRes) const {
    return animatedMaps->createInstance(instanceMemRes);
}

void RigLogicImpl::mapGUIToRawControls(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    controls->mapGUIToRaw(pRigInstance->getControlsInputInstance());
}

void RigLogicImpl::mapRawToGUIControls(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    controls->mapRawToGUI(pRigInstance->getControlsInputInstance());
}

void RigLogicImpl::calculateMLControls(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    machineLearnedBehavior->calculate(pRigInstance->getControlsInputInstance(),
                                      pRigInstance->getMachineLearnedBehaviorOutputInstance(),
                                      pRigInstance->getLOD());
}

void RigLogicImpl::calculateMLControls(RigInstance* instance,
                                       std::uint16_t mlTypeIndex,
                                       std::uint16_t mlOperationSetIndex,
                                       std::uint16_t mlOperationIndex) const {
    auto pRigInstance = castInstance(instance);
    machineLearnedBehavior->calculate(pRigInstance->getControlsInputInstance(),
                                      pRigInstance->getMachineLearnedBehaviorOutputInstance(),
                                      pRigInstance->getLOD(),
                                      mlTypeIndex,
                                      mlOperationSetIndex,
                                      mlOperationIndex);
}

void RigLogicImpl::calculateRBFControls(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    rbfBehavior->calculate(pRigInstance->getControlsInputInstance(),
                           pRigInstance->getRBFBehaviorOutputInstance(),
                           pRigInstance->getLOD());
}

void RigLogicImpl::calculateRBFControls(RigInstance* instance, std::uint16_t solverIndex) const {
    auto pRigInstance = castInstance(instance);
    rbfBehavior->calculate(pRigInstance->getControlsInputInstance(),
                           pRigInstance->getRBFBehaviorOutputInstance(),
                           pRigInstance->getLOD(),
                           solverIndex);
}

void RigLogicImpl::calculatePSDControls(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    auto inputInstance = pRigInstance->getControlsInputInstance();
    auto outputInstance = pRigInstance->getPSDNetOutputInstance();
    auto lod = pRigInstance->getLOD();
    psds->calculate(inputInstance, outputInstance, lod);
}

void RigLogicImpl::calculateJoints(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    joints->calculate(pRigInstance->getControlsInputInstance(), pRigInstance->getJointsOutputInstance(), pRigInstance->getLOD());
}

void RigLogicImpl::calculateJoints(RigInstance* instance, std::uint16_t jointGroupIndex) const {
    auto pRigInstance = castInstance(instance);
    joints->calculate(pRigInstance->getControlsInputInstance(),
                      pRigInstance->getJointsOutputInstance(),
                      pRigInstance->getLOD(),
                      jointGroupIndex);
}

void RigLogicImpl::calculateBlendShapes(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    blendShapes->calculate(pRigInstance->getControlsInputInstance(),
                           pRigInstance->getBlendShapesOutputInstance(),
                           pRigInstance->getLOD());
}

void RigLogicImpl::calculateAnimatedMaps(RigInstance* instance) const {
    auto pRigInstance = castInstance(instance);
    animatedMaps->calculate(pRigInstance->getControlsInputInstance(),
                            pRigInstance->getAnimatedMapOutputInstance(),
                            pRigInstance->getLOD());
}

void RigLogicImpl::calculate(RigInstance* instance) const {
    calculateMLControls(instance);
    calculateRBFControls(instance);
    calculatePSDControls(instance);
    calculateJoints(instance);
    calculateBlendShapes(instance);
    calculateAnimatedMaps(instance);
}

void RigLogicImpl::collectCalculationStats(const RigInstance* instance, Stats* stats) const {
    const auto lod = instance->getLOD();
    stats->calculationType = activeFeatures.calculationType;
    stats->floatingPointType = activeFeatures.floatingPointType;
    stats->rbfSolverCount = static_cast<std::uint16_t>(rbfBehavior->getSolverIndicesForLOD(lod).size());
    stats->mlOperationCount = {};
    for (std::uint16_t mlTypeIndex = {}; mlTypeIndex < meta->mlTypeCount; ++mlTypeIndex) {
        const auto mlOperationSetCount = machineLearnedBehavior->getMLOperationSetCount(mlTypeIndex);
        for (std::uint16_t mlOperationSetIndex = {}; mlOperationSetIndex < mlOperationSetCount; ++mlOperationSetIndex) {
            const auto mlOperationIndicesForLOD =
                machineLearnedBehavior->getMLOperationIndicesForLOD(lod, mlTypeIndex, mlOperationSetIndex);
            stats->mlOperationCount = static_cast<std::uint16_t>(stats->mlOperationCount + mlOperationIndicesForLOD.size());
        }
    }
    stats->psdCount = static_cast<std::uint16_t>(psds->getPSDOutputIndicesForLOD(lod).size());
    stats->blendShapeChannelCount = static_cast<std::uint16_t>(blendShapes->getBlendShapeChannelIndicesForLOD(lod).size());
    stats->animatedMapCount = static_cast<std::uint16_t>(animatedMaps->getAnimatedMapIndicesForLOD(lod).size());
    stats->jointCount = static_cast<std::uint16_t>(joints->getJointIndicesForLOD(lod).size());
    stats->jointDeltaValueCount = joints->getJointDeltaValueCountForLOD(lod);
}

MemoryResource* RigLogicImpl::getMemoryResource() {
    return memRes;
}

}  // namespace rl4
