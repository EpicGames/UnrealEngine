// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMaps.h"
#include "riglogic/animatedmaps/AnimatedMapsOutputInstance.h"
#include "riglogic/blendshapes/BlendShapes.h"
#include "riglogic/blendshapes/BlendShapesOutputInstance.h"
#include "riglogic/controls/Controls.h"
#include "riglogic/controls/ControlsInputInstance.h"
#include "riglogic/joints/Joints.h"
#include "riglogic/joints/JointsOutputInstance.h"
#include "riglogic/ml/MachineLearnedBehavior.h"
#include "riglogic/ml/MachineLearnedBehaviorOutputInstance.h"
#include "riglogic/psdnet/PSDNet.h"
#include "riglogic/rbf/RBFBehavior.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigLogic.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Utils.h"

namespace rl4 {

class RigLogicImpl : public RigLogic {
public:
    RigLogicImpl(const Configuration& config_,
                 ActiveFeatures activeFeatures_,
                 RigMetadata::Pointer meta_,
                 Controls::Pointer controls_,
                 MachineLearnedBehavior::Pointer machineLearnedBehavior_,
                 RBFBehavior::Pointer rbfBehavior_,
                 PSDNet::Pointer psds_,
                 Joints::Pointer joints_,
                 BlendShapes::Pointer blendShapes_,
                 AnimatedMaps::Pointer animatedMaps_,
                 MemoryResource* memRes_);

    void dump(BoundedIOStream* destination) const override;
    const Configuration& getConfiguration() const override;
    RigMetadata* getRigMetadata() const;
    std::uint16_t getLODCount() const override;
    ConstArrayView<std::uint16_t> getRBFSolverIndicesForLOD(std::uint16_t lod) const override;
    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t lod,
                                                              std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex) const override;
    ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const override;
    ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t lod) const override;
    ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const override;
    ConstArrayView<float> getNeutralJointValues() const override;
    ConstArrayView<std::uint16_t> getJointVariableAttributeIndices(std::uint16_t lod) const override;
    std::uint16_t getJointGroupCount() const override;
    std::uint16_t getRBFSolverCount() const override;
    std::uint16_t getTwistCount() const override;
    std::uint16_t getSwingCount() const override;
    std::uint16_t getMeshCount() const override;
    std::uint16_t getMeshRegionCount(std::uint16_t meshIndex) const override;
    std::uint16_t getMLTypeCount() const override;
    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override;
    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override;

    ControlsInputInstance::Pointer createControlsInstance(MemoryResource* instanceMemRes) const;
    MachineLearnedBehaviorOutputInstance::Pointer createMachineLearnedBehaviorInstance(MemoryResource* instanceMemRes) const;
    RBFBehaviorOutputInstance::Pointer createRBFBehaviorInstance(MemoryResource* instanceMemRes) const;
    PSDNetOutputInstance::Pointer createPSDNetInstance(MemoryResource* instanceMemRes) const;
    JointsOutputInstance::Pointer createJointsInstance(MemoryResource* instanceMemRes) const;
    BlendShapesOutputInstance::Pointer createBlendShapesInstance(MemoryResource* instanceMemRes) const;
    AnimatedMapsOutputInstance::Pointer createAnimatedMapsInstance(MemoryResource* instanceMemRes) const;

    void mapGUIToRawControls(RigInstance* instance) const override;
    void mapRawToGUIControls(RigInstance* instance) const override;
    void calculateMLControls(RigInstance* instance) const override;
    void calculateMLControls(RigInstance* instance,
                             std::uint16_t mlTypeIndex,
                             std::uint16_t mlOperationSetIndex,
                             std::uint16_t mlOperationIndex) const override;
    void calculateRBFControls(RigInstance* instance) const override;
    void calculateRBFControls(RigInstance* instance, std::uint16_t solverIndex) const override;
    void calculatePSDControls(RigInstance* instance) const override;
    void calculateJoints(RigInstance* instance) const override;
    void calculateJoints(RigInstance* instance, std::uint16_t jointGroupIndex) const override;
    void calculateBlendShapes(RigInstance* instance) const override;
    void calculateAnimatedMaps(RigInstance* instance) const override;
    void calculate(RigInstance* instance) const override;
    void collectCalculationStats(const RigInstance* instance, Stats* stats) const override;

    MemoryResource* getMemoryResource();

private:
    MemoryResource* memRes;
    Configuration config;
    ActiveFeatures activeFeatures;
    RigMetadata::Pointer meta;
    Controls::Pointer controls;
    MachineLearnedBehavior::Pointer machineLearnedBehavior;
    RBFBehavior::Pointer rbfBehavior;
    PSDNet::Pointer psds;
    Joints::Pointer joints;
    BlendShapes::Pointer blendShapes;
    AnimatedMaps::Pointer animatedMaps;
};

}  // namespace rl4
