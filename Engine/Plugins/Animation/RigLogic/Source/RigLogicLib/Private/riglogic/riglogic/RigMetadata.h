// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/riglogic/Configuration.h"

#include <cstdint>

namespace rl4 {

enum class InitializationMethod : std::uint16_t {
    Create,
    Restore
};

enum class EvaluatorType : std::uint16_t {
    Auto,
    Null,
    Concrete
};

struct RigMetadata {
    using Pointer = UniqueInstance<RigMetadata>::PointerType;

    tdm::coord_sys coordinateSystem;
    tdm::rot_seq rotationSequence;
    tdm::rot_sign rotationSigns;
    std::uint16_t lodCount;
    std::uint16_t guiControlCount;
    std::uint16_t rawControlCount;
    std::uint16_t psdControlCount;
    std::uint16_t mlControlCount;
    std::uint16_t rbfControlCount;
    std::uint16_t jointGroupCount;
    std::uint16_t jointAttributeCount;
    std::uint16_t blendShapeCount;
    std::uint16_t animatedMapCount;
    std::uint16_t mlTypeCount;
    std::uint16_t rbfSolverCount;
    std::uint16_t twistCount;
    std::uint16_t swingCount;
    Vector<std::uint16_t> evaluators;
    // Non-serialized, runtime parameter
    InitializationMethod initializationMethod;

    static Pointer create(const Configuration& config,
                          const dna::Reader* reader,
                          MemoryResource* memRes,
                          InitializationMethod initializationMethod) {
        Pointer meta = UniqueInstance<RigMetadata>::with(memRes).create(memRes, initializationMethod);
        meta->coordinateSystem = reader->getCoordinateSystem();
        meta->rotationSequence = reader->getRotationSequence();
        meta->rotationSigns = reader->getRotationSign();
        meta->lodCount = reader->getLODCount();
        meta->guiControlCount = reader->getGUIControlCount();
        meta->rawControlCount = reader->getRawControlCount();
        meta->psdControlCount = reader->getPSDCount();
        meta->jointGroupCount = reader->getJointGroupCount();
        const auto numAttrsPerJoint =
            (static_cast<std::uint8_t>(config.translationType) + static_cast<std::uint8_t>(config.rotationType) +
             static_cast<std::uint8_t>(config.scaleType));
        meta->jointAttributeCount = static_cast<std::uint16_t>(reader->getJointCount() * numAttrsPerJoint);
        meta->blendShapeCount = reader->getBlendShapeChannelCount();
        meta->animatedMapCount = reader->getAnimatedMapCount();
        meta->mlControlCount = reader->getMLControlCount();
        meta->mlTypeCount = reader->getMLTypeCount();
        meta->rbfControlCount = reader->getRBFPoseControlCount();
        meta->rbfSolverCount = reader->getRBFSolverCount();
        meta->twistCount = reader->getTwistCount();
        meta->swingCount = reader->getSwingCount();
        return meta;
    }

    explicit RigMetadata(MemoryResource* memRes, InitializationMethod initializationMethod_) :
        coordinateSystem{},
        rotationSequence{},
        rotationSigns{},
        lodCount{},
        guiControlCount{},
        rawControlCount{},
        psdControlCount{},
        mlControlCount{},
        rbfControlCount{},
        jointGroupCount{},
        jointAttributeCount{},
        blendShapeCount{},
        animatedMapCount{},
        mlTypeCount{},
        rbfSolverCount{},
        twistCount{},
        swingCount{},
        evaluators{memRes},
        initializationMethod{initializationMethod_} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(coordinateSystem,
                rotationSequence,
                rotationSigns,
                lodCount,
                guiControlCount,
                rawControlCount,
                psdControlCount,
                mlControlCount,
                rbfControlCount,
                jointGroupCount,
                jointAttributeCount,
                blendShapeCount,
                animatedMapCount,
                mlTypeCount,
                rbfSolverCount,
                twistCount,
                swingCount,
                evaluators);
    }

    EvaluatorType popFrontEvaluator() {
        std::uint16_t type = evaluators.front();
        evaluators.erase(evaluators.begin());
        return static_cast<EvaluatorType>(type);
    }

    void pushBackEvaluator(EvaluatorType id) {
        evaluators.push_back(static_cast<std::uint16_t>(id));
    }
};

}  // namespace rl4
