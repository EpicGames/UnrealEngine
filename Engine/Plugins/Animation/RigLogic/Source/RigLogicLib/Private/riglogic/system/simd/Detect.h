// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trimd/Platform.h"

#if defined(RL_AUTODETECT_SSE) && !defined(RL_BUILD_WITH_SSE)
    // Assume any x86 architecture we try to build on will support SSE
    #if defined(TRIMD_PLATFORM_X86)
        #define RL_BUILD_WITH_SSE 1
        #if defined(RL_AUTODETECT_HALF_FLOATS) && !defined(RL_BUILD_WITH_HALF_FLOATS)
            #if defined(__F16C__)
                #define RL_BUILD_WITH_HALF_FLOATS 1
            #endif
        #endif  // RL_AUTODETECT_HALF_FLOATS
    #endif
#endif  // RL_AUTODETECT_SSE

#if defined(RL_AUTODETECT_AVX) && !defined(RL_BUILD_WITH_AVX)
    #if defined(TRIMD_PLATFORM_X86) && defined(__AVX__)
        #define RL_BUILD_WITH_AVX 1
        #if defined(RL_AUTODETECT_HALF_FLOATS) && !defined(RL_BUILD_WITH_HALF_FLOATS)
            #if defined(__F16C__) || (defined(_MSC_VER) && (_MSC_VER >= 1700)) || defined(__INTEL_COMPILER)
                #define RL_BUILD_WITH_HALF_FLOATS 1
            #endif
        #endif  // RL_AUTODETECT_HALF_FLOATS
    #endif
#endif  // RL_AUTODETECT_AVX

#if defined(RL_AUTODETECT_NEON) && !defined(RL_BUILD_WITH_NEON)
    #if defined(TRIMD_PLATFORM_ARM) && defined(__ARM_NEON)
        #define RL_BUILD_WITH_NEON 1
        #if defined(RL_AUTODETECT_HALF_FLOATS) && !defined(RL_BUILD_WITH_HALF_FLOATS)
            // Half-float support will compile if NEON is available, and the actual feature check will be performed at runtime
            #define RL_BUILD_WITH_HALF_FLOATS 1
        #endif  // RL_AUTODETECT_HALF_FLOATS
    #endif
#endif  // RL_AUTODETECT_NEON
