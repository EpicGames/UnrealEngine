// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef RL_DISABLE_RUNTIME_FEATURE_DETECTION
    #ifndef TRIMD_ENABLE_RUNTIME_FEATURE_DETECTION
        #define TRIMD_ENABLE_RUNTIME_FEATURE_DETECTION
    #endif  // TRIMD_ENABLE_RUNTIME_FEATURE_DETECTION
#endif      // RL_DISABLE_RUNTIME_FEATURE_DETECTION

#include "riglogic/system/simd/Detect.h"

#if defined(RL_BUILD_WITH_AVX)
    #if defined(RL_BUILD_WITH_HALF_FLOATS) && !defined(TRIMD_ENABLE_F16C)
        #define TRIMD_ENABLE_F16C
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    #if !defined(TRIMD_ENABLE_AVX)
        #define TRIMD_ENABLE_AVX
    #endif
    #if !defined(TRIMD_ENABLE_SSE)
        #define TRIMD_ENABLE_SSE
    #endif
#endif  // RL_BUILD_WITH_AVX

#if defined(RL_BUILD_WITH_SSE) && !defined(TRIMD_ENABLE_SSE)
    #if defined(RL_BUILD_WITH_HALF_FLOATS) && !defined(TRIMD_ENABLE_F16C)
        #define TRIMD_ENABLE_F16C
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    #define TRIMD_ENABLE_SSE
#endif  // RL_BUILD_WITH_SSE

#if defined(RL_BUILD_WITH_NEON) && !defined(TRIMD_ENABLE_NEON)
    #if defined(RL_BUILD_WITH_HALF_FLOATS) && !defined(TRIMD_ENABLE_NEON_FP16)
        #define TRIMD_ENABLE_NEON_FP16
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    #define TRIMD_ENABLE_NEON
#endif  // RL_BUILD_WITH_NEON

#include <trimd/TRiMD.h>
