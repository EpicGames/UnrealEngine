// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/system/simd/SIMD.h"
#include "riglogic/utils/Macros.h"

namespace rl4 {

struct ActiveFeatures {
    CalculationType calculationType;
    FloatingPointType floatingPointType;
};

inline bool operator==(const ActiveFeatures& lhs, const ActiveFeatures& rhs) {
    return (lhs.calculationType == rhs.calculationType) && (lhs.floatingPointType == rhs.floatingPointType);
}

inline bool operator!=(const ActiveFeatures& lhs, const ActiveFeatures& rhs) {
    return !(lhs == rhs);
}

inline ActiveFeatures getActiveFeatures(const Configuration& config) {
    ActiveFeatures result = {};

    auto features = trimd::getCPUFeatures();
    RL_UNUSED(features);
    RL_UNUSED(config);
#ifdef RL_BUILD_WITH_SSE
    #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
    features.SSE2 = true;
    #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
    if (features.SSE2 &&
        ((config.calculationType == CalculationType::SSE) || (config.calculationType == CalculationType::AnyVector))) {
        result.calculationType = CalculationType::SSE;
        result.floatingPointType = FloatingPointType::Float;
    #ifdef RL_BUILD_WITH_HALF_FLOATS
        #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
        features.F16C = true;
        #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
        if (features.F16C && (config.floatingPointType == FloatingPointType::HalfFloat)) {
            result.floatingPointType = FloatingPointType::HalfFloat;
        }
    #endif  // RL_BUILD_WITH_HALF_FLOATS
        return result;
    }
#endif  // RL_BUILD_WITH_SSE
#ifdef RL_BUILD_WITH_AVX
    #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
    features.AVX = true;
    #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
    if (features.AVX &&
        ((config.calculationType == CalculationType::AVX) || (config.calculationType == CalculationType::AnyVector))) {
        result.calculationType = CalculationType::AVX;
        result.floatingPointType = FloatingPointType::Float;
    #ifdef RL_BUILD_WITH_HALF_FLOATS
        #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
        features.F16C = true;
        #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
        if (features.F16C && (config.floatingPointType == FloatingPointType::HalfFloat)) {
            result.floatingPointType = FloatingPointType::HalfFloat;
        }
    #endif  // RL_BUILD_WITH_HALF_FLOATS
        return result;
    }
#endif  // RL_BUILD_WITH_AVX
#ifdef RL_BUILD_WITH_NEON
    #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
    features.NEON = true;
    #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
    if (features.NEON &&
        ((config.calculationType == CalculationType::NEON) || (config.calculationType == CalculationType::AnyVector))) {
        result.calculationType = CalculationType::NEON;
        result.floatingPointType = FloatingPointType::Float;
    #ifdef RL_BUILD_WITH_HALF_FLOATS
        #ifdef RL_DISABLE_RUNTIME_FEATURE_DETECTION
        features.FP16 = true;
        #endif  // RL_DISABLE_RUNTIME_FEATURE_DETECTION
        if (features.FP16 && (config.floatingPointType == FloatingPointType::HalfFloat)) {
            result.floatingPointType = FloatingPointType::HalfFloat;
        }
    #endif  // RL_BUILD_WITH_HALF_FLOATS
        return result;
    }
#endif  // RL_BUILD_WITH_NEON

    result.calculationType = CalculationType::Scalar;
    result.floatingPointType = FloatingPointType::Float;
    return result;
}

struct RuntimeTemplateInstantiator {
    const Configuration* config;

    template<template<class...> class TClass, class TReturnType, typename... Args>
    TReturnType invoke(Args&&... args) {
        auto features = getActiveFeatures(*config);
        RL_UNUSED(features);

#ifdef RL_BUILD_WITH_SSE
        if (features.calculationType == CalculationType::SSE) {
    #ifdef RL_BUILD_WITH_HALF_FLOATS
            if (features.floatingPointType == FloatingPointType::HalfFloat) {
                return TClass<std::uint16_t, trimd::sse::F256, trimd::sse::F128>()(std::forward<Args>(args)...);
            }
    #endif  // RL_BUILD_WITH_HALF_FLOATS

            return TClass<float, trimd::sse::F256, trimd::sse::F128>()(std::forward<Args>(args)...);
        }
#endif  // RL_BUILD_WITH_SSE

#ifdef RL_BUILD_WITH_AVX
        if (features.calculationType == CalculationType::AVX) {
    #ifdef RL_BUILD_WITH_HALF_FLOATS
            if (features.floatingPointType == FloatingPointType::HalfFloat) {
                return TClass<std::uint16_t, trimd::avx::F256, trimd::sse::F128>()(std::forward<Args>(args)...);
            }
    #endif  // RL_BUILD_WITH_HALF_FLOATS

            return TClass<float, trimd::avx::F256, trimd::sse::F128>()(std::forward<Args>(args)...);
        }
#endif  // RL_BUILD_WITH_AVX

#ifdef RL_BUILD_WITH_NEON
        if (features.calculationType == CalculationType::NEON) {
    #ifdef RL_BUILD_WITH_HALF_FLOATS
            if (features.floatingPointType == FloatingPointType::HalfFloat) {
                return TClass<std::uint16_t, trimd::neon::F256, trimd::neon::F128>()(std::forward<Args>(args)...);
            }
    #endif  // RL_BUILD_WITH_HALF_FLOATS

            return TClass<float, trimd::neon::F256, trimd::neon::F128>()(std::forward<Args>(args)...);
        }
#endif  // RL_BUILD_WITH_NEON

        return TClass<float, trimd::scalar::F256, trimd::scalar::F128>()(std::forward<Args>(args)...);
    }
};

}  // namespace rl4
