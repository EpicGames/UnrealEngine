// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

namespace rl4 {

struct Extent {
    std::uint32_t rows;
    std::uint32_t cols;

    std::uint32_t size() const {
        return rows * cols;
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(rows, cols);
    }
};

inline bool operator==(const Extent& lhs, const Extent& rhs) {
    return (lhs.rows == rhs.rows && lhs.cols == rhs.cols);
}

inline bool operator!=(const Extent& lhs, const Extent& rhs) {
    return !(lhs == rhs);
}

}  // namespace rl4
