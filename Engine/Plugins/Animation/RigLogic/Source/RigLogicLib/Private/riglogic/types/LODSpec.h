// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

namespace rl4 {

template<typename T>
struct LODSpec {
    static_assert(std::is_unsigned<T>::value, "Template parameter must be an unsigned integer type");
    Matrix<T> indicesPerLOD;
    // Actual object count that we are LODing eg. RBF solver count, neural network count...
    T count;

    explicit LODSpec(MemoryResource* memRes) :
        indicesPerLOD{memRes},
        count{} {
    }

    LODSpec(Matrix<T>&& indicesPerLOD_, T count_) :
        indicesPerLOD{std::move(indicesPerLOD_)},
        count{count_} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(indicesPerLOD, count);
    }
};

}  // namespace rl4
