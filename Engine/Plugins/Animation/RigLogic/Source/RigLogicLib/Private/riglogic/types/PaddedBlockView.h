// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/utils/Extd.h"

#include <cstdint>

namespace rl4 {

struct PaddedBlockView {
    // Unpadded size
    std::uint32_t size;
    // If the last full block needs to be masked-off because it contains padding,
    // this view will include that last full block as well
    std::uint32_t sizePaddedToLastFullBlock;
    // If the last full block needs to be masked-off because it contains padding,
    // that full block won't be included in this boundary as it needs special care
    std::uint32_t sizePaddedToSecondLastFullBlock;

    PaddedBlockView() = default;
    PaddedBlockView(std::uint32_t viewRowCount, std::uint32_t paddedRowCount, std::uint32_t blockHeight, std::uint32_t padTo) :
        size{viewRowCount} {
        // Check whether the block was padded to blockHeight size or padTo size
        const bool endsWithPadToBlock = (paddedRowCount % blockHeight) == padTo;
        // Check if view boundary is within the region of the last padTo rows
        const bool isViewBoundaryInLastPadToRows = (paddedRowCount - viewRowCount) < padTo;
        // Check if view boundary is within the region of the last padTo-sized block
        const bool isViewBoundaryInLastPadToBlock = (endsWithPadToBlock && isViewBoundaryInLastPadToRows);
        // Round up the view boundary row to the nearest index that is modulo padTo or blockHeight
        // If boundary is in the last padTo-sized block, it rounds to padTo, otherwise to blockHeight
        const std::uint32_t padToTarget = (isViewBoundaryInLastPadToBlock ? padTo : blockHeight);
        const std::uint32_t paddedViewRowCount = extd::roundUp(viewRowCount, padToTarget);
        // Check if the unpadded boundary coincides with a block boundary
        const bool isViewBoundaryOnBlockBoundary = (paddedViewRowCount == viewRowCount);
        // Should the last blockHeight-sized block be masked-off or not
        // If view boundary coincides with a block boundary, no mask-off is needed at all
        // If view boundary is within the last padTo-sized block, no mask-off is needed for the last blockHeight-sized block
        const bool maskOffLastBlock = (!isViewBoundaryOnBlockBoundary && !isViewBoundaryInLastPadToBlock);
        // Find the last blockHeight-sized block boundary before the padded view boundary
        // Sometimes this equals the padded view boundary, but not if the matrix is padded to padTo block-size
        // and the view boundary is within that last padTo-sized block region
        sizePaddedToLastFullBlock = paddedViewRowCount - (paddedViewRowCount % blockHeight);
        // This is one blockHeight before `sizePaddedToLastFullBlock` because the last blockHeight-sized block
        // might need special care to mask off results that came from rows after the view boundary
        // If no mask-off is needed, it equals to `sizePaddedToLastFullBlock`
        sizePaddedToSecondLastFullBlock =
            (maskOffLastBlock ? sizePaddedToLastFullBlock - blockHeight : sizePaddedToLastFullBlock);
    }

    PaddedBlockView(std::uint32_t viewRowCount,
                    std::uint32_t boundaryRowCountPaddedToLastFullBlock,
                    std::uint32_t boundaryRowCountPaddedToSecondLastFullBlock) :
        size{viewRowCount},
        sizePaddedToLastFullBlock{boundaryRowCountPaddedToLastFullBlock},
        sizePaddedToSecondLastFullBlock{boundaryRowCountPaddedToSecondLastFullBlock} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(size, sizePaddedToLastFullBlock, sizePaddedToSecondLastFullBlock);
    }
};

}  // namespace rl4
