// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/system/simd/Utils.h"

namespace rl4 {

template<typename T, typename TF256, typename TF128>
struct FloatArrayDeserializer;

template<typename T, typename TF256, typename TF128>
struct FloatArraySerializer;

struct FloatArray {
    AlignedVector<float> fp32;
    AlignedVector<std::uint16_t> fp16;

    explicit FloatArray(MemoryResource* memRes) :
        fp32{memRes},
        fp16{memRes} {
    }

    template<class Archive>
    void load(Archive& archive) {
        const Configuration* config = static_cast<Configuration*>(archive.getUserData());
        RuntimeTemplateInstantiator instantiator{config};
        instantiator.invoke<FloatArrayDeserializer, void>(*this, archive);
    }

    template<class Archive>
    void save(Archive& archive) {
        const Configuration* config = static_cast<Configuration*>(archive.getUserData());
        RuntimeTemplateInstantiator instantiator{config};
        instantiator.invoke<FloatArraySerializer, void>(*this, archive);
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, float>::value, T*>::type data() {
        return fp32.data();
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, std::uint16_t>::value, T*>::type data() {
        return fp16.data();
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, float>::value, const T*>::type data() const {
        return fp32.data();
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, std::uint16_t>::value, const T*>::type data() const {
        return fp16.data();
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, float>::value, std::size_t>::type size() const {
        return fp32.size();
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, std::uint16_t>::value, std::size_t>::type size() const {
        return fp16.size();
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, float>::value, void>::type resize(std::size_t newSize) {
        return fp32.resize(newSize);
    }

    template<typename T>
    typename std::enable_if<std::is_same<T, std::uint16_t>::value, void>::type resize(std::size_t newSize) {
        return fp16.resize(newSize);
    }
};

template<typename TF256, typename TF128>
struct FloatArrayDeserializer<float, TF256, TF128> {

    template<class TArchive>
    void operator()(FloatArray& instance, TArchive& archive) {
        archive(instance.fp32);
    }
};

template<typename TF256, typename TF128>
struct FloatArrayDeserializer<std::uint16_t, TF256, TF128> {

    template<class TArchive>
    void operator()(FloatArray& instance, TArchive& archive) {
        archive(instance.fp32);
        assert(instance.fp32.size() % TF128::size() == 0);
        instance.fp16.resize(instance.fp32.size());
        for (std::uint32_t blkOffset = {}; blkOffset < instance.fp32.size();
             blkOffset += static_cast<std::uint32_t>(TF128::size())) {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ < 7)
            TF128::fromAlignedSource(instance.fp32.data() + blkOffset).unalignedStore(instance.fp16.data() + blkOffset);
#else
            TF128::fromAlignedSource(instance.fp32.data() + blkOffset).alignedStore(instance.fp16.data() + blkOffset);
#endif
        }
        instance.fp32.clear();
    }
};

template<typename TF256, typename TF128>
struct FloatArraySerializer<float, TF256, TF128> {

    template<class TArchive>
    void operator()(FloatArray& instance, TArchive& archive) {
        archive(instance.fp32);
    }
};

template<typename TF256, typename TF128>
struct FloatArraySerializer<std::uint16_t, TF256, TF128> {

    template<class TArchive>
    void operator()(FloatArray& instance, TArchive& archive) {
        assert(instance.fp16.size() % TF128::size() == 0);
        instance.fp32.resize(instance.fp16.size());
        for (std::uint32_t blkOffset = {}; blkOffset < instance.fp16.size();
             blkOffset += static_cast<std::uint32_t>(TF128::size())) {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ < 7)
            TF128::fromAlignedSource(instance.fp16.data() + blkOffset).unalignedStore(instance.fp32.data() + blkOffset);
#else
            TF128::fromAlignedSource(instance.fp16.data() + blkOffset).alignedStore(instance.fp32.data() + blkOffset);
#endif
        }
        archive(instance.fp32);
        instance.fp32.clear();
    }
};

}  // namespace rl4
