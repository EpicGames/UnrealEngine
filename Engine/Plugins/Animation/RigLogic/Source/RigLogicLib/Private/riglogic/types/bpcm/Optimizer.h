// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "riglogic/system/simd/SIMD.h"
#include "riglogic/types/Extent.h"
#include "riglogic/utils/Extd.h"
#include "riglogic/utils/Macros.h"

#include <cstdint>

namespace rl4 {

namespace bpcm {

template<typename TFVec, std::uint32_t BlockHeight, std::uint32_t PadTo, std::uint32_t Stride>
struct Optimizer {
    static_assert(BlockHeight % TFVec::size() == 0, "BlockHeight must be a multiple of TFVec::size()");
    static_assert(BlockHeight % Stride == 0, "BlockHeight must be a multiple of Stride");

    static std::uint32_t optimize(float* dest, const float* source, Extent dimensions) {
        const std::uint32_t remainder = dimensions.rows % BlockHeight;
        const std::uint32_t target = dimensions.rows - remainder;
        std::uint32_t offset = {};
        // Copy the portion of the source matrix that is partitionable into BlockHeight chunks
        for (std::uint32_t row = {}; row < target; row += BlockHeight) {
            for (std::uint32_t col = {}; col < dimensions.cols; ++col) {
                for (std::uint32_t base = {}; base < Stride; ++base) {
                    for (std::uint32_t blkIdx = base; blkIdx < BlockHeight; blkIdx += Stride, ++offset) {
                        dest[offset] = source[(row + blkIdx) * dimensions.cols + col];
                    }
                }
            }
        }
        // Copy the non-BlockHeight sized remainder portion of the source matrix
        const std::uint32_t paddedBlockHeight = extd::roundUp(remainder, PadTo);
        for (std::uint32_t row = target; row < dimensions.rows; row += remainder) {
            for (std::uint32_t col = 0u; col < dimensions.cols; ++col) {
                for (std::uint32_t base = {}; base < Stride; ++base) {
                    for (std::uint32_t blkIdx = base; blkIdx < remainder; blkIdx += Stride, ++offset) {
                        dest[offset] = source[(row + blkIdx) * dimensions.cols + col];
                    }
                    // Skip over padding (already filled with zeros) in destination storage
                    offset += (paddedBlockHeight - remainder) / Stride;
                }
            }
        }

        return offset;
    }

    static std::uint32_t optimize(std::uint16_t* dest, const float* source, Extent dimensions) {
        static constexpr std::size_t alignment = TFVec::alignment();
        const std::uint32_t remainder = dimensions.rows % BlockHeight;
        const std::uint32_t target = dimensions.rows - remainder;
        std::uint32_t offset = {};
        // Copy the portion of the source matrix that is partitionable into BlockHeight chunks
        for (std::uint32_t row = {}; row < target; row += BlockHeight) {
            for (std::uint32_t col = {}; col < dimensions.cols; ++col) {
                alignas(alignment) float input[BlockHeight];
                alignas(alignment) std::uint16_t output[BlockHeight];

                for (std::uint32_t base = {}, inIdx = {}; base < Stride; ++base) {
                    for (std::uint32_t blkIdx = base; blkIdx < BlockHeight; blkIdx += Stride) {
                        input[inIdx++] = source[(row + blkIdx) * dimensions.cols + col];
                    }
                }

                for (std::uint32_t blkOffset = {}; blkOffset < BlockHeight;
                     blkOffset += static_cast<std::uint32_t>(TFVec::size())) {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ < 7)
                    TFVec::fromAlignedSource(input + blkOffset).unalignedStore(output + blkOffset);
#else
                    TFVec::fromAlignedSource(input + blkOffset).alignedStore(output + blkOffset);
#endif
                }

                for (std::uint32_t blkIdx = {}; blkIdx < BlockHeight; ++blkIdx, ++offset) {
                    dest[offset] = output[blkIdx];
                }
            }
        }
        // Copy the non-BlockHeight sized remainder portion of the source matrix
        const std::uint32_t paddedBlockHeight = extd::roundUp(remainder, PadTo);
        for (std::uint32_t row = target; row < dimensions.rows; row += remainder) {
            for (std::uint32_t col = {}; col < dimensions.cols; ++col) {
                alignas(alignment) float input[BlockHeight] = {};
                alignas(alignment) std::uint16_t output[BlockHeight] = {};

                for (std::uint32_t base = {}, inIdx = {}; base < Stride; ++base) {
                    for (std::uint32_t blkIdx = base; blkIdx < remainder; blkIdx += Stride) {
                        input[inIdx++] = source[(row + blkIdx) * dimensions.cols + col];
                    }
                    inIdx += (paddedBlockHeight - remainder) / Stride;
                }

                for (std::uint32_t blkOffset = {}; blkOffset < BlockHeight;
                     blkOffset += static_cast<std::uint32_t>(TFVec::size())) {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ < 7)
                    TFVec::fromAlignedSource(input + blkOffset).unalignedStore(output + blkOffset);
#else
                    TFVec::fromAlignedSource(input + blkOffset).alignedStore(output + blkOffset);
#endif
                }

                // GCC-7 has a bug in its `array-bounds` check, and incorrectly warns about OOB access in the
                // below loop on `output[blkIdx]`, but `blkIdx` can never be higher than `paddedBlockHeight`,
                // which in turn is always either below or equal to `BlockHeight`, thus OOB is impossible.
                ASSUME_TRUE(paddedBlockHeight <= BlockHeight);
                // Copy even the padded zero values as the representation of half-floats might
                // differ from the storage's zero-initialized state
                for (std::uint32_t blkIdx = {}; blkIdx < paddedBlockHeight; ++blkIdx, ++offset) {
                    dest[offset] = output[blkIdx];
                }
            }
        }

        return offset;
    }
};

}  // namespace bpcm

}  // namespace rl4
