// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/version/VersionInfo.h"

#include "riglogic/version/Version.h"

#include <cstring>

namespace rl4 {

namespace {

constexpr int majorVersion = RL_MAJOR_VERSION;
constexpr int minorVersion = RL_MINOR_VERSION;
constexpr int patchVersion = RL_PATCH_VERSION;
constexpr const char* versionString = RL_VERSION_STRING;

}  // namespace

int VersionInfo::getMajorVersion() {
    return majorVersion;
}

int VersionInfo::getMinorVersion() {
    return minorVersion;
}

int VersionInfo::getPatchVersion() {
    return patchVersion;
}

StringView VersionInfo::getVersionString() {
    return {versionString, std::strlen(versionString)};
}

}  // namespace rl4
