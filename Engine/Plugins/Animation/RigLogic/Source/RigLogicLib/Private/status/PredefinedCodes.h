// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "status/StatusCode.h"

namespace sc {

static const StatusCode OK{0, "Ok"};

}  // namespace sc
