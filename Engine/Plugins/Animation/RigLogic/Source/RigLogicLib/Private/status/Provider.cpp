// Copyright Epic Games, Inc. All Rights Reserved.

#include "status/Provider.h"

#include "status/Registry.h"
#include "status/StatusCode.h"
#include "status/Storage.h"

#include <cassert>

namespace sc {

StatusProvider::StatusProvider(std::initializer_list<StatusCode> statuses) {
    // The Release build will eliminate this call, as it's really just a sanity check
    // to avoid defining duplicate error codes
    assert(StatusCodeRegistry::insert(statuses));
    // Avoid warning in Release builds
    static_cast<void>(statuses);
}

void StatusProvider::reset() {
    StatusStorage::reset();
}

StatusCode StatusProvider::get() {
    return StatusStorage::get();
}

bool StatusProvider::isOk() {
    return StatusStorage::isOk();
}

void StatusProvider::set(StatusCode status) {
    status.message = execHook(status, 0ul, status.message);
    execSet(status);
}

void StatusProvider::execSet(StatusCode status) {
    StatusStorage::set(status);
}

const char* StatusProvider::execHook(StatusCode status, std::size_t index, const char* data) {
    auto hook = StatusStorage::getHook();
    return (hook == nullptr ? data : hook(status, index, data));
}

}  // namespace sc
