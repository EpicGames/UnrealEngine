// Copyright Epic Games, Inc. All Rights Reserved.

#include "status/Registry.h"

#include "status/PredefinedCodes.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <mutex>
#include <thread>
#include <unordered_set>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace sc {

static std::unordered_set<int>& getRegistry() {
#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif
    static std::unordered_set<int> registry;
#ifdef __clang__
    #pragma clang diagnostic pop
#endif
    return registry;
}

bool StatusCodeRegistry::insert(std::initializer_list<StatusCode> statuses) {
    auto& registry = getRegistry();
#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif
    static std::mutex mut;
#ifdef __clang__
    #pragma clang diagnostic pop
#endif
    std::lock_guard<std::mutex> lock{mut};
    // Zero is a reserved code
    registry.insert(OK.code);
    bool allUnique = true;
    for (auto s : statuses) {
        allUnique = allUnique && registry.insert(s.code).second;
    }
    return allUnique;
}

bool StatusCodeRegistry::contains(StatusCode status) {
    auto& registry = getRegistry();
    return (registry.find(status.code) != registry.end());
}

}  // namespace sc
