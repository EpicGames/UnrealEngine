// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "status/StatusCode.h"

#include <initializer_list>

namespace sc {

class StatusCodeRegistry {
public:
    static bool insert(std::initializer_list<StatusCode> statuses);
    static bool contains(StatusCode status);
};

}  // namespace sc
