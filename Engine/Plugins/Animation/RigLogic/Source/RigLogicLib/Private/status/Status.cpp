// Copyright Epic Games, Inc. All Rights Reserved.

#include "status/Status.h"

#include "status/Storage.h"

namespace sc {

StatusCode Status::get() {
    return StatusStorage::get();
}

bool Status::isOk() {
    return StatusStorage::isOk();
}

HookFunction Status::getHook() {
    return StatusStorage::getHook();
}

void Status::setHook(HookFunction hook) {
    StatusStorage::setHook(hook);
}

}  // namespace sc
