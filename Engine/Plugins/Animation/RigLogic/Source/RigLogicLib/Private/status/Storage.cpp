// Copyright Epic Games, Inc. All Rights Reserved.

#include "status/Storage.h"

#include "status/PredefinedCodes.h"
#include "status/Registry.h"
#include "status/StatusCode.h"

#include <cassert>
#include <cstddef>
#include <cstring>

namespace sc {

namespace {

struct StatusCodeStorage {
    int code;
    char message[512];
    HookFunction hook;
};

void strcopy(char* destination, const char* source, std::size_t bufferSize) {
#ifdef _MSC_VER
    strncpy_s(destination, bufferSize, source, bufferSize - 1ul);
#else
    std::strncpy(destination, source, bufferSize - 1ul);
#endif
}

}  // namespace

thread_local static StatusCodeStorage currentStatus{0, "Ok", nullptr};

void StatusStorage::set(StatusCode status) {
    // The Release build will eliminate this call, as it's really just a sanity check
    // to avoid using unregistered status codes
    assert(StatusCodeRegistry::contains(status));
    currentStatus.code = status.code;
    strcopy(currentStatus.message, status.message, bufferSize());
}

void StatusStorage::reset() {
    currentStatus.code = OK.code;
    strcopy(currentStatus.message, OK.message, bufferSize());
}

StatusCode StatusStorage::get() {
    return {currentStatus.code, currentStatus.message};
}

bool StatusStorage::isOk() {
    return (currentStatus.code == OK.code);
}

constexpr std::size_t StatusStorage::bufferSize() {
    return sizeof(currentStatus.message);
}

HookFunction StatusStorage::getHook() {
    return currentStatus.hook;
}

void StatusStorage::setHook(HookFunction hook) {
    currentStatus.hook = hook;
}

}  // namespace sc
