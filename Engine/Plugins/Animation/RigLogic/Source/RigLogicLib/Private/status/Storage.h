// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "status/Status.h"
#include "status/StatusCode.h"

#include <cstddef>

namespace sc {

class StatusStorage {
public:
    static void set(StatusCode status);
    static void reset();
    static StatusCode get();
    static bool isOk();
    static constexpr std::size_t bufferSize();
    static HookFunction getHook();
    static void setHook(HookFunction hook);
};

}  // namespace sc
