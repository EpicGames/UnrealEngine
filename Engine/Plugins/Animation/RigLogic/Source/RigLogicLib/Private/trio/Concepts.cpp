// Copyright Epic Games, Inc. All Rights Reserved.

#include "trio/Concepts.h"

namespace trio {

Readable::~Readable() = default;
Writable::~Writable() = default;
Seekable::~Seekable() = default;
Openable::~Openable() = default;
Closeable::~Closeable() = default;
Controllable::~Controllable() = default;
Bounded::~Bounded() = default;
Buffered::~Buffered() = default;
Resizable::~Resizable() = default;

}  // namespace trio
