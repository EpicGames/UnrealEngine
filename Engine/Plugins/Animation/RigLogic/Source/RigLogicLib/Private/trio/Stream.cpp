// Copyright Epic Games, Inc. All Rights Reserved.

#include "trio/Stream.h"

namespace trio {

const sc::StatusCode BoundedIOStream::OpenError{100, "Error opening file"};
const sc::StatusCode BoundedIOStream::ReadError{101, "Error reading file"};
const sc::StatusCode BoundedIOStream::WriteError{102, "Error writing file"};
const sc::StatusCode BoundedIOStream::AlreadyOpenError{103, "File already open"};
const sc::StatusCode BoundedIOStream::SeekError{104, "Error seeking file"};

BoundedIOStream::~BoundedIOStream() = default;

}  // namespace trio
