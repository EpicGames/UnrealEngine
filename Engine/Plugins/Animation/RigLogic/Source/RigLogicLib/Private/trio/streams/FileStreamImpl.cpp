// Copyright Epic Games, Inc. All Rights Reserved.

#include "trio/streams/FileStreamImpl.h"

#include "trio/utils/NativeString.h"
#include "trio/utils/ScopedEnumEx.h"

#include <pma/PolyAllocator.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <ios>
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trio {

namespace {

constexpr std::size_t bufferSize = 4096ul;

inline std::uint64_t getFileSizeStd(const NativeCharacter* path) {
    std::streamoff fileSize = std::ifstream(path, std::ios_base::ate | std::ios_base::binary).tellg();
    return (fileSize > 0 ? static_cast<std::uint64_t>(fileSize) : 0ul);
}

inline void ensureFileExistsStd(const NativeCharacter* path) {
    std::fstream file;
    file.open(path);
    if (file.fail()) {
        // File does not exist, create it
        file.open(path, std::ios_base::out);
    }
}

}  // namespace

FileStream::~FileStream() = default;

FileStream* FileStream::create(const char* path, AccessMode accessMode, OpenMode openMode, MemoryResource* memRes) {
    pma::PolyAllocator<FileStreamImpl> alloc{memRes};
    return alloc.newObject(path, accessMode, openMode, memRes);
}

void FileStream::destroy(FileStream* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto stream = static_cast<FileStreamImpl*>(instance);
    pma::PolyAllocator<FileStreamImpl> alloc{stream->getMemoryResource()};
    alloc.deleteObject(stream);
}

FileStreamImpl::FileStreamImpl(const char* path_, AccessMode accessMode_, OpenMode openMode_, MemoryResource* memRes_) :
    filePath{NativeStringConverter::from(path_, memRes_)},
    fileAccessMode{accessMode_},
    fileOpenMode{openMode_},
    fileSize{getFileSizeStd(filePath.c_str())},
    filePos{},
    memRes{memRes_} {
}

void FileStreamImpl::open() {
    status->reset();
    if (file.is_open()) {
        status->set(AlreadyOpenError, filePath.c_str());
        return;
    }

    std::ios_base::openmode flags{};
    flags |= (contains(fileAccessMode, AccessMode::Read) ? std::ios_base::in : flags);
    flags |= (contains(fileAccessMode, AccessMode::Write) ? std::ios_base::out : flags);
    flags |= (contains(fileOpenMode, OpenMode::Binary) ? std::ios_base::binary : flags);
    flags |= std::ios_base::ate;

    if (fileAccessMode == AccessMode::ReadWrite) {
        ensureFileExistsStd(filePath.c_str());
    }

    file.open(filePath.c_str(), flags);
    if (!file.good()) {
        status->set(OpenError, filePath.c_str());
        return;
    }
    fileSize = static_cast<std::uint64_t>(file.tellg());
    seek(0ul);
}

void FileStreamImpl::close() {
    file.close();
}

std::uint64_t FileStreamImpl::tell() {
    return filePos;
}

void FileStreamImpl::seek(std::uint64_t position) {
    const bool seekable = (position == 0ul || position <= size()) && file.good();
    if (!seekable) {
        status->set(SeekError, filePath.c_str());
        return;
    }

    file.seekp(static_cast<std::streamoff>(position));
    filePos = position;
    if (!file.good()) {
        status->set(SeekError, filePath.c_str());
    }
}

std::size_t FileStreamImpl::read(char* destination, std::size_t size) {
    if (file.eof()) {
        return 0ul;
    }

    if ((destination == nullptr) || file.fail() || !file.is_open() || !contains(fileAccessMode, AccessMode::Read)) {
        status->set(ReadError, filePath.c_str());
        return 0ul;
    }

    file.read(destination, static_cast<std::streamsize>(size));
    if (file.fail() && !file.eof()) {
        status->set(ReadError, filePath.c_str());
    }

    const auto bytesRead = file.gcount();
    if (bytesRead < 0) {
        return 0ul;
    }

    filePos += static_cast<std::size_t>(bytesRead);
    return static_cast<std::size_t>(bytesRead);
}

std::size_t FileStreamImpl::read(Writable* destination, std::size_t size) {
    if (file.eof()) {
        return 0ul;
    }

    if ((destination == nullptr) || file.fail() || !file.is_open() || !contains(fileAccessMode, AccessMode::Read)) {
        status->set(ReadError, filePath.c_str());
        return 0ul;
    }

    char buffer[bufferSize];
    while (size > bufferSize) {
        file.read(buffer, static_cast<std::streamsize>(bufferSize));
        destination->write(buffer, bufferSize);
        size -= bufferSize;
    }
    file.read(buffer, static_cast<std::streamsize>(size));
    destination->write(buffer, size);

    if (file.fail() && !file.eof()) {
        status->set(ReadError, filePath.c_str());
    }

    const auto bytesRead = file.gcount();
    if (bytesRead < 0) {
        return 0ul;
    }

    filePos += static_cast<std::size_t>(bytesRead);
    return static_cast<std::size_t>(bytesRead);
}

std::size_t FileStreamImpl::write(const char* source, std::size_t size) {
    if ((source == nullptr) || file.fail() || !file.is_open() || !contains(fileAccessMode, AccessMode::Write)) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

#if defined(TRIO_MSVC_READ_AFTER_WRITE_WORKAROUND) && defined(_MSC_VER) && (_MSC_VER < 1930)
    // MSVC stdlib or compiler bug workaround
    file.seekp(file.tellp());
#endif  // _MSC_VER

    file.write(source, static_cast<std::streamsize>(size));
    if (!file.good()) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    fileSize = std::max(filePos + size, fileSize);
    filePos += size;
    return size;
}

std::size_t FileStreamImpl::write(Readable* source, std::size_t size) {
    if ((source == nullptr) || file.fail() || !file.is_open() || !contains(fileAccessMode, AccessMode::Write)) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

#if defined(TRIO_MSVC_READ_AFTER_WRITE_WORKAROUND) && defined(_MSC_VER) && (_MSC_VER < 1930)
    // MSVC stdlib or compiler bug workaround
    file.seekp(file.tellp());
#endif  // _MSC_VER

    char buffer[bufferSize];
    while (size > bufferSize) {
        source->read(buffer, bufferSize);
        file.write(buffer, static_cast<std::streamsize>(bufferSize));
        size -= bufferSize;
    }
    source->read(buffer, size);
    file.write(buffer, static_cast<std::streamsize>(size));
    if (!file.good()) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    fileSize = std::max(filePos + size, fileSize);
    filePos += size;
    return size;
}

std::uint64_t FileStreamImpl::size() {
    return fileSize;
}

MemoryResource* FileStreamImpl::getMemoryResource() {
    return memRes;
}

}  // namespace trio
