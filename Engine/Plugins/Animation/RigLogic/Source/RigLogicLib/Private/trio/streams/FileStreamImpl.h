// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/streams/FileStream.h"
#include "trio/streams/StreamStatus.h"
#include "trio/types/Aliases.h"
#include "trio/utils/NativeString.h"

#include <pma/TypeDefs.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <cstdint>
#include <fstream>
#include <ios>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trio {

class FileStreamImpl : public FileStream {
public:
    FileStreamImpl(const char* path_, AccessMode accessMode_, OpenMode openMode_, MemoryResource* memRes_);

    void open() override;
    void close() override;
    std::uint64_t tell() override;
    void seek(std::uint64_t position) override;
    std::uint64_t size() override;
    std::size_t read(char* destination, std::size_t size) override;
    std::size_t read(Writable* destination, std::size_t size) override;
    std::size_t write(const char* source, std::size_t size) override;
    std::size_t write(Readable* source, std::size_t size) override;

    MemoryResource* getMemoryResource();

private:
    std::fstream file;
    NativeString filePath;
    AccessMode fileAccessMode;
    OpenMode fileOpenMode;
    std::uint64_t fileSize;
    std::uint64_t filePos;
    MemoryResource* memRes;
    StreamStatus status;
};

}  // namespace trio
