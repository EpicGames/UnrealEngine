// Copyright Epic Games, Inc. All Rights Reserved.

#include "trio/streams/MemoryMappedFileStream.h"

#include "trio/streams/MemoryMappedFileStreamFallback.h"
#include "trio/streams/MemoryMappedFileStreamUnix.h"
#include "trio/streams/MemoryMappedFileStreamWindows.h"

#include <pma/PolyAllocator.h>

namespace trio {

#if defined(TRIO_WINDOWS_FILE_MAPPING_AVAILABLE)
using MemoryMappedFileStreamImpl = MemoryMappedFileStreamWindows;
#elif defined(TRIO_MMAP_AVAILABLE)
using MemoryMappedFileStreamImpl = MemoryMappedFileStreamUnix;
#else
using MemoryMappedFileStreamImpl = MemoryMappedFileStreamFallback;
#endif

MemoryMappedFileStream* MemoryMappedFileStream::create(const char* path, AccessMode accessMode, MemoryResource* memRes) {
    pma::PolyAllocator<MemoryMappedFileStreamImpl> alloc{memRes};
    return alloc.newObject(path, accessMode, memRes);
}

void MemoryMappedFileStream::destroy(MemoryMappedFileStream* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto stream = static_cast<MemoryMappedFileStreamImpl*>(instance);
    pma::PolyAllocator<MemoryMappedFileStreamImpl> alloc{stream->getMemoryResource()};
    alloc.deleteObject(stream);
}

MemoryMappedFileStream::~MemoryMappedFileStream() = default;

}  // namespace trio
