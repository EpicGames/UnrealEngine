// Copyright Epic Games, Inc. All Rights Reserved.

// *INDENT-OFF*
#if !defined(TRIO_WINDOWS_FILE_MAPPING_AVAILABLE) && !defined(TRIO_MMAP_AVAILABLE)

    #include "trio/streams/MemoryMappedFileStreamFallback.h"

    #include "trio/utils/ScopedEnumEx.h"

    #include <pma/ScopedPtr.h>

    #include <cstddef>
    #include <cstdint>

namespace trio {

MemoryMappedFileStreamFallback::MemoryMappedFileStreamFallback(const char* path_,
                                                               AccessMode accessMode_,
                                                               MemoryResource* memRes_) :
    stream{pma::makeScoped<FileStream>(path_, accessMode_, OpenMode::Binary, memRes_)},
    memRes{memRes_} {
}

MemoryMappedFileStreamFallback::~MemoryMappedFileStreamFallback() {
    MemoryMappedFileStreamFallback::close();
}

void MemoryMappedFileStreamFallback::open() {
    stream->open();
}

void MemoryMappedFileStreamFallback::close() {
    stream->close();
}

std::uint64_t MemoryMappedFileStreamFallback::tell() {
    return stream->tell();
}

void MemoryMappedFileStreamFallback::seek(std::uint64_t position) {
    stream->seek(position);
}

std::size_t MemoryMappedFileStreamFallback::read(char* destination, std::size_t size) {
    return stream->read(destination, size);
}

std::size_t MemoryMappedFileStreamFallback::read(Writable* destination, std::size_t size) {
    return stream->read(destination, size);
}

std::size_t MemoryMappedFileStreamFallback::write(const char* source, std::size_t size) {
    return stream->write(source, size);
}

std::size_t MemoryMappedFileStreamFallback::write(Readable* source, std::size_t size) {
    return stream->write(source, size);
}

void MemoryMappedFileStreamFallback::flush() {
    // Unbuffered, so noop
}

void MemoryMappedFileStreamFallback::resize(std::uint64_t /*unused*/) {
    // No-op, as it's written to disk directly
}

std::uint64_t MemoryMappedFileStreamFallback::size() {
    return stream->size();
}

MemoryResource* MemoryMappedFileStreamFallback::getMemoryResource() {
    return memRes;
}

}  // namespace trio

#endif
// *INDENT-OFF*
