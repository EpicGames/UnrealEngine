// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#if !defined(TRIO_WINDOWS_FILE_MAPPING_AVAILABLE) && !defined(TRIO_MMAP_AVAILABLE)

    #include "trio/streams/FileStream.h"
    #include "trio/streams/MemoryMappedFileStream.h"
    #include "trio/streams/StreamStatus.h"
    #include "trio/types/Aliases.h"

    #include <pma/ScopedPtr.h>
    #include <pma/TypeDefs.h>

    #include <cstddef>
    #include <cstdint>

namespace trio {

class MemoryMappedFileStreamFallback : public MemoryMappedFileStream {
public:
    MemoryMappedFileStreamFallback(const char* path_, AccessMode accessMode_, MemoryResource* memRes_);
    ~MemoryMappedFileStreamFallback();

    MemoryMappedFileStreamFallback(const MemoryMappedFileStreamFallback&) = delete;
    MemoryMappedFileStreamFallback& operator=(const MemoryMappedFileStreamFallback&) = delete;

    MemoryMappedFileStreamFallback(MemoryMappedFileStreamFallback&&) = delete;
    MemoryMappedFileStreamFallback& operator=(MemoryMappedFileStreamFallback&&) = delete;

    void open() override;
    void close() override;
    std::uint64_t tell() override;
    void seek(std::uint64_t position) override;
    std::uint64_t size() override;
    std::size_t read(char* destination, std::size_t size) override;
    std::size_t read(Writable* destination, std::size_t size) override;
    std::size_t write(const char* source, std::size_t size) override;
    std::size_t write(Readable* source, std::size_t size) override;
    void flush() override;
    void resize(std::uint64_t size) override;

    MemoryResource* getMemoryResource();

private:
    pma::ScopedPtr<FileStream> stream;
    MemoryResource* memRes;
};

}  // namespace trio

#endif
// *INDENT-OFF*
