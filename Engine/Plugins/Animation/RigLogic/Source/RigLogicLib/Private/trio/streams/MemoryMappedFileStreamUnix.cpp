// Copyright Epic Games, Inc. All Rights Reserved.

// *INDENT-OFF*
#ifdef TRIO_MMAP_AVAILABLE

    #ifdef TRIO_LARGE_FILE_SUPPORT_AVAILABLE
        #define _FILE_OFFSET_BITS 64
    #endif  // TRIO_LARGE_FILE_SUPPORT

    #include "trio/streams/MemoryMappedFileStreamUnix.h"
    #include "trio/utils/NativeString.h"
    #include "trio/utils/ScopedEnumEx.h"

    #include <pma/PolyAllocator.h>

    #include <fcntl.h>
    #include <sys/mman.h>
    #include <sys/stat.h>
    #include <sys/types.h>
    #include <unistd.h>

    #ifdef _MSC_VER
        #pragma warning(push)
        #pragma warning(disable : 4365 4987)
    #endif
    #include <algorithm>
    #include <cassert>
    #include <cstdint>
    #include <cstdio>
    #include <cstring>
    #include <ios>
    #include <limits>
    #include <type_traits>
    #ifdef _MSC_VER
        #pragma warning(pop)
    #endif

namespace trio {

namespace {

constexpr std::size_t minViewSizeUnix = 65536ul;

inline std::uint64_t getFileSizeUnix(const NativeCharacter* path) {
    struct stat st{};
    if (::stat(path, &st) != 0) {
        return 0ul;
    }
    return static_cast<std::uint64_t>(st.st_size);
}

inline std::uint64_t getPageSizeUnix() {
    #ifdef TRIO_PAGE_SIZE_UNIX
    return static_cast<std::uint64_t>(TRIO_PAGE_SIZE_UNIX);
    #else
    return static_cast<std::uint64_t>(sysconf(_SC_PAGE_SIZE));
    #endif  // TRIO_PAGE_SIZE_UNIX
}

inline std::uint64_t alignOffsetUnix(std::uint64_t offset) {
    const std::uint64_t pageSize = getPageSizeUnix();
    return offset / pageSize * pageSize;
}

class MemoryReaderUnix : public Readable {
public:
    explicit MemoryReaderUnix(const char* source_) :
        source{source_} {
    }

    std::size_t read(char* destination, std::size_t size) override {
        std::memcpy(destination, source, size);
        source += size;
        return size;
    }

    std::size_t read(Writable* destination, std::size_t size) override {
        destination->write(source, size);
        source += size;
        return size;
    }

private:
    const char* source;
};

class MemoryWriterUnix : public Writable {
public:
    explicit MemoryWriterUnix(char* destination_) :
        destination{destination_} {
    }

    std::size_t write(const char* source, std::size_t size) override {
        std::memcpy(destination, source, size);
        destination += size;
        return size;
    }

    std::size_t write(Readable* source, std::size_t size) override {
        source->read(destination, size);
        destination += size;
        return size;
    }

private:
    char* destination;
};

}  // namespace

MemoryMappedFileStreamUnix::MemoryMappedFileStreamUnix(const char* path_, AccessMode accessMode_, MemoryResource* memRes_) :
    filePath{NativeStringConverter::from(path_, memRes_)},
    fileAccessMode{accessMode_},
    memRes{memRes_},
    file{-1},
    data{nullptr},
    position{},
    fileSize{getFileSizeUnix(filePath.c_str())},
    viewOffset{},
    viewSize{},
    delayedMapping{false},
    dirty{false} {
}

MemoryMappedFileStreamUnix::~MemoryMappedFileStreamUnix() {
    MemoryMappedFileStreamUnix::close();
}

MemoryResource* MemoryMappedFileStreamUnix::getMemoryResource() {
    return memRes;
}

std::uint64_t MemoryMappedFileStreamUnix::size() {
    return fileSize;
}

void MemoryMappedFileStreamUnix::open() {
    status->reset();
    if (file != -1) {
        status->set(AlreadyOpenError, filePath.c_str());
        return;
    }

    delayedMapping = false;

    openFile();
    if (file == -1) {
        status->set(OpenError, filePath.c_str());
        return;
    }

    struct stat st{};
    if (::fstat(file, &st) != 0) {
        fileSize = 0ul;
        closeFile();
        status->set(OpenError, filePath.c_str());
        return;
    }

    fileSize = static_cast<std::uint64_t>(st.st_size);
    // Mapping of 0-length files is delayed until the file is resized to a non-zero size.
    delayedMapping = (fileSize == 0ul);
    if (delayedMapping) {
        return;
    }

    mapFile(0ul, fileSize);
    if (data == reinterpret_cast<void*>(-1)) {
        status->set(OpenError, filePath.c_str());
        delayedMapping = false;
        unmapFile();
        closeFile();
        return;
    }

    MemoryMappedFileStreamUnix::seek(0ul);
    dirty = false;
}

void MemoryMappedFileStreamUnix::close() {
    delayedMapping = false;

    flush();
    unmapFile();
    closeFile();
}

std::uint64_t MemoryMappedFileStreamUnix::tell() {
    return position;
}

void MemoryMappedFileStreamUnix::seek(std::uint64_t position_) {
    const bool seekable = ((position_ == 0ul) || (position_ <= size())) && (data != nullptr);
    if (!seekable) {
        status->set(SeekError, filePath.c_str());
        return;
    }

    position = position_;
    if ((position < viewOffset) || (position >= (viewOffset + viewSize))) {
        flush();
        if (dirty) {
            return;
        }
        unmapFile();
        mapFile(position, fileSize - position);
    }
}

std::size_t MemoryMappedFileStreamUnix::read(char* destination, std::size_t size) {
    if (destination == nullptr) {
        status->set(ReadError, filePath.c_str());
        return 0ul;
    }

    MemoryWriterUnix writer{destination};
    return read(&writer, size);
}

std::size_t MemoryMappedFileStreamUnix::read(Writable* destination, std::size_t size) {
    if ((destination == nullptr) || !contains(fileAccessMode, AccessMode::Read)) {
        status->set(ReadError, filePath.c_str());
        return 0ul;
    }

    if (data == nullptr) {
        if (!delayedMapping) {
            status->set(ReadError, filePath.c_str());
        }
        return 0ul;
    }

    const std::uint64_t bytesAvailable = fileSize - position;
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
    const std::size_t bytesToRead = static_cast<std::size_t>(std::min(static_cast<std::uint64_t>(size), bytesAvailable));
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    std::size_t bytesRead = 0ul;

    while (bytesRead != bytesToRead) {
        const std::size_t bytesRemaining = bytesToRead - bytesRead;
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
        std::size_t viewPosition = static_cast<std::size_t>(position - viewOffset);
        std::size_t bytesReadable = static_cast<std::size_t>(viewSize - viewPosition);
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
        // If the view is exhausted during reading, remap a new view till the end of file if possible,
        // starting at the current position
        if (bytesReadable == 0ul) {
            unmapFile();
            mapFile(position, fileSize - position);
            if (data == nullptr) {
                // Failed to map new view
                status->set(ReadError, filePath.c_str());
                break;
            }
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
            viewPosition = static_cast<std::size_t>(position - viewOffset);
            bytesReadable = static_cast<std::size_t>(viewSize - viewPosition);
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
        }

        const std::size_t chunkSize = std::min(bytesRemaining, bytesReadable);
        const std::size_t chunkCopied = destination->write(static_cast<char*>(data) + viewPosition, chunkSize);
        bytesRead += chunkCopied;
        position += chunkCopied;
    }

    return bytesRead;
}

std::size_t MemoryMappedFileStreamUnix::write(const char* source, std::size_t size) {
    if (source == nullptr) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    MemoryReaderUnix reader{source};
    return write(&reader, size);
}

std::size_t MemoryMappedFileStreamUnix::write(Readable* source, std::size_t size) {
    if ((source == nullptr) || !contains(fileAccessMode, AccessMode::Write)) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    if ((data == nullptr) && !delayedMapping) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    if (size == 0ul) {
        return 0ul;
    }

    if (position + size > fileSize) {
        resize(position + size);
        if (fileSize != (position + size)) {
            // Resize not successful (resize sets status in such cases)
            return 0ul;
        }
    }

    std::size_t bytesWritten = 0ul;

    while (bytesWritten != size) {
        const std::size_t bytesRemaining = size - bytesWritten;
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
        std::size_t viewPosition = static_cast<std::size_t>(position - viewOffset);
        std::size_t bytesWritable = static_cast<std::size_t>(viewSize - viewPosition);
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
        // If the view is exhausted during writing, remap a new view till the end of file if possible,
        // starting at the current position
        if (bytesWritable == 0ul) {
            flush();
            if (dirty) {
                break;
            }
            unmapFile();
            mapFile(position, fileSize - position);
            if (data == nullptr) {
                // Failed to map new view
                status->set(WriteError, filePath.c_str());
                break;
            }
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
            viewPosition = static_cast<std::size_t>(position - viewOffset);
            bytesWritable = static_cast<std::size_t>(viewSize - viewPosition);
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
        }

        const std::size_t chunkSize = std::min(bytesRemaining, bytesWritable);
        const std::size_t chunkCopied = source->read(static_cast<char*>(data) + viewPosition, chunkSize);
        bytesWritten += chunkCopied;
        position += chunkCopied;
    }

    dirty = (bytesWritten > 0ul);

    return bytesWritten;
}

void MemoryMappedFileStreamUnix::flush() {
    if (data != nullptr) {
        if (::msync(data, viewSize, MS_SYNC) != 0) {
            status->set(WriteError, filePath.c_str());
            return;
        }
    }
    dirty = false;
}

void MemoryMappedFileStreamUnix::resize(std::uint64_t size) {
    if (fileAccessMode == AccessMode::Read) {
        status->set(WriteError, filePath.c_str());
        return;
    }

    flush();
    if (dirty) {
        return;
    }

    unmapFile();
    resizeFile(size);
    if (fileSize != size) {
        status->set(WriteError, filePath.c_str());
        return;
    }

    // Either mremap is not available, or there was no data pointer to be remapped in the
    // first place. In both cases, fallback to mmap
    mapFile(position, fileSize - position);
    if (data == nullptr) {
        status->set(WriteError, filePath.c_str());
        return;
    }
}

void MemoryMappedFileStreamUnix::openFile() {
    int openFlags{};
    if (fileAccessMode == AccessMode::ReadWrite) {
        openFlags = O_RDWR | O_CREAT;
    } else if (fileAccessMode == AccessMode::Read) {
        openFlags = O_RDONLY;
    } else if (fileAccessMode == AccessMode::Write) {
        // mmap needs also read permission to the underlying file descriptor
        openFlags = O_RDWR | O_CREAT;
    }

    const int mode = (S_IRUSR | S_IWUSR | S_IRGRP | S_IROTH);
    file = ::open(filePath.c_str(), openFlags, mode);
}

void MemoryMappedFileStreamUnix::closeFile() {
    if (file != -1) {
        ::close(file);
        file = -1;
    }
}

void MemoryMappedFileStreamUnix::mapFile(std::uint64_t offset, std::uint64_t size) {
    int prot{};
    prot |= (contains(fileAccessMode, AccessMode::Write) ? PROT_WRITE : prot);
    prot |= (contains(fileAccessMode, AccessMode::Read) ? PROT_READ : prot);

    const int flags = (fileAccessMode == AccessMode::Read ? MAP_PRIVATE : MAP_SHARED);

    const std::uint64_t alignedOffset = alignOffsetUnix(offset);

    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
    // Make sure size does not exceed system limits
    std::size_t safeSize =
        static_cast<std::size_t>(std::min(static_cast<std::uint64_t>(std::numeric_limits<std::size_t>::max()), size));
    // Increase to-be-mapped size by the difference caused by alignment (guard against wrap-around on overflow)
    safeSize = std::max(static_cast<std::size_t>(safeSize + (offset - alignedOffset)), safeSize);
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif

    // Try mapping requested size, but if it fails keep repeating by halving the view size each time (e.g. if not enough VA space)
    std::size_t nextSize = safeSize;
    do {
        safeSize = nextSize;
        data = ::mmap(nullptr, safeSize, prot, flags, file, static_cast<off_t>(alignedOffset));
        if (data != reinterpret_cast<void*>(-1)) {
            break;
        }
        nextSize = safeSize / 2ul;
    } while (nextSize > minViewSizeUnix);

    if (data != reinterpret_cast<void*>(-1)) {
        viewOffset = alignedOffset;
        viewSize = safeSize;
    }
}

void MemoryMappedFileStreamUnix::unmapFile() {
    if (data != nullptr) {
        ::munmap(data, viewSize);
        data = nullptr;
    }

    viewOffset = 0ul;
    viewSize = 0ul;
}

void MemoryMappedFileStreamUnix::resizeFile(std::uint64_t size) {
    if (file == -1) {
        return;
    }

    if (::ftruncate(file, static_cast<off_t>(size)) != 0) {
        return;
    }

    fileSize = size;
}

}  // namespace trio

#endif  // TRIO_MMAP_AVAILABLE
// *INDENT-ON*
