// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#ifdef TRIO_MMAP_AVAILABLE

    #include "trio/streams/MemoryMappedFileStream.h"
    #include "trio/streams/StreamStatus.h"
    #include "trio/types/Aliases.h"
    #include "trio/utils/NativeString.h"

    #include <pma/TypeDefs.h>

    #include <cstdint>

namespace trio {

class MemoryMappedFileStreamUnix : public MemoryMappedFileStream {
public:
    MemoryMappedFileStreamUnix(const char* path_, AccessMode accessMode_, MemoryResource* memRes_);
    ~MemoryMappedFileStreamUnix();

    MemoryMappedFileStreamUnix(const MemoryMappedFileStreamUnix&) = delete;
    MemoryMappedFileStreamUnix& operator=(const MemoryMappedFileStreamUnix&) = delete;

    MemoryMappedFileStreamUnix(MemoryMappedFileStreamUnix&&) = delete;
    MemoryMappedFileStreamUnix& operator=(MemoryMappedFileStreamUnix&&) = delete;

    void open() override;
    void close() override;
    std::uint64_t tell() override;
    void seek(std::uint64_t position) override;
    std::uint64_t size() override;
    std::size_t read(char* destination, std::size_t size) override;
    std::size_t read(Writable* destination, std::size_t size) override;
    std::size_t write(const char* source, std::size_t size) override;
    std::size_t write(Readable* source, std::size_t size) override;
    void flush() override;
    void resize(std::uint64_t size) override;

    MemoryResource* getMemoryResource();

private:
    void openFile();
    void closeFile();
    void mapFile(std::uint64_t offset, std::uint64_t size);
    void unmapFile();
    void resizeFile(std::uint64_t size);

private:
    StreamStatus status;
    NativeString filePath;
    AccessMode fileAccessMode;
    MemoryResource* memRes;
    int file;
    void* data;
    std::uint64_t position;
    std::uint64_t fileSize;
    std::uint64_t viewOffset;
    std::size_t viewSize;
    bool delayedMapping;
    bool dirty;
};

}  // namespace trio

#endif  // TRIO_MMAP_AVAILABLE
// *INDENT-ON*
