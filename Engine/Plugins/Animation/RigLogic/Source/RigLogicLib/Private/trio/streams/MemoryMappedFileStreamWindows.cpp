// Copyright Epic Games, Inc. All Rights Reserved.

// *INDENT-OFF*
#ifdef TRIO_WINDOWS_FILE_MAPPING_AVAILABLE

    #include "trio/streams/MemoryMappedFileStreamWindows.h"

    #include "trio/utils/NativeString.h"
    #include "trio/utils/ScopedEnumEx.h"

    #include <pma/PolyAllocator.h>

    #ifdef _MSC_VER
        #pragma warning(push)
        #pragma warning(disable : 4365 4987)
    #endif
    #include <algorithm>
    #include <cassert>
    #include <cstddef>
    #include <cstdint>
    #include <cstring>
    #include <ios>
    #include <limits>
    #include <type_traits>
    #ifdef _MSC_VER
        #pragma warning(pop)
    #endif

namespace trio {

namespace {

constexpr std::size_t minViewSizeWindows = 65536ul;

inline std::uint64_t getFileSizeWindows(const NativeCharacter* path) {
    WIN32_FILE_ATTRIBUTE_DATA w32fad;
    if (GetFileAttributesEx(path, GetFileExInfoStandard, &w32fad) == 0) {
        return 0ul;
    }
    ULARGE_INTEGER size;
    size.HighPart = w32fad.nFileSizeHigh;
    size.LowPart = w32fad.nFileSizeLow;
    return static_cast<std::uint64_t>(size.QuadPart);
}

inline std::uint64_t getPageSizeWindows() {
    SYSTEM_INFO SystemInfo;
    GetSystemInfo(&SystemInfo);
    return SystemInfo.dwAllocationGranularity;
}

inline std::uint64_t alignOffsetWindows(std::uint64_t offset) {
    const std::uint64_t pageSize = getPageSizeWindows();
    return offset / pageSize * pageSize;
}

class MemoryReaderWindows : public Readable {
public:
    explicit MemoryReaderWindows(const char* source_) :
        source{source_} {
    }

    std::size_t read(char* destination, std::size_t size) override {
        CopyMemory(destination, source, size);
        source += size;
        return size;
    }

    std::size_t read(Writable* destination, std::size_t size) override {
        destination->write(source, size);
        source += size;
        return size;
    }

private:
    const char* source;
};

class MemoryWriterWindows : public Writable {
public:
    explicit MemoryWriterWindows(char* destination_) :
        destination{destination_} {
    }

    std::size_t write(const char* source, std::size_t size) override {
        CopyMemory(destination, source, size);
        destination += size;
        return size;
    }

    std::size_t write(Readable* source, std::size_t size) override {
        source->read(destination, size);
        destination += size;
        return size;
    }

private:
    char* destination;
};

}  // namespace

MemoryMappedFileStreamWindows::MemoryMappedFileStreamWindows(const char* path_, AccessMode accessMode_, MemoryResource* memRes_) :
    filePath{NativeStringConverter::from(path_, memRes_)},
    fileAccessMode{accessMode_},
    memRes{memRes_},
    file{INVALID_HANDLE_VALUE},
    mapping{nullptr},
    data{nullptr},
    position{},
    fileSize{getFileSizeWindows(filePath.c_str())},
    viewOffset{},
    viewSize{},
    delayedMapping{false},
    dirty{false} {
}

MemoryMappedFileStreamWindows::~MemoryMappedFileStreamWindows() {
    delayedMapping = false;
    flush();
    unmapFile();
    closeFile();
}

MemoryResource* MemoryMappedFileStreamWindows::getMemoryResource() {
    return memRes;
}

std::uint64_t MemoryMappedFileStreamWindows::size() {
    return fileSize;
}

void MemoryMappedFileStreamWindows::open() {
    status->reset();
    if (file != INVALID_HANDLE_VALUE) {
        status->set(AlreadyOpenError, filePath.c_str());
        return;
    }

    delayedMapping = false;

    openFile();
    if (file == INVALID_HANDLE_VALUE) {
        status->set(OpenError, filePath.c_str());
        return;
    }

    // Retrieve file size
    LARGE_INTEGER size{};
    if (GetFileSizeEx(file, &size) == 0) {
        fileSize = 0ul;
        closeFile();
        status->set(OpenError, filePath.c_str());
        return;
    }

    fileSize = static_cast<std::uint64_t>(size.QuadPart);
    // Mapping of 0-length files is delayed until the file is resized to a non-zero size.
    delayedMapping = (fileSize == 0ul);
    if (delayedMapping) {
        return;
    }

    // Create file mapping
    mapFile(0ul, fileSize);
    if (data == nullptr) {
        status->set(OpenError, filePath.c_str());
        delayedMapping = false;
        unmapFile();
        closeFile();
        return;
    }

    seek(0ul);
    dirty = false;
}

void MemoryMappedFileStreamWindows::close() {
    delayedMapping = false;

    flush();
    unmapFile();
    closeFile();
}

std::uint64_t MemoryMappedFileStreamWindows::tell() {
    return position;
}

void MemoryMappedFileStreamWindows::seek(std::uint64_t position_) {
    const bool seekable = ((position_ == 0ul) || (position_ <= size())) && (data != nullptr);
    if (!seekable) {
        status->set(SeekError, filePath.c_str());
        return;
    }

    position = position_;
    if ((position < viewOffset) || (position >= (viewOffset + viewSize))) {
        flush();
        if (dirty) {
            return;
        }
        unmapFile();
        mapFile(position, fileSize - position);
    }
}

std::size_t MemoryMappedFileStreamWindows::read(char* destination, std::size_t size) {
    if (destination == nullptr) {
        status->set(ReadError, filePath.c_str());
        return 0ul;
    }

    MemoryWriterWindows writer{destination};
    return read(&writer, size);
}

std::size_t MemoryMappedFileStreamWindows::read(Writable* destination, std::size_t size) {
    if ((destination == nullptr) || !contains(fileAccessMode, AccessMode::Read)) {
        status->set(ReadError, filePath.c_str());
        return 0ul;
    }

    if (data == nullptr) {
        if (!delayedMapping) {
            status->set(ReadError, filePath.c_str());
        }
        return 0ul;
    }

    const std::uint64_t bytesAvailable = fileSize - position;
    const std::size_t bytesToRead = static_cast<std::size_t>(std::min(static_cast<std::uint64_t>(size), bytesAvailable));
    std::size_t bytesRead = 0ul;

    while (bytesRead != bytesToRead) {
        const std::size_t bytesRemaining = bytesToRead - bytesRead;
        std::size_t viewPosition = static_cast<std::size_t>(position - viewOffset);
        std::size_t bytesReadable = static_cast<std::size_t>(viewSize - viewPosition);
        // If the view is exhausted during reading, remap a new view till the end of file if possible,
        // starting at the current position
        if (bytesReadable == 0ul) {
            unmapFile();
            mapFile(position, fileSize - position);
            if (data == nullptr) {
                // Failed to map new view
                status->set(ReadError, filePath.c_str());
                break;
            }
            viewPosition = static_cast<std::size_t>(position - viewOffset);
            bytesReadable = static_cast<std::size_t>(viewSize - viewPosition);
        }

        const std::size_t chunkSize = std::min(bytesRemaining, bytesReadable);
        const std::size_t chunkCopied = destination->write(static_cast<char*>(data) + viewPosition, chunkSize);
        bytesRead += chunkCopied;
        position += chunkCopied;
    }

    return bytesRead;
}

std::size_t MemoryMappedFileStreamWindows::write(const char* source, std::size_t size) {
    if (source == nullptr) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    MemoryReaderWindows reader{source};
    return write(&reader, size);
}

std::size_t MemoryMappedFileStreamWindows::write(Readable* source, std::size_t size) {
    if ((source == nullptr) || !contains(fileAccessMode, AccessMode::Write)) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    if ((data == nullptr) && !delayedMapping) {
        status->set(WriteError, filePath.c_str());
        return 0ul;
    }

    if (size == 0ul) {
        return 0ul;
    }

    if (position + size > fileSize) {
        resize(position + size);
        if (fileSize != (position + size)) {
            // Resize not successful (resize sets status in such cases)
            return 0ul;
        }
    }

    std::size_t bytesWritten = 0ul;

    while (bytesWritten != size) {
        const std::size_t bytesRemaining = size - bytesWritten;
        std::size_t viewPosition = static_cast<std::size_t>(position - viewOffset);
        std::size_t bytesWritable = static_cast<std::size_t>(viewSize - viewPosition);
        // If the view is exhausted during writing, remap a new view till the end of file if possible,
        // starting at the current position
        if (bytesWritable == 0ul) {
            flush();
            if (dirty) {
                break;
            }
            unmapFile();
            mapFile(position, fileSize - position);
            if (data == nullptr) {
                // Failed to map new view
                status->set(WriteError, filePath.c_str());
                break;
            }
            viewPosition = static_cast<std::size_t>(position - viewOffset);
            bytesWritable = static_cast<std::size_t>(viewSize - viewPosition);
        }

        const std::size_t chunkSize = std::min(bytesRemaining, bytesWritable);
        const std::size_t chunkCopied = source->read(static_cast<char*>(data) + viewPosition, chunkSize);
        bytesWritten += chunkCopied;
        position += chunkCopied;
    }

    dirty = (bytesWritten > 0ul);

    return bytesWritten;
}

void MemoryMappedFileStreamWindows::flush() {
    if ((data != nullptr) && (fileAccessMode != AccessMode::Read)) {
        if (!FlushViewOfFile(data, 0ul)) {
            status->set(WriteError, filePath.c_str());
            return;
        }
    }
    dirty = false;
}

void MemoryMappedFileStreamWindows::resize(std::uint64_t size) {
    if (fileAccessMode == AccessMode::Read) {
        status->set(WriteError, filePath.c_str());
        return;
    }

    flush();
    if (dirty) {
        return;
    }

    unmapFile();

    resizeFile(size);
    if (fileSize != size) {
        status->set(WriteError, filePath.c_str());
        return;
    }

    // Remap file from current position, till the end of file
    mapFile(position, fileSize - position);
    if (data == nullptr) {
        status->set(WriteError, filePath.c_str());
        return;
    }
}

void MemoryMappedFileStreamWindows::openFile() {
    DWORD access{GENERIC_READ};
    access |= (contains(fileAccessMode, AccessMode::Write) ? GENERIC_WRITE : access);

    // 0 == no sharing in any way
    DWORD sharing{};

    // Non-existing files are created unless in read-only mode
    DWORD creationDisposition{};
    if (fileAccessMode == AccessMode::Read) {
        creationDisposition = static_cast<DWORD>(OPEN_EXISTING);
    } else {
        creationDisposition = static_cast<DWORD>(OPEN_ALWAYS);
    }

    file = CreateFile(filePath.c_str(), access, sharing, nullptr, creationDisposition, FILE_ATTRIBUTE_NORMAL, nullptr);
}

void MemoryMappedFileStreamWindows::closeFile() {
    if (file != INVALID_HANDLE_VALUE) {
        CloseHandle(file);
        file = INVALID_HANDLE_VALUE;
    }
}

void MemoryMappedFileStreamWindows::mapFile(std::uint64_t offset, std::uint64_t size) {
    // Create file mapping
    const auto protect = static_cast<DWORD>(contains(fileAccessMode, AccessMode::Write) ? PAGE_READWRITE : PAGE_READONLY);
    mapping = CreateFileMapping(file, nullptr, protect, 0u, 0u, nullptr);
    if (mapping == nullptr) {
        return;
    }

    // Map a view of the file mapping into the address space
    DWORD desiredAccess{};
    desiredAccess |= (contains(fileAccessMode, AccessMode::Write) ? FILE_MAP_WRITE : desiredAccess);
    desiredAccess |= (contains(fileAccessMode, AccessMode::Read) ? FILE_MAP_READ : desiredAccess);

    ULARGE_INTEGER alignedOffset{};
    alignedOffset.QuadPart = static_cast<decltype(alignedOffset.QuadPart)>(alignOffsetWindows(offset));

    // Make sure size does not exceed system limits
    std::size_t safeSize =
        static_cast<std::size_t>(std::min(static_cast<std::uint64_t>(std::numeric_limits<std::size_t>::max()), size));
    // Increase to-be-mapped size by the difference caused by alignment
    safeSize = std::max(static_cast<std::size_t>(safeSize + (offset - alignedOffset.QuadPart)), safeSize);

    // Try mapping requested size, but if it fails keep repeating by halving the view size each time (e.g. if not enough VA space)
    std::size_t nextSize = safeSize;
    do {
        safeSize = nextSize;
        data = MapViewOfFile(mapping, desiredAccess, alignedOffset.HighPart, alignedOffset.LowPart, safeSize);
        if (data != nullptr) {
            break;
        }
        nextSize = safeSize / 2ul;
    } while (nextSize > minViewSizeWindows);

    if (data != nullptr) {
        viewOffset = alignedOffset.QuadPart;
        viewSize = safeSize;
    }
}

void MemoryMappedFileStreamWindows::unmapFile() {
    if (data != nullptr) {
        UnmapViewOfFile(data);
        data = nullptr;
    }

    if (mapping != nullptr) {
        CloseHandle(mapping);
        mapping = nullptr;
    }

    viewOffset = 0ul;
    viewSize = 0ul;
}

void MemoryMappedFileStreamWindows::resizeFile(std::uint64_t size) {
    if (file == INVALID_HANDLE_VALUE) {
        return;
    }

    // Seek to the new size
    LARGE_INTEGER moveBy{};
    moveBy.QuadPart = static_cast<decltype(moveBy.QuadPart)>(size);
    if (SetFilePointerEx(file, moveBy, nullptr, FILE_BEGIN) == 0) {
        return;
    }

    // Resize the file to it's current position
    if (SetEndOfFile(file) == 0) {
        return;
    }

    fileSize = size;
}

}  // namespace trio

#endif  // TRIO_WINDOWS_FILE_MAPPING_AVAILABLE
// *INDENT-ON*
