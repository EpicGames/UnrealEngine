// Copyright Epic Games, Inc. All Rights Reserved.

#include "trio/streams/MemoryStreamImpl.h"

#include <pma/PolyAllocator.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstdint>
#include <cstring>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trio {

namespace {

class MemoryReader : public Readable {
public:
    explicit MemoryReader(const char* source_) :
        source{source_} {
    }

    std::size_t read(char* destination, std::size_t size) override {
        std::memcpy(destination, source, size);
        source += size;
        return size;
    }

    std::size_t read(Writable* destination, std::size_t size) override {
        destination->write(source, size);
        source += size;
        return size;
    }

private:
    const char* source;
};

class MemoryWriter : public Writable {
public:
    explicit MemoryWriter(char* destination_) :
        destination{destination_} {
    }

    std::size_t write(const char* source, std::size_t size) override {
        std::memcpy(destination, source, size);
        destination += size;
        return size;
    }

    std::size_t write(Readable* source, std::size_t size) override {
        source->read(destination, size);
        destination += size;
        return size;
    }

private:
    char* destination;
};

}  // namespace

MemoryStream::~MemoryStream() = default;

MemoryStream* MemoryStream::create(MemoryResource* memRes) {
    return create(0ul, memRes);
}

MemoryStream* MemoryStream::create(std::size_t initialSize, MemoryResource* memRes) {
    pma::PolyAllocator<MemoryStreamImpl> alloc{memRes};
    return alloc.newObject(initialSize, memRes);
}

void MemoryStream::destroy(MemoryStream* instance) {
    // NOLINTNEXTLINE(cppcoreguidelines-pro-type-static-cast-downcast)
    auto stream = static_cast<MemoryStreamImpl*>(instance);
    pma::PolyAllocator<MemoryStreamImpl> alloc{stream->getMemoryResource()};
    alloc.deleteObject(stream);
}

MemoryStreamImpl::MemoryStreamImpl(std::size_t initialSize, MemoryResource* memRes_) :
    data{initialSize, static_cast<char>(0), memRes_},
    position{},
    memRes{memRes_} {
}

void MemoryStreamImpl::open() {
    position = 0ul;
}

void MemoryStreamImpl::close() {
    position = 0ul;
}

std::uint64_t MemoryStreamImpl::tell() {
    return position;
}

void MemoryStreamImpl::seek(std::uint64_t position_) {
    if ((position_ == 0ul) || (position_ <= size())) {
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
        position = static_cast<std::size_t>(position_);
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
    } else {
        status->set(SeekError);
    }
}

std::size_t MemoryStreamImpl::read(char* destination, std::size_t size) {
    if (destination == nullptr) {
        status->set(ReadError);
        return 0ul;
    }

    MemoryWriter writer{destination};
    return read(&writer, size);
}

std::size_t MemoryStreamImpl::read(Writable* destination, std::size_t size) {
    if (destination == nullptr) {
        status->set(ReadError);
        return 0ul;
    }

    const std::size_t available = data.size() - position;
    const std::size_t bytesToRead = std::min(size, available);
    const std::size_t bytesCopied = (bytesToRead > 0ul ? destination->write(&data[position], bytesToRead) : 0ul);
    position += bytesCopied;
    return bytesCopied;
}

std::size_t MemoryStreamImpl::write(const char* source, std::size_t size) {
    if (source == nullptr) {
        status->set(WriteError);
        return 0ul;
    }

    MemoryReader reader{source};
    return write(&reader, size);
}

std::size_t MemoryStreamImpl::write(Readable* source, std::size_t size) {
    if (source == nullptr) {
        status->set(WriteError);
        return 0ul;
    }
    const std::size_t available = data.size() - position;
    if (available < size) {
        const std::size_t newSize = data.size() + (size - available);
        // Check for overflow / wrap-around
        if (newSize < data.size()) {
            status->set(WriteError);
            return 0ul;
        }
        data.resize(newSize);
    }
    const std::size_t bytesCopied = source->read(&data[position], size);
    position += bytesCopied;
    return bytesCopied;
}

std::uint64_t MemoryStreamImpl::size() {
    return data.size();
}

MemoryResource* MemoryStreamImpl::getMemoryResource() {
    return memRes;
}

}  // namespace trio
