// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/streams/MemoryStream.h"
#include "trio/streams/StreamStatus.h"
#include "trio/types/Aliases.h"

#include <pma/TypeDefs.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <cstdint>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trio {

class MemoryStreamImpl : public MemoryStream {
public:
    MemoryStreamImpl(std::size_t initialSize, MemoryResource* memRes_);

    void open() override;
    void close() override;
    std::uint64_t tell() override;
    void seek(std::uint64_t position_) override;
    std::uint64_t size() override;
    std::size_t read(char* destination, std::size_t size) override;
    std::size_t read(Writable* destination, std::size_t size) override;
    std::size_t write(const char* source, std::size_t size) override;
    std::size_t write(Readable* source, std::size_t size) override;

    MemoryResource* getMemoryResource();

private:
    StreamStatus status;
    Vector<char> data;
    std::size_t position;
    MemoryResource* memRes;
};

}  // namespace trio
