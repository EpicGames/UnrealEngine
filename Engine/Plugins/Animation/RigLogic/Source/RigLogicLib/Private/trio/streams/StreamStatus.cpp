// Copyright Epic Games, Inc. All Rights Reserved.

#include "trio/streams/StreamStatus.h"

#include "trio/Stream.h"

namespace trio {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
#endif
sc::StatusProvider StreamStatus::status{BoundedIOStream::OpenError,
                                        BoundedIOStream::ReadError,
                                        BoundedIOStream::WriteError,
                                        BoundedIOStream::AlreadyOpenError,
                                        BoundedIOStream::SeekError};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

}  // namespace trio
