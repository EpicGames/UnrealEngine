// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/types/Aliases.h"

#include <status/Provider.h>

namespace trio {

class StreamStatus {
public:
    sc::StatusProvider* operator->() {
        return &status;
    }

private:
    static sc::StatusProvider status;
};

}  // namespace trio
