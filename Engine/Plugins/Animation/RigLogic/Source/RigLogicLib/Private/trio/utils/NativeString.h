// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _WIN32
    #include "trio/utils/PlatformWindows.h"
#endif

#include <pma/TypeDefs.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <clocale>
#include <cstddef>
#include <cstring>
#include <cwchar>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trio {

template<typename TCharacter>
struct StringConverter;

template<>
struct StringConverter<char> {
    using CharacterType = char;
    using StringType = pma::String<CharacterType>;

    static StringType from(const char* source, pma::MemoryResource* memRes) {
        return StringType{source, memRes};
    }
};

template<>
struct StringConverter<wchar_t> {
    using CharacterType = wchar_t;
    using StringType = pma::String<CharacterType>;

    static StringType from(const char* source, pma::MemoryResource* memRes) {
#ifdef _WIN32
        const std::size_t length = std::strlen(source);
        const int neededSize = MultiByteToWideChar(CP_UTF8, 0, source, static_cast<int>(length), nullptr, 0);
        StringType result{static_cast<std::size_t>(neededSize), 0, memRes};
        MultiByteToWideChar(CP_UTF8, 0, source, static_cast<int>(length), &result[0], neededSize);
        return result;
#else
        const char* current = std::setlocale(LC_ALL, nullptr);
        std::setlocale(LC_ALL, "en_US.utf8");  // Any UTF-8 locale will work

        std::mbstate_t state{};
        const std::size_t neededSize = std::mbsrtowcs(nullptr, &source, 0, &state) + 1ul;
        StringType result{neededSize, 0, memRes};
        std::mbsrtowcs(&result[0], &source, result.size(), &state);

        if (current != nullptr) {
            std::setlocale(LC_ALL, current);
        }

        return result;
#endif
    }
};

#if defined(_WIN32) && defined(UNICODE)
using NativeCharacter = wchar_t;
#else
using NativeCharacter = char;
#endif

using NativeString = pma::String<NativeCharacter>;
using NativeStringConverter = StringConverter<NativeCharacter>;

}  // namespace trio
