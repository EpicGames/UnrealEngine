// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4005 4365 4987)
#endif

#ifdef TRIO_CUSTOM_WINDOWS_H
    #include TRIO_CUSTOM_WINDOWS_H
#else
    #define WIN32_LEAN_AND_MEAN
    #define NOGDICAPMASKS
    #define NOVIRTUALKEYCODES
    #define NOWINMESSAGES
    #define NOWINSTYLES
    #define NOSYSMETRICS
    #define NOMENUS
    #define NOICONS
    #define NOKEYSTATES
    #define NOSYSCOMMANDS
    #define NORASTEROPS
    #define NOSHOWWINDOW
    #define NOATOM
    #define NOCLIPBOARD
    #define NOCOLOR
    #define NOCTLMGR
    #define NODRAWTEXT
    #define NOGDI
    #define NOKERNEL
    #define NOUSER
    #define NOMB
    #define NOMEMMGR
    #define NOMETAFILE
    #define NOMINMAX
    #define NOMSG
    #define NOOPENFILE
    #define NOSCROLL
    #define NOSERVICE
    #define NOSOUND
    #define NOTEXTMETRIC
    #define NOWH
    #define NOWINOFFSETS
    #define NOCOMM
    #define NOKANJI
    #define NOHELP
    #define NOPROFILER
    #define NODEFERWINDOWPOS
    #define NOMCX
    #include <windows.h>
#endif  // TRIO_CUSTOM_WINDOWS_H
#ifdef _MSC_VER
    #pragma warning(pop)
#endif
