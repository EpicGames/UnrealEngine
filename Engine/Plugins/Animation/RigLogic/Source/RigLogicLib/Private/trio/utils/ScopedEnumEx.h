// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace trio {

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, TEnum>::type operator&(TEnum lhs, TEnum rhs) {
    using Underlying = typename std::underlying_type<TEnum>::type;
    return static_cast<TEnum>(static_cast<Underlying>(lhs) & static_cast<Underlying>(rhs));
}

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, TEnum>::type operator|(TEnum lhs, TEnum rhs) {
    using Underlying = typename std::underlying_type<TEnum>::type;
    return static_cast<TEnum>(static_cast<Underlying>(lhs) | static_cast<Underlying>(rhs));
}

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, TEnum>::type operator^(TEnum lhs, TEnum rhs) {
    using Underlying = typename std::underlying_type<TEnum>::type;
    return static_cast<TEnum>(static_cast<Underlying>(lhs) ^ static_cast<Underlying>(rhs));
}

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, TEnum>::type operator~(TEnum value) {
    using Underlying = typename std::underlying_type<TEnum>::type;
    return static_cast<TEnum>(~static_cast<Underlying>(value));
}

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, TEnum>::type operator&=(TEnum& lhs, TEnum rhs) {
    return lhs = (lhs & rhs);
}

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, TEnum>::type operator|=(TEnum& lhs, TEnum rhs) {
    return lhs = (lhs | rhs);
}

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, TEnum>::type operator^=(TEnum& lhs, TEnum rhs) {
    return lhs = (lhs ^ rhs);
}

template<typename TEnum>
typename std::enable_if<std::is_enum<TEnum>::value, bool>::type contains(TEnum lhs, TEnum rhs) {
    return (lhs & rhs) == rhs;
}

}  // namespace trio
