// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef RIGLOGIC_MODULE_DISCARD
	#define RIGLOGICLIB_API
#endif  // RIGLOGIC_MODULE_DISCARD

#include "CoreMinimal.h"

#include "riglogic/RigLogic.h"

class FArchive;

/** Adapter that allows using an FArchive instance as a rl4::BoundedIOStream */
class FArchiveMemoryStream: public rl4::BoundedIOStream
{
public:
	RIGLOGICLIB_API explicit FArchiveMemoryStream(FArchive* Archive);

	RIGLOGICLIB_API void seek(std::uint64_t Position) override;
	RIGLOGICLIB_API std::uint64_t tell() override;
	RIGLOGICLIB_API void open() override;
	RIGLOGICLIB_API void close() override;
	RIGLOGICLIB_API size_t read(char* ReadToBuffer, size_t Size) override;
	RIGLOGICLIB_API size_t read(Writable* Destination, size_t Size) override;
	RIGLOGICLIB_API size_t write(const char* WriteFromBuffer, size_t Size) override;
	RIGLOGICLIB_API size_t write(Readable* Source, size_t Size) override;
	RIGLOGICLIB_API std::uint64_t size() override;

private:
	FArchive* Archive;
	int64 Origin;

};
