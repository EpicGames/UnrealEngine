// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"

#include "riglogic/RigLogic.h"

class FMemoryResource : public rl4::MemoryResource
{
public:
	void* allocate(std::size_t size, std::size_t alignment) override
	{
		return FMemory::Malloc(size, alignment);
	}

	void deallocate(void* ptr, std::size_t size, std::size_t alignment) override
	{
		FMemory::Free(ptr);
	}

	static TSharedPtr<FMemoryResource> SharedInstance()
	{
		static TSharedPtr<FMemoryResource> MemoryResource = MakeShared<FMemoryResource>();
		return MemoryResource;
	}

	static MemoryResource* Instance()
	{
		return SharedInstance().Get();
	}

};
