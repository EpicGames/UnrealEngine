// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef RIGLOGIC_MODULE_DISCARD

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRigLogicLib, Log, All);

class FRigLogicLib : public IModuleInterface
{
public:
	void StartupModule() override;
	void ShutdownModule() override;
};

#endif  // RIGLOGIC_MODULE_DISCARD
