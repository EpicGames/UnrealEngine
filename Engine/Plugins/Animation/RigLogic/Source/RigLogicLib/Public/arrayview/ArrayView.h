// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstddef>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace av {

template<typename T>
struct ArrayViewTraits {
    using value_type = T;
    using reference = T&;
    using const_reference = const T&;
    using pointer = T*;
    using const_pointer = const T*;
    using size_type = std::size_t;
    using difference_type = std::ptrdiff_t;
};

template<typename T>
struct ArrayViewTraits<const T> {
    using value_type = const T;
    using reference = const T&;
    using const_reference = const T&;
    using pointer = const T*;
    using const_pointer = const T*;
    using size_type = std::size_t;
    using difference_type = std::ptrdiff_t;
};

template<typename T, typename U>
struct IsCompatible {
    static constexpr bool value = std::is_same<typename std::remove_cv<T>::type, typename std::remove_cv<U>::type>::value &&
                                  (std::is_const<typename std::remove_reference<T>::type>::value ||
                                   !std::is_const<typename std::remove_reference<U>::type>::value);
};

/**
    @brief A view over a continuous sequence of objects.
    @tparam T element type
    Provides a view over a continuous sequence of objects owned by some other object.
    Contains a count of elements and a pointer to a location where they are stored.
    ArrayView does not own the memory it points to - it does not perform any allocation
    and deallocation. It can be constructed given a pointer and element count, or a container
    type argument. The class provides helper methods for creating subviews over the objects,
    and methods for by-value comparison with containers. ConstArrayView represents an immutable view.
*/
template<typename T>
class ArrayView {
public:
    using value_type = typename ArrayViewTraits<T>::value_type;
    using reference = typename ArrayViewTraits<T>::reference;
    using const_reference = typename ArrayViewTraits<T>::const_reference;
    using const_pointer = typename ArrayViewTraits<T>::const_pointer;
    using pointer = typename ArrayViewTraits<T>::pointer;
    using size_type = typename ArrayViewTraits<T>::size_type;
    using difference_type = typename ArrayViewTraits<T>::difference_type;

    ArrayView() = default;
    ~ArrayView() noexcept = default;

    ArrayView(const ArrayView&) = default;
    ArrayView& operator=(const ArrayView&) = default;

    ArrayView(ArrayView&&) = default;
    ArrayView& operator=(ArrayView&&) = default;

    ArrayView(pointer src, size_type size) :
        ptr{src},
        sz{size} {
    }

    template<typename U, typename std::enable_if<IsCompatible<T, U>::value, int>::type = 0>
    ArrayView(ArrayView<U>& src) :
        ArrayView{src.data(), src.size()} {
    }

    template<typename U, typename std::enable_if<IsCompatible<T, U>::value, int>::type = 0>
    ArrayView(const ArrayView<U>& src) :
        ArrayView{src.data(), src.size()} {
    }

    template<typename U, typename std::enable_if<IsCompatible<T, U>::value, int>::type = 0>
    ArrayView(ArrayView<U>&& src) :
        ArrayView{src.data(), src.size()} {
    }

    template<typename U,
             typename std::enable_if<!std::is_rvalue_reference<U&&>::value &&
                                         IsCompatible<T, typename std::remove_reference<U>::type::value_type>::value,
                                     int>::type = 0>
    ArrayView(U&& src) :
        ArrayView{src.data(), static_cast<size_type>(src.size())} {
    }

    size_type size() const {
        return sz;
    }

    pointer data() {
        return ptr;
    }

    const_pointer data() const {
        return ptr;
    }

    pointer begin() {
        return ptr;
    }

    pointer end() {
        return ptr + sz;
    }

    const_pointer cbegin() const {
        return ptr;
    }

    const_pointer cend() const {
        return ptr + sz;
    }

    const_pointer begin() const {
        return cbegin();
    }

    const_pointer end() const {
        return cend();
    }

    reference operator[](std::size_t index) {
        assert(index < sz);
        return ptr[index];
    }

    const_reference operator[](std::size_t index) const {
        assert(index < sz);
        return ptr[index];
    }

    reference at(std::size_t index) {
        return this->operator[](index);
    }

    const_reference at(std::size_t index) const {
        return this->operator[](index);
    }

    ArrayView subview(std::size_t offset, std::size_t count) const {
        assert(offset <= sz);
        assert((offset + count) <= sz);
        return {ptr + offset, count};
    }

    ArrayView first(std::size_t count) const {
        assert(count <= sz);
        return {ptr, count};
    }

    ArrayView last(std::size_t count) const {
        assert(count <= sz);
        return {ptr + (sz - count), count};
    }

private:
    pointer ptr{nullptr};
    size_type sz{};
};

template<typename T, typename U>
bool operator==(const ArrayView<T>& lhs, const ArrayView<U>& rhs) {
    if (lhs.size() != rhs.size()) {
        return false;
    }
    if (lhs.data() == rhs.data()) {
        return true;
    }
#if __cplusplus >= 201402L || (defined(_MSC_VER) && _MSC_VER >= 1900)
    // Under Visual Studio 2015, the overload of std::equal accepting 4 parameters must be used,
    // because the 3-parameter version causes insuppressible warnings
    return std::equal(lhs.begin(), lhs.end(), rhs.begin(), rhs.end());
#else
    return std::equal(lhs.begin(), lhs.end(), rhs.begin());
#endif
}

template<typename T, typename U>
bool operator!=(const ArrayView<T>& lhs, const ArrayView<U>& rhs) {
    return !(lhs == rhs);
}

template<typename T, typename TContainer>
typename std::enable_if<!std::is_base_of<ArrayView<T>, TContainer>::value, bool>::type operator==(const ArrayView<T>& lhs,
                                                                                                  const TContainer& rhs) {
    if (lhs.size() != rhs.size()) {
        return false;
    }
#if __cplusplus >= 201402L || (defined(_MSC_VER) && _MSC_VER >= 1900)
    // Under Visual Studio 2015, the overload of std::equal accepting 4 parameters must be used,
    // because the 3-parameter version causes insuppressible warnings
    return std::equal(lhs.begin(), lhs.end(), rhs.begin(), rhs.end());
#else
    return std::equal(lhs.begin(), lhs.end(), rhs.begin());
#endif
}

template<typename T, typename TContainer>
typename std::enable_if<!std::is_base_of<ArrayView<T>, TContainer>::value, bool>::type operator!=(const ArrayView<T>& lhs,
                                                                                                  const TContainer& rhs) {
    return !(lhs == rhs);
}

template<typename T, typename TContainer>
typename std::enable_if<!std::is_base_of<ArrayView<T>, TContainer>::value, bool>::type operator==(const TContainer& lhs,
                                                                                                  const ArrayView<T>& rhs) {
    return (rhs == lhs);
}

template<typename T, typename TContainer>
typename std::enable_if<!std::is_base_of<ArrayView<T>, TContainer>::value, bool>::type operator!=(const TContainer& lhs,
                                                                                                  const ArrayView<T>& rhs) {
    return !(lhs == rhs);
}

template<typename T>
using ConstArrayView = ArrayView<const T>;

}  // namespace av
