// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "arrayview/ArrayView.h"

namespace av {

class StringView : public ConstArrayView<char> {
public:
    using Base = ConstArrayView<char>;

public:
    using Base::ArrayView;

    const char* c_str() const {
        return dataOrEmpty();
    }

    operator const char*() const {
        return dataOrEmpty();
    }

    const char* operator*() const {
        return dataOrEmpty();
    }

private:
    const char* dataOrEmpty() const {
        return (data() == nullptr ? "" : data());
    }
};

}  // namespace av
