// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define AV_MAJOR_VERSION 1
#define AV_MINOR_VERSION 1
#define AV_PATCH_VERSION 8
#define AV_VERSION_STRING "1.1.8"
