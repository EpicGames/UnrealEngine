// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"
#include "dna/Defs.h"
#include "dna/StreamReader.h"
#include "dna/types/Aliases.h"

namespace dna {

class DNAAPI BinaryStreamReader : public StreamReader {
public:
    /**
        @brief Factory method for creation of BinaryStreamReader
        @param stream
            Source stream from which data is going to be read.
        @param config
            Specify the filtering and adaptation parameters. @see Configuration
        @param memRes
            Memory resource to be used for allocations.
        @note
            If a memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static BinaryStreamReader* create(BoundedIOStream* stream,
                                      const Configuration& config = {},
                                      MemoryResource* memRes = nullptr);
    /**
        @brief Factory method for creation of BinaryStreamReader
        @param stream
            Source stream from which data is going to be read.
        @param layer
            Specify the layer up to which the data needs to be loaded.
        @note
            The Definition data layer depends on and thus implicitly loads the Descriptor layer.
            The Behavior data layer depends on and thus implicitly loads the Definition layer.
            The Geometry data layer depends on and thus also implicitly loads the Definition layer.
            The MachineLearnedBehavior data layer depends on and thus also implicitly loads the Definition layer.
        @param policy
            Specify whether unknown layers are to be preserved or just ignored.
        @param maxLOD
            The maximum level of details to be loaded.
        @note
            A value of zero indicates to load all LODs.
        @warning
            The maxLOD value must be less than the value returned by getLODCount.
        @see getLODCount
        @param memRes
            Memory resource to be used for allocations.
        @note
            If a memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static BinaryStreamReader* create(BoundedIOStream* stream,
                                      DataLayer layer,
                                      UnknownLayerPolicy policy = UnknownLayerPolicy::Preserve,
                                      std::uint16_t maxLOD = 0u,
                                      MemoryResource* memRes = nullptr);
    /**
        @brief Factory method for creation of BinaryStreamReader
        @param stream
            Source stream from which data is going to be read.
        @param layer
            Specify the layer up to which the data needs to be loaded.
        @note
            The Definition data layer depends on and thus implicitly loads the Descriptor layer.
            The Behavior data layer depends on and thus implicitly loads the Definition layer.
            The Geometry data layer depends on and thus also implicitly loads the Definition layer.
            The MachineLearnedBehavior data layer depends on and thus also implicitly loads the Definition layer.
        @param policy
            Specify whether unknown layers are to be preserved or just ignored.
        @param maxLOD
            The maximum level of details to be loaded.
        @param minLOD
            The minimum level of details to be loaded.
        @note
            A range of [0, LOD count - 1] for maxLOD / minLOD respectively indicates to load all LODs.
        @warning
            Both maxLOD and minLOD values must be less than the value returned by getLODCount.
        @see getLODCount
        @param memRes
            Memory resource to be used for allocations.
        @note
            If a memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static BinaryStreamReader* create(BoundedIOStream* stream,
                                      DataLayer layer,
                                      UnknownLayerPolicy policy,
                                      std::uint16_t maxLOD,
                                      std::uint16_t minLOD,
                                      MemoryResource* memRes = nullptr);
    /**
        @brief Factory method for creation of BinaryStreamReader
        @param stream
            Source stream from which data is going to be read.
        @param layer
            Specify the layer up to which the data needs to be loaded.
        @note
            The Definition data layer depends on and thus implicitly loads the Descriptor layer.
            The Behavior data layer depends on and thus implicitly loads the Definition layer.
            The Geometry data layer depends on and thus also implicitly loads the Definition layer.
            The MachineLearnedBehavior data layer depends on and thus also implicitly loads the Definition layer.
        @param policy
            Specify whether unknown layers are to be preserved or just ignored.
        @param lods
            An array specifying which exact lods to load.
        @warning
            All values in the array must be less than the value returned by getLODCount.
        @see getLODCount
        @param lodCount
            The number of elements in the lods array.
        @warning
            There cannot be more elements in the array than the value returned by getLODCount.
        @see getLODCount
        @param memRes
            Memory resource to be used for allocations.
        @note
            If a memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static BinaryStreamReader* create(BoundedIOStream* stream,
                                      DataLayer layer,
                                      UnknownLayerPolicy policy,
                                      std::uint16_t* lods,
                                      std::uint16_t lodCount,
                                      MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a BinaryStreamReader instance.
        @param instance
            Instance of BinaryStreamReader to be freed.
        @see create
    */
    static void destroy(BinaryStreamReader* instance);

    ~BinaryStreamReader() override;
};

}  // namespace dna

namespace pma {

template<>
struct DefaultInstanceCreator<dna::BinaryStreamReader> {
    using type = pma::FactoryCreate<dna::BinaryStreamReader>;
};

template<>
struct DefaultInstanceDestroyer<dna::BinaryStreamReader> {
    using type = pma::FactoryDestroy<dna::BinaryStreamReader>;
};

}  // namespace pma
