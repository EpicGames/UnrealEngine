// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/StreamWriter.h"
#include "dna/types/Aliases.h"

namespace dna {

class BinaryStreamReader;
class JSONStreamReader;

class DNAAPI BinaryStreamWriter : public StreamWriter {
public:
    /**
        @brief Factory method for creation of BinaryStreamWriter
        @param stream
            Stream into which the data is going to be written.
        @param memRes
            Memory resource to be used for allocations.
        @note
            If a memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static BinaryStreamWriter* create(BoundedIOStream* stream, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a BinaryStreamWriter instance.
        @param instance
            Instance of BinaryStreamWriter to be freed.
        @see create
    */
    static void destroy(BinaryStreamWriter* instance);

    ~BinaryStreamWriter() override;

    using StreamWriter::setFrom;
    virtual void setFrom(const BinaryStreamReader* source,
                         DataLayer layer = DataLayer::All,
                         UnknownLayerPolicy policy = UnknownLayerPolicy::Preserve,
                         MemoryResource* memRes = nullptr) = 0;
#ifdef DNA_BUILD_WITH_JSON_SUPPORT
    virtual void setFrom(const JSONStreamReader* source,
                         DataLayer layer = DataLayer::All,
                         UnknownLayerPolicy policy = UnknownLayerPolicy::Preserve,
                         MemoryResource* memRes = nullptr) = 0;
#endif  // DNA_BUILD_WITH_JSON_SUPPORT
};

}  // namespace dna

namespace pma {

template<>
struct DefaultInstanceCreator<dna::BinaryStreamWriter> {
    using type = pma::FactoryCreate<dna::BinaryStreamWriter>;
};

template<>
struct DefaultInstanceDestroyer<dna::BinaryStreamWriter> {
    using type = pma::FactoryDestroy<dna::BinaryStreamWriter>;
};

}  // namespace pma
