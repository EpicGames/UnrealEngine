// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

enum class DataLayer : std::uint32_t {
    Descriptor = 1,
    Definition = 2 | Descriptor,                   // Implicitly loads Descriptor
    Behavior = 4 | Definition,                     // Implicitly loads Definition
    Geometry = 8 | Definition,                     // Implicitly loads Definition
    GeometryWithoutBlendShapes = 16 | Definition,  // Implicitly loads Definition
    MachineLearnedBehavior = 32 | Definition,      // Implicitly loads Definition
    RBFBehavior = 64 | Behavior,                   // Implicitly loads Behavior
    JointBehaviorMetadata = 128 | Definition,      // Implicitly loads Definition
    TwistSwingBehavior = 256 | Definition,         // Implicitly loads Definition
    All = RBFBehavior | Geometry | MachineLearnedBehavior | JointBehaviorMetadata | TwistSwingBehavior
};

inline DataLayer operator|(DataLayer lhs, DataLayer rhs) {
    return static_cast<DataLayer>(static_cast<std::uint32_t>(lhs) | static_cast<std::uint32_t>(rhs));
}

enum class UnknownLayerPolicy {
    Preserve,
    Ignore
};

enum class CoordinateSystemTransformPolicy {
    Preserve,
    Transform
};

enum class TranslationUnit {
    cm,
    m
};

enum class RotationUnit {
    degrees,
    radians
};

// Face vertex winding order, viewed along the outward surface normal.
// CCW: cross product of consecutive face vertices agrees in direction with stored normals.
// CW:  cross product of consecutive face vertices opposes stored normals (left-handed / DirectX style).
enum class FaceWindingOrder : std::uint8_t {
    ccw = 0,
    cw = 1
};

using Direction = tdm::axis_dir;
using RotationDirection = tdm::rot_dir;
using RotationSequence = tdm::rot_seq;
using RotationSign = tdm::rot_sign;
using CoordinateSystem = tdm::coord_sys;

struct Configuration {
    // Specify the layer up to which the data needs to be loaded.
    // @see DataLayer
    DataLayer layer = DataLayer::All;
    // Specify whether unknown layers are to be preserved or just ignored.
    UnknownLayerPolicy unknownLayerPolicy = UnknownLayerPolicy::Preserve;
    // An array specifying which exact LODs to load.
    // @note usage of this parameter causes the values set for maxLOD / minLOD to be ignored.
    // @warning All values in the array must be less than the value returned by getLODCount.
    ConstArrayView<std::uint16_t> lods = {};
    // The maximum level of details to be loaded.
    // @note A range of [0, LOD count - 1] for maxLOD / minLOD respectively indicates to load all LODs.
    // @warning The maxLOD value must be less than the value returned by getLODCount.
    std::uint16_t maxLOD = 0;
    // The minimum level of details to be loaded.
    // @note A range of [0, LOD count - 1] for maxLOD / minLOD respectively indicates to load all LODs.
    // @warning The minLOD value must be less than the value returned by getLODCount.
    std::uint16_t minLOD = static_cast<std::uint16_t>(-1);
    // Preserve transform policy will perform no conversion whatsoever
    // Transform policy will perform conversion to the specified destination coordinate system unless the data is already in that
    // system.
    CoordinateSystemTransformPolicy coordinateSystemTransformPolicy = CoordinateSystemTransformPolicy::Preserve;
    // The axis directions for all coordinate axes.
    CoordinateSystem coordinateSystem = {};
    // The rotation sequence applied globally.
    RotationSequence rotationSequence = RotationSequence::xyz;
    // Rotation direction per each coordinate axis.
    RotationSign rotationSign = {RotationDirection::positive, RotationDirection::positive, RotationDirection::positive};
    // The face winding order of the geometry data. Authored DNAs are CCW (cross product
    // of consecutive face vertices points along the outward surface normal). Converters
    // normalize to this value when the CoordinateSystemTransformPolicy is set to Transform.
    FaceWindingOrder faceWindingOrder = FaceWindingOrder::ccw;
};

}  // namespace dna
