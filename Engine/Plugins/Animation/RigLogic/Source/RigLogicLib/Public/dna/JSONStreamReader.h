// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #include "dna/Defs.h"
    #include "dna/StreamReader.h"
    #include "dna/types/Aliases.h"

namespace dna {

class DNAAPI JSONStreamReader : public StreamReader {
public:
    /**
        @brief Factory method for creation of JSONStreamReader
        @param stream
            Source stream from which data is going to be read.
        @param memRes
            Memory resource to be used for allocations.
        @note
            If a memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static JSONStreamReader* create(BoundedIOStream* stream, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a JSONStreamReader instance.
        @param instance
            Instance of JSONStreamReader to be freed.
        @see create
    */
    static void destroy(JSONStreamReader* instance);

    ~JSONStreamReader() override;
};

}  // namespace dna

namespace pma {

template<>
struct DefaultInstanceCreator<dna::JSONStreamReader> {
    using type = pma::FactoryCreate<dna::JSONStreamReader>;
};

template<>
struct DefaultInstanceDestroyer<dna::JSONStreamReader> {
    using type = pma::FactoryDestroy<dna::JSONStreamReader>;
};

}  // namespace pma

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
// *INDENT-ON*
