// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #include "dna/Defs.h"
    #include "dna/StreamWriter.h"
    #include "dna/types/Aliases.h"

namespace dna {

class BinaryStreamReader;
class JSONStreamReader;

class DNAAPI JSONStreamWriter : public StreamWriter {
public:
    /**
        @brief Factory method for creation of JSONStreamWriter
        @param stream
            Stream into which the data is going to be written.
        @param indentWidth
            Number of spaces to use for indentation.
        @param memRes
            Memory resource to be used for allocations.
        @note
            If a memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static JSONStreamWriter* create(BoundedIOStream* stream, std::uint32_t indentWidth = 4u, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a JSONStreamWriter instance.
        @param instance
            Instance of JSONStreamWriter to be freed.
        @see create
    */
    static void destroy(JSONStreamWriter* instance);

    ~JSONStreamWriter() override;

    using StreamWriter::setFrom;
    virtual void setFrom(const BinaryStreamReader* source,
                         DataLayer layer = DataLayer::All,
                         UnknownLayerPolicy policy = UnknownLayerPolicy::Preserve,
                         MemoryResource* memRes = nullptr) = 0;
    virtual void setFrom(const JSONStreamReader* source,
                         DataLayer layer = DataLayer::All,
                         UnknownLayerPolicy policy = UnknownLayerPolicy::Preserve,
                         MemoryResource* memRes = nullptr) = 0;
};

}  // namespace dna

namespace pma {

template<>
struct DefaultInstanceCreator<dna::JSONStreamWriter> {
    using type = pma::FactoryCreate<dna::JSONStreamWriter>;
};

template<>
struct DefaultInstanceDestroyer<dna::JSONStreamWriter> {
    using type = pma::FactoryDestroy<dna::JSONStreamWriter>;
};

}  // namespace pma

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
// *INDENT-ON*
