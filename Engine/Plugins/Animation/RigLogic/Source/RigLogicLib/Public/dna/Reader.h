// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"
#include "dna/Defs.h"
#include "dna/layers/GeometryReader.h"
#include "dna/layers/JointBehaviorMetadataReader.h"
#include "dna/layers/MachineLearnedBehaviorExtReader.h"
#include "dna/layers/RBFBehaviorReader.h"
#include "dna/layers/TwistSwingBehaviorReader.h"

namespace dna {

/**
    @brief The abstract Reader which its implementations are expected to inherit.
    @note
        This class combines the various different reader interfaces into a single interface.
        The artificial separation into multiple interfaces mirrors the DNA file structure that
        is separated into matching layers under the same names. As these layers can be
        selectively loaded, it might be convenient to slice-off interfaces which layers were
        not loaded.
*/
class DNAAPI Reader : public RBFBehaviorReader,
                      public GeometryReader,
                      public MachineLearnedBehaviorExtReader,
                      public JointBehaviorMetadataReader,
                      public TwistSwingBehaviorReader {
public:
    ~Reader() override;
    /**
        @brief Unload all data of the specified layer and all layers dependent on it.
        @param layer
            Layer whose data should be unloaded.
    */
    virtual void unload(DataLayer layer) = 0;
};

}  // namespace dna
