// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"
#include "dna/Defs.h"
#include "dna/Reader.h"
#include "dna/types/Aliases.h"

namespace dna {

class DNAAPI StreamReader : public Reader {
public:
    static const sc::StatusCode SignatureMismatchError;
    static const sc::StatusCode VersionMismatchError;
    static const sc::StatusCode InvalidDataError;

public:
    ~StreamReader() override;
    /**
       @brief read data from stream into internal structures.
    */
    virtual void read() = 0;
};

}  // namespace dna
