// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/Writer.h"
#include "dna/types/Aliases.h"

namespace dna {

class DNAAPI StreamWriter : public Writer {
public:
    ~StreamWriter() override;
    /**
        @brief Write data to stream from internal structures.
     */
    virtual void write() = 0;
};

}  // namespace dna
