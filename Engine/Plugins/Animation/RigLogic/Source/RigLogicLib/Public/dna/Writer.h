// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"
#include "dna/Defs.h"
#include "dna/layers/GeometryWriter.h"
#include "dna/layers/JointBehaviorMetadataWriter.h"
#include "dna/layers/MachineLearnedBehaviorExtWriter.h"
#include "dna/layers/RBFBehaviorWriter.h"
#include "dna/layers/TwistSwingBehaviorWriter.h"

namespace dna {

class Reader;

/**
    @brief The abstract Writer which its implementations are expected to inherit.
    @note
        This class combines the various different writer interfaces into a single interface.
        The artificial separation into multiple interfaces in this case just mirrors the
        structure of the Reader hierarchy, as it's not possible to selectively write only
        specific layers.
*/
class DNAAPI Writer : public RBFBehaviorWriter,
                      public GeometryWriter,
                      public MachineLearnedBehaviorExtWriter,
                      public JointBehaviorMetadataWriter,
                      public TwistSwingBehaviorWriter {
public:
    ~Writer() override;
    /**
        @brief Initialize the Writer from the given Reader.
        @note
            This function copies all the data from the given Reader into the Writer instance,
            by calling each getter function of the Reader, and passing the return values to
            the matching setter functions in the Writer.
            It is implemented in the abstract class itself to provide the functionality for
            all DNA Writers.
        @param source
            The source DNA Reader from which the data needs to be copied.
        @param layer
            Limit which layers should be taken over from the given source reader.
        @param policy
            Specify whether unknown layers are to be preserved or just ignored.
        @param memRes
            Optional memory resource to use for temporary allocations during copying.
    */
    virtual void setFrom(const Reader* source,
                         DataLayer layer = DataLayer::All,
                         UnknownLayerPolicy policy = UnknownLayerPolicy::Preserve,
                         MemoryResource* memRes = nullptr);
};

}  // namespace dna
