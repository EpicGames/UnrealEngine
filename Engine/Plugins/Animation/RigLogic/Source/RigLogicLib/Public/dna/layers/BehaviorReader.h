// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionReader.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors for DNA attributes that define the rig's evaluation.
    @warning
        Implementors should inherit from Reader itself and not this class.
    @see Reader
*/
class DNAAPI BehaviorReader : public virtual DefinitionReader {
protected:
    virtual ~BehaviorReader();

public:
    /**
        @brief Input indices used for mapping gui to raw controls.
        @return View over the array of input indices.
    */
    virtual ConstArrayView<std::uint16_t> getGUIToRawInputIndices() const = 0;
    /**
        @brief Output indices used for mapping gui to raw controls.
        @return View over the array of output indices.
    */
    virtual ConstArrayView<std::uint16_t> getGUIToRawOutputIndices() const = 0;
    /**
        @brief Filter values(lower-bounds) used to decide whether a particular
            entry should be evaluated or not during gui to raw control mapping.
        @return View over the array of filter values.
    */
    virtual ConstArrayView<float> getGUIToRawFromValues() const = 0;
    /**
        @brief Filter values(upper-bounds) used to decide whether a particular
            entry should be evaluated or not during gui to raw control mapping.
        @return View over the array of filter values.
    */
    virtual ConstArrayView<float> getGUIToRawToValues() const = 0;
    /**
        @brief Computational values(slope/gradient) used for calculating the
            output value during gui to raw control mapping.
        @return View over the array of computational values.
    */
    virtual ConstArrayView<float> getGUIToRawSlopeValues() const = 0;
    /**
        @brief Computational values(vertical intercept) used for calculating the
            output value during gui to raw control mapping.
        @return View over the array of computational values.
    */
    virtual ConstArrayView<float> getGUIToRawCutValues() const = 0;
    /**
        @brief The number of distinct PSD expressions.
    */
    virtual std::uint16_t getPSDCount() const = 0;
    /**
        @brief PSD(input) indices.
        @return View over the array of PSD indices.
    */
    virtual ConstArrayView<std::uint16_t> getPSDRowIndices() const = 0;
    /**
        @brief Control(input) indices.
        @return View over the array of control indices.
    */
    virtual ConstArrayView<std::uint16_t> getPSDColumnIndices() const = 0;
    /**
        @brief Weights associated with each PSD row and column pair.
        @return View over the array of weights.
    */
    virtual ConstArrayView<float> getPSDValues() const = 0;
    /**
        @brief Number of rows in the entire, uncompressed joint matrix.
    */
    virtual std::uint16_t getJointRowCount() const = 0;
    /**
        @brief Number of columns in the entire, uncompressed joint matrix.
    */
    virtual std::uint16_t getJointColumnCount() const = 0;
    /**
        @brief Joint attribute indices (output indices) for the requested LOD.
        @return View over the array of joint indices.
    */
    virtual ConstArrayView<std::uint16_t> getJointVariableAttributeIndices(std::uint16_t lod) const = 0;
    /**
        @brief Number of joint groups present in the entire joint matrix.
    */
    virtual std::uint16_t getJointGroupCount() const = 0;
    /**
        @brief Number of rows per each level of detail for the requested joint group.
        @note
            Each element's position represents the level itself, while the value denotes
            the number of rows within the joint group belonging to that level. e.g.:
            [12, 9, 3]
             |   |  + LOD-2 contains first 3 rows
             |   + LOD-1 contains first 9 rows
             + LOD-0 contains first 12 rows
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @warning
            jointGroupIndex must be less than the value returned by getJointGroupCount.
        @return View over the array of LOD bounds.
    */
    virtual ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t jointGroupIndex) const = 0;
    /**
        @brief Column indices that the requested joint group contains.
        @note
            The column indices point into the entire, uncompressed joint matrix.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @warning
            jointGroupIndex must be less than the value returned by getJointGroupCount.
        @return View over the array of column indices.
    */
    virtual ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t jointGroupIndex) const = 0;
    /**
        @brief Row indices that the requested joint group contains.
        @note
            The row indices point into the entire, uncompressed joint matrix.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @warning
            jointGroupIndex must be less than the value returned by getJointGroupCount.
        @return View over the array of row indices.
    */
    virtual ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const = 0;
    /**
        @brief Values that the requested joint group contains.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @warning
            jointGroupIndex must be less than the value returned by getJointGroupCount.
        @return View over the array of values.
    */
    virtual ConstArrayView<float> getJointGroupValues(std::uint16_t jointGroupIndex) const = 0;
    /**
        @brief Joint indices that the requested joint group contains.
        @note
            These joint indices can be used to get the joint names through DefinitionReader::getJointName.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @warning
            jointGroupIndex must be less than the value returned by getJointGroupCount.
        @return View over the array of joint indices.
        @see DefinitionReader
    */
    virtual ConstArrayView<std::uint16_t> getJointGroupJointIndices(std::uint16_t jointGroupIndex) const = 0;
    /**
        @brief Input index count per each level of detail for blend shape channels.
        @note
            Each element's position represents the level itself  (e.g. [0,1,2,3,4,5] Value 0 is LOD with highest of details,
            value 5 is LOD with lowest details), while the value denotes the number of input indices belonging to that level.
        @warning
            These LOD values are not interchangeable with the LOD indices from
       DefinitionReader::getBlendShapeChannelIndicesForLOD.
        @return View over the array of LOD bounds.
    */
    virtual ConstArrayView<std::uint16_t> getBlendShapeChannelLODs() const = 0;
    /**
        @brief Input indices used to index into the input vector.
        @return View over the array of input indices.
    */
    virtual ConstArrayView<std::uint16_t> getBlendShapeChannelInputIndices() const = 0;
    /**
        @brief Output indices specify the positions of blend shape channel output values.
        @return View over the array of output indices.
    */
    virtual ConstArrayView<std::uint16_t> getBlendShapeChannelOutputIndices() const = 0;
    /**
        @brief Row count per each level of detail for animated maps.
        @note
            Each element's position represents the level itself  (e.g. [0,1,2,3,4,5] Value 0 is LOD with highest of details,
            value 5 is LOD with lowest details), while the value denotes the number of rows (within the conditional table),
            belonging to that level.
        @return View over the array of LOD bounds.
    */
    virtual ConstArrayView<std::uint16_t> getAnimatedMapLODs() const = 0;
    /**
        @brief Input indices used to index into the array of input values.
        @return View over the array of input indices.
    */
    virtual ConstArrayView<std::uint16_t> getAnimatedMapInputIndices() const = 0;
    /**
        @brief Output indices that specify the computed output value's position.
        @return View over the array of output indices.
    */
    virtual ConstArrayView<std::uint16_t> getAnimatedMapOutputIndices() const = 0;
    /**
        @brief Filter values(lower-bounds) used to decide whether a particular
            entry should be evaluated or not.
        @return View over the array of filter values.
    */
    virtual ConstArrayView<float> getAnimatedMapFromValues() const = 0;
    /**
        @brief Filter values(upper-bounds) used to decide whether a particular
            entry should be evaluated or not.
        @return View over the array of filter values.
    */
    virtual ConstArrayView<float> getAnimatedMapToValues() const = 0;
    /**
        @brief Computational values(slope/gradient) used for calculating the output value.
        @return View over the array of computational values.
    */
    virtual ConstArrayView<float> getAnimatedMapSlopeValues() const = 0;
    /**
        @brief Computational values(vertical intercept) used for calculating the output value.
        @return View over the array of computational values.
    */
    virtual ConstArrayView<float> getAnimatedMapCutValues() const = 0;
};

}  // namespace dna
