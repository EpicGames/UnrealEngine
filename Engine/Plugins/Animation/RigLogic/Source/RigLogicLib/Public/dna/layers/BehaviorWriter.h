// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionWriter.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors for DNA attributes that define the rig's evaluation.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI BehaviorWriter : public virtual DefinitionWriter {
protected:
    virtual ~BehaviorWriter();

public:
    /**
        @brief Input indices used for mapping gui to raw controls.
        @param inputIndices
            The source address from which the input indices are to be copied.
        @param count
            The number of input indices to copy.
    */
    virtual void setGUIToRawInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) = 0;
    /**
        @brief Output indices used for mapping gui to raw controls.
        @param outputIndices
            The source address from which the output indices are to be copied.
        @param count
            The number of output indices to copy.
    */
    virtual void setGUIToRawOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) = 0;
    /**
        @brief Filter values(lower-bounds) used to decide whether a particular
            entry should be evaluated or not during gui to raw control mapping.
        @param fromValues
            The source address from which the filter values are to be copied.
        @param count
            The number of filter values to copy.
    */
    virtual void setGUIToRawFromValues(const float* fromValues, std::uint16_t count) = 0;
    /**
        @brief Filter values(upper-bounds) used to decide whether a particular
            entry should be evaluated or not during gui to raw control mapping.
        @param toValues
            The source address from which the filter values are to be copied.
        @param count
            The number of filter values to copy.
    */
    virtual void setGUIToRawToValues(const float* toValues, std::uint16_t count) = 0;
    /**
        @brief Computational values(slope/gradient) used for calculating the
            output value during gui to raw control mapping.
        @param slopeValues
            The source address from which the computational values are to be copied.
        @param count
            The number of computational values to copy.
    */
    virtual void setGUIToRawSlopeValues(const float* slopeValues, std::uint16_t count) = 0;
    /**
        @brief Computational values(vertical intercept) used for calculating the
            output value during gui to raw control mapping.
        @param cutValues
            The source address from which the computational values are to be copied.
        @param count
            The number of computational values to copy.
    */
    virtual void setGUIToRawCutValues(const float* cutValues, std::uint16_t count) = 0;
    /**
        @brief The number of distinct PSD expressions.
    */
    virtual void setPSDCount(std::uint16_t count) = 0;
    /**
        @brief PSD(input) indices which will become the rows of the PSD matrix.
        @param rowIndices
            The source address from which the PSD indices are to be copied.
        @param count
            The number of PSD indices to copy.
    */
    virtual void setPSDRowIndices(const std::uint16_t* rowIndices, std::uint16_t count) = 0;
    /**
        @brief Control(input) indices which will become the columns of the PSD matrix.
        @param columnIndices
            The source address from which the control indices are to be copied.
        @param count
            The number of control indices to copy.
    */
    virtual void setPSDColumnIndices(const std::uint16_t* columnIndices, std::uint16_t count) = 0;
    /**
        @brief Weights associated with each PSD row and column pair.
        @param weights
            The source address from which the weight values are to be copied.
        @param count
            The number of weight values to copy.
    */
    virtual void setPSDValues(const float* weights, std::uint16_t count) = 0;
    /**
        @brief Number of rows in the entire, uncompressed joint matrix.
    */
    virtual void setJointRowCount(std::uint16_t rowCount) = 0;
    /**
        @brief Number of columns in the entire, uncompressed joint matrix.
    */
    virtual void setJointColumnCount(std::uint16_t columnCount) = 0;
    /**
        @brief Delete all joint groups.
    */
    virtual void clearJointGroups() = 0;
    /**
        @brief Delete the specified joint group.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @warning
            jointGroupIndex must be less than the value returned by getJointGroupCount.
    */
    virtual void deleteJointGroup(std::uint16_t jointGroupIndex) = 0;
    /**
        @brief Number of rows per each level of detail for the specified joint group.
        @note
            Each element's position represents the level itself, while the value denotes
            the number of rows within the joint group belonging to that level. e.g.:
            [12, 9, 3]
             |   |  + LOD-2 contains first 3 rows
             |   + LOD-1 contains first 9 rows
             + LOD-0 contains first 12 rows
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @note
            The joint group storage will be implicitly resized (if needed) to provide
            storage for the number of joint groups that is inferred from the specified index.
        @param lods
            The source address from which the lod bounds are to be copied.
        @param count
            The number of lod bounds to copy.
    */
    virtual void setJointGroupLODs(std::uint16_t jointGroupIndex, const std::uint16_t* lods, std::uint16_t count) = 0;
    /**
        @brief Column indices that the specified joint group contains.
        @note
            The column indices point into the entire, uncompressed joint matrix.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @note
            The joint group storage will be implicitly resized (if needed) to provide
            storage for the number of joint groups that is inferred from the specified index.
        @param inputIndices
            The source address from which the column indices are to be copied.
        @param count
            The number of column indices to copy.
    */
    virtual void setJointGroupInputIndices(std::uint16_t jointGroupIndex,
                                           const std::uint16_t* inputIndices,
                                           std::uint16_t count) = 0;
    /**
        @brief Row indices that the specified joint group contains.
        @note
            The row indices point into the entire, uncompressed joint matrix.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @note
            The joint group storage will be implicitly resized (if needed) to provide
            storage for the number of joint groups that is inferred from the specified index.
        @param outputIndices
            The source address from which the row indices are to be copied.
        @param count
            The number of row indices to copy.
    */
    virtual void setJointGroupOutputIndices(std::uint16_t jointGroupIndex,
                                            const std::uint16_t* outputIndices,
                                            std::uint16_t count) = 0;
    /**
        @brief Values that the specified joint group contains.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @note
            The joint group storage will be implicitly resized (if needed) to provide
            storage for the number of joint groups that is inferred from the specified index.
        @param values
            The source address from which the values are to be copied.
        @param count
            The number of values to copy.
    */
    virtual void setJointGroupValues(std::uint16_t jointGroupIndex, const float* values, std::uint32_t count) = 0;
    /**
        @brief Joint indices that the specified joint group contains.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @note
            The joint group storage will be implicitly resized (if needed) to provide
            storage for the number of joint groups that is inferred from the specified index.
        @param jointIndices
            The source address from which the joint indices are to be copied.
        @param count
            The number of joint indices to copy.
    */
    virtual void setJointGroupJointIndices(std::uint16_t jointGroupIndex,
                                           const std::uint16_t* jointIndices,
                                           std::uint16_t count) = 0;
    /**
        @brief Input index count per each level of detail for blend shapes.
        @note
            Each element's position represents the level itself  (e.g. [0,1,2,3,4,5] Value 0 is LOD with highest of details,
            value 5 is LOD with lowest details), while the value denotes the number of input indices belonging to that level.
        @param lods
            The source address from which the lod bounds are to be copied.
        @param count
            The number of lod bounds to copy.
        @warning
            The LOD values set here are not interchangeable with the LOD indices set in DefinitionWriter::setBlendShapeNameIndices
            and DefinitionWriter::setLODBlendShapeMapping
    */
    virtual void setBlendShapeChannelLODs(const std::uint16_t* lods, std::uint16_t count) = 0;
    /**
        @brief Input indices used to index into the input vector.
        @param inputIndices
            The source address from which the input indices are to be copied.
        @param count
            The number of input indices to copy.
    */
    virtual void setBlendShapeChannelInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) = 0;
    /**
        @brief Output indices specify the positions of blend shape output values.
        @param outputIndices
            The source address from which the output indices are to be copied.
        @param count
            The number of output indices to copy.
    */
    virtual void setBlendShapeChannelOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) = 0;
    /**
        @brief Row count per each level of detail for animated maps.
        @note
            Each element's position represents the level itself  (e.g. [0,1,2,3,4,5] Value 0 is LOD with highest of details,
            value 5 is LOD with lowest details), while the value denotes the number of rows (within the conditional table),
            belonging to that level.
        @param lods
            The source address from which the lod bounds are to be copied.
        @param count
            The number of lod bounds to copy.
    */
    virtual void setAnimatedMapLODs(const std::uint16_t* lods, std::uint16_t count) = 0;
    /**
        @brief Input indices used to index into the array of input values.
        @param inputIndices
            The source address from which the input indices are to be copied.
        @param count
            The number of input indices to copy.
    */
    virtual void setAnimatedMapInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) = 0;
    /**
        @brief Output indices that specify the computed output value's position.
        @param outputIndices
            The source address from which the output indices are to be copied.
        @param count
            The number of output indices to copy.
    */
    virtual void setAnimatedMapOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) = 0;
    /**
        @brief Filter values(lower-bounds) used to decide whether a particular
            entry should be evaluated or not.
        @param fromValues
            The source address from which the filter values are to be copied.
        @param count
            The number of filter values to copy.
    */
    virtual void setAnimatedMapFromValues(const float* fromValues, std::uint16_t count) = 0;
    /**
        @brief Filter values(upper-bounds) used to decide whether a particular
            entry should be evaluated or not.
        @param toValues
            The source address from which the filter values are to be copied.
        @param count
            The number of filter values to copy.
    */
    virtual void setAnimatedMapToValues(const float* toValues, std::uint16_t count) = 0;
    /**
        @brief Computational values(slope/gradient) used for calculating the output value.
        @param slopeValues
            The source address from which the computational values are to be copied.
        @param count
            The number of computational values to copy.
    */
    virtual void setAnimatedMapSlopeValues(const float* slopeValues, std::uint16_t count) = 0;
    /**
        @brief Computational values(vertical intercept) used for calculating the output value.
        @param cutValues
            The source address from which the computational values are to be copied.
        @param count
            The number of computational values to copy.
    */
    virtual void setAnimatedMapCutValues(const float* cutValues, std::uint16_t count) = 0;
};

}  // namespace dna
