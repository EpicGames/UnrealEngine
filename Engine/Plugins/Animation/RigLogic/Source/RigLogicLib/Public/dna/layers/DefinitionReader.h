// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DescriptorReader.h"
#include "dna/types/Aliases.h"
#include "dna/types/Vector3.h"

#include <cstdint>

namespace dna {

/**
    @brief Mapping that associates a blend shape channel to it's mesh.
*/
struct MeshBlendShapeChannelMapping {
    std::uint16_t meshIndex;
    std::uint16_t blendShapeChannelIndex;
};

/**
    @brief Read-only accessors for DNA attributes that represent the rig's static data.
    @warning
        Implementors should inherit from Reader itself and not this class.
    @see Reader
*/
class DNAAPI DefinitionReader : public DescriptorReader {
protected:
    virtual ~DefinitionReader();

public:
    /**
        @brief Number of GUI controls.
    */
    virtual std::uint16_t getGUIControlCount() const = 0;
    /**
        @brief Name of the requested GUI control.
        @param index
            A name's position in the zero-indexed array of GUI control names.
        @warning
            The index must be less than the value returned by getGUIControlCount.
        @return View over the GUI control name string.
    */
    virtual StringView getGUIControlName(std::uint16_t index) const = 0;
    /**
        @brief Number of raw controls.
    */
    virtual std::uint16_t getRawControlCount() const = 0;
    /**
        @brief Name of the requested raw control.
        @param index
            A name's position in the zero-indexed array of raw control names.
        @warning
            The index must be less than the value returned by getRawControlCount.
        @return View over the control name string.
    */
    virtual StringView getRawControlName(std::uint16_t index) const = 0;
    /**
        @brief Number of raw joints.
    */
    virtual std::uint16_t getJointCount() const = 0;
    /**
        @brief Name of the requested joint.
        @param index
            A name's position in the zero-indexed array of joint names.
        @warning
            The index must be less than the value returned by getJointCount.
        @return View over the joint name string.
    */
    virtual StringView getJointName(std::uint16_t index) const = 0;
    /**
        @brief Number of joint index lists.
        @note
            This value is useful only in the context of DefinitionWriter.
    */
    virtual std::uint16_t getJointIndexListCount() const = 0;
    /**
        @brief List of joint indices for the specified LOD.
        @param lod
            The level of detail which joints are being requested.
        @warning
            The lod index must be less than the value returned by getLODCount.
        @return View over the joint indices.
        @see getLODCount
        @see getJointName
    */
    virtual ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief Index of the requested joint's parent.
        @note
            The joint hierarchy may be traversed and reconstructed using this function. Example:
            Joint names: [A, B, C, D, E, F, G, H, I]
            Hierarchy:   [0, 0, 0, 1, 1, 4, 2, 6, 2]
            Describes the following hierarchy:
            A
            + B
            | + D
            | + E
            |   + F
            + C
              + G
              | + H
              + I

            Requesting the parent index of joint 5 (joint name: F) would return 4 (joint name: E).
            Requesting the parent index of the root joint: 0 (joint name: A) would return the same index 0.
            An out of bounds request (an index greater than the number of joints returns UINT16_MAX).
        @param index
            The joint index which parent is being requested.
    */
    virtual std::uint16_t getJointParentIndex(std::uint16_t index) const = 0;
    /**
        @brief Number of blend shape channels.
    */
    virtual std::uint16_t getBlendShapeChannelCount() const = 0;
    /**
        @brief Name of the requested blend shape channel.
        @param index
            A name's position in the zero-indexed array of blend shape channel names.
        @warning
            The index must be less than the value returned by getBlendShapeChannelCount.
        @return View over the blend shape channel name string.
    */
    virtual StringView getBlendShapeChannelName(std::uint16_t index) const = 0;
    /**
        @brief Number of blend shape channel index lists.
        @note
            This value is useful only in the context of DefinitionWriter.
    */
    virtual std::uint16_t getBlendShapeChannelIndexListCount() const = 0;
    /**
        @brief List of blend shape channel indices for the specified LOD.
        @param lod
            The level of detail which blend shape channels are being requested.
        @warning
            The lod index must be less than the value returned by DescriptorReader::getLODCount.
        @return View over the blend shape channel indices.
        @warning
            These LOD indices are not interchangeable with the LOD values from BehaviorReader::getBlendShapeChannelLODs.
        @see DescriptorReader::getLODCount
        @see getBlendShapeChannelName
    */
    virtual ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief Number of animated maps.
    */
    virtual std::uint16_t getAnimatedMapCount() const = 0;
    /**
        @brief Name of the requested animated map.
        @param index
            A name's position in the zero-indexed array of animated map names.
        @warning
            The index must be less than the value returned by getAnimatedMapCount.
        @return View over the animated map name string.
    */
    virtual StringView getAnimatedMapName(std::uint16_t index) const = 0;
    /**
        @brief Number of animated map index lists.
        @note
            This value is useful only in the context of DefinitionWriter.
    */
    virtual std::uint16_t getAnimatedMapIndexListCount() const = 0;
    /**
        @brief List of animated map indices for the specified LOD.
        @param lod
            The level of detail which animated maps are being requested.
        @warning
            The lod index must be less than the value returned by getLODCount.
        @return View over the animated map indices.
        @see getLODCount
        @see getAnimatedMapName
    */
    virtual ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief Number of meshes.
    */
    virtual std::uint16_t getMeshCount() const = 0;
    /**
        @brief Name of the requested mesh.
        @param index
            A name's position in the zero-indexed array of mesh names.
        @warning
            The index must be less than the value returned by getMeshCount.
        @return View over the mesh name string.
    */
    virtual StringView getMeshName(std::uint16_t index) const = 0;
    /**
        @brief Number of mesh index lists.
        @note
            This value is useful only in the context of DefinitionWriter.
    */
    virtual std::uint16_t getMeshIndexListCount() const = 0;
    /**
        @brief List of mesh indices for the specified LOD.
        @param lod
            The level of detail which meshes are being requested.
        @warning
            The lod index must be less than the value returned by getLODCount.
        @return View over the mesh indices.
        @see getLODCount
        @see getMeshName
    */
    virtual ConstArrayView<std::uint16_t> getMeshIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief Number of mesh-blend shape channel mapping items.
    */
    virtual std::uint16_t getMeshBlendShapeChannelMappingCount() const = 0;
    /**
        @param index
            A mapping's position in the zero-indexed array of mesh-blend shape channel mappings.
        @warning
            The index must be less than the value returned by getMeshBlendShapeChannelMappingCount.
        @return A structure holding the mesh index and the associated blend shape channel index.
    */
    virtual MeshBlendShapeChannelMapping getMeshBlendShapeChannelMapping(std::uint16_t index) const = 0;
    /**
        @brief List of mesh-blend shape channel mapping indices for the specified LOD.
        @note
            The indices from this list can be used with the getMeshBlendShapeChannelMapping API
            to retrieve individual mapping items.
        @param lod
            The level of detail which meshes are being requested.
        @warning
            The lod index must be less than the value returned by getLODCount.
        @return View over the mesh blend shape channel mapping indices.
        @see getLODCount
        @see getMeshBlendShapeChannelMapping
    */
    virtual ConstArrayView<std::uint16_t> getMeshBlendShapeChannelMappingIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @param index
            A joint's position in the zero-indexed array of joint translations.
        @warning
            The index must be less than the value returned by getJointCount.
        @return The joint's translation (x, y, z).
    */
    virtual Vector3 getNeutralJointTranslation(std::uint16_t index) const = 0;
    /**
        @brief List of all translation X values.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getNeutralJointTranslation.
        @return View over all X values.
        @see getNeutralJointTranslation
    */
    virtual ConstArrayView<float> getNeutralJointTranslationXs() const = 0;
    /**
        @brief List of all translation Y values.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getNeutralJointTranslation.
        @return View over all Y values.
        @see getNeutralJointTranslation
    */
    virtual ConstArrayView<float> getNeutralJointTranslationYs() const = 0;
    /**
        @brief List of all translation Z values.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getNeutralJointTranslation.
        @return View over all Z values.
        @see getNeutralJointTranslation
    */
    virtual ConstArrayView<float> getNeutralJointTranslationZs() const = 0;
    /**
        @param index
            A joint's position in the zero-indexed array of joint rotations.
        @warning
            The index must be less than the value returned by getJointCount.
        @return The joint's rotation (x, y, z).
    */
    virtual Vector3 getNeutralJointRotation(std::uint16_t index) const = 0;
    /**
        @brief List of all rotation X values.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getNeutralJointRotation.
        @return View over all X values.
        @see getNeutralJointRotation
    */
    virtual ConstArrayView<float> getNeutralJointRotationXs() const = 0;
    /**
        @brief List of all rotation Y values.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getNeutralJointRotation.
        @return View over all Y values.
        @see getNeutralJointRotation
    */
    virtual ConstArrayView<float> getNeutralJointRotationYs() const = 0;
    /**
        @brief List of all rotation Z values.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getNeutralJointRotation.
        @return View over all Z values.
        @see getNeutralJointRotation
    */
    virtual ConstArrayView<float> getNeutralJointRotationZs() const = 0;
};

}  // namespace dna
