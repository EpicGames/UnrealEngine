// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DescriptorWriter.h"
#include "dna/types/Aliases.h"
#include "dna/types/Vector3.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors for DNA attributes that represent the rig's static data.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI DefinitionWriter : public DescriptorWriter {
protected:
    virtual ~DefinitionWriter();

public:
    /**
        @brief Delete all stored GUI control names.
    */
    virtual void clearGUIControlNames() = 0;
    /**
        @brief Name of the specified GUI control.
        @param index
            A name's position in the zero-indexed array of GUI control names.
        @note
            The control name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setGUIControlName(std::uint16_t index, const char* name) = 0;
    /**
        @brief Delete all stored raw control names.
    */
    virtual void clearRawControlNames() = 0;
    /**
        @brief Name of the specified raw control.
        @param index
            A name's position in the zero-indexed array of raw control names.
        @note
            The control name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setRawControlName(std::uint16_t index, const char* name) = 0;
    /**
        @brief Delete all stored joint names.
    */
    virtual void clearJointNames() = 0;
    /**
        @brief Name of the specified joint.
        @param index
            A name's position in the zero-indexed array of joint names.
        @note
            The joint name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setJointName(std::uint16_t index, const char* name) = 0;
    /**
        @brief Delete all stored joint indices.
    */
    virtual void clearJointIndices() = 0;
    /**
        @brief Store a list of joint indices onto a specified index.
        @param index
            A position in a zero-indexed array where joint indices are stored.
        @note
            The index denotes the position of an entire joint index list,
            not the position of it's individual elements, i.e. the row index in a 2D
            matrix of joint indices.
        @note
            The joint index storage will be implicitly resized (if needed) to provide
            storage for the number of joint indices that is inferred from the specified index.
        @param jointIndices
            The source address from which the joint indices are to be copied.
        @note
            These indices can be used to access joint names through DefinitionReader::getJointName.
        @param count
            The number of joint indices to copy.
    */
    virtual void setJointIndices(std::uint16_t index, const std::uint16_t* jointIndices, std::uint16_t count) = 0;
    /**
        @brief Delete all stored LOD to joint list index mapping entries.
    */
    virtual void clearLODJointMappings() = 0;
    /**
        @brief Set which joints belong to which level of detail.
        @param lod
            The actual level of detail to which the joints are being associated.
        @param index
            The index onto which joints indices were assigned using setJointIndices.
        @see setJointIndices
    */
    virtual void setLODJointMapping(std::uint16_t lod, std::uint16_t index) = 0;
    /**
        @brief Delete all stored blend shape channel names.
    */
    virtual void clearBlendShapeChannelNames() = 0;
    /**
        @brief Name of the specified blend shape channel.
        @param index
            A name's position in the zero-indexed array of blend shape channel names.
        @note
            The blend shape channel name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setBlendShapeChannelName(std::uint16_t index, const char* name) = 0;
    /**
        @brief Delete all stored blend shape channel indices.
    */
    virtual void clearBlendShapeChannelIndices() = 0;
    /**
        @brief Store a list of blend shape channel name indices onto a specified index.
        @param index
            A position in a zero-indexed array where blend shape channel name indices are stored.
        @note
            The index denotes the position of an entire blend shape channel index list,
            not the position of it's individual elements, i.e. the row index in a 2D
            matrix of blend shape channel indices.
        @note
            The blend shape channel index storage will be implicitly resized (if needed) to provide storage
            for the number of blend shape channel name indices that is inferred from the specified index.
        @param blendShapeChannelIndices
            The source address from which the blend shape channel name indices are to be copied.
        @note
            These indices can be used to access blend shape channel names through DefinitionReader::getBlendShapeChannelName.
        @param count
            The number of blend shape channel name indices to copy.
    */
    virtual void setBlendShapeChannelIndices(std::uint16_t index,
                                             const std::uint16_t* blendShapeChannelIndices,
                                             std::uint16_t count) = 0;
    /**
        @brief Delete all stored LOD to blend shape channel list index mapping entries.
    */
    virtual void clearLODBlendShapeChannelMappings() = 0;
    /**
        @brief Set which blend shape channels belong to which level of detail.
        @param lod
            The actual level of detail to which the blend shape channels are being associated.
        @param index
            The index onto which blend shape channel name indices were assigned using setBlendShapeChannelIndices.
        @warning
            The LOD indices set here are not interchangeable with the LOD values set in BehaviorWriter::setBlendShapeChannelLODs.
        @see setBlendShapeChannelIndices
    */
    virtual void setLODBlendShapeChannelMapping(std::uint16_t lod, std::uint16_t index) = 0;
    /**
        @brief Delete all stored animated map names.
    */
    virtual void clearAnimatedMapNames() = 0;
    /**
        @brief Name of the specified animated map.
        @param index
            A name's position in the zero-indexed array of animated map names.
        @note
            The animated map name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setAnimatedMapName(std::uint16_t index, const char* name) = 0;
    /**
        @brief Delete all stored animated map indices.
    */
    virtual void clearAnimatedMapIndices() = 0;
    /**
        @brief Store a list of animated map name indices onto a specified index.
        @param index
            A position in a zero-indexed array where animated map name indices are stored.
        @note
            The index denotes the position of an entire animated map index list,
            not the position of it's individual elements, i.e. the row index in a 2D
            matrix of animated map indices.
        @note
            The animated map index storage will be implicitly resized (if needed) to provide storage
            for the number of animated map name indices that is inferred from the specified index.
        @param animatedMapIndices
            The source address from which the animated map name indices are to be copied.
        @note
            These indices can be used to access animated map names through DefinitionReader::getAnimatedMapName.
        @param count
            The number of animated map name indices to copy.
    */
    virtual void setAnimatedMapIndices(std::uint16_t index, const std::uint16_t* animatedMapIndices, std::uint16_t count) = 0;
    /**
        @brief Delete all stored LOD to animated map list index mapping entries.
    */
    virtual void clearLODAnimatedMapMappings() = 0;
    /**
        @brief Set which animated maps belong to which level of detail.
        @param lod
            The actual level of detail to which the animated maps are being associated.
        @param index
            The index onto which animated map indices were assigned using setAnimatedMapIndices.
        @see setAnimatedMapIndices
    */
    virtual void setLODAnimatedMapMapping(std::uint16_t lod, std::uint16_t index) = 0;
    /**
        @brief Delete all stored mesh names.
    */
    virtual void clearMeshNames() = 0;
    /**
        @brief Name of the specified mesh.
        @param index
            A name's position in the zero-indexed array of mesh names.
        @note
            The mesh name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setMeshName(std::uint16_t index, const char* name) = 0;
    /**
        @brief Delete all stored mesh indices.
    */
    virtual void clearMeshIndices() = 0;
    /**
        @brief Store a list of mesh name indices onto a specified index.
        @param index
            A position in a zero-indexed array where mesh name indices are stored.
        @note
            The index denotes the position of an entire mesh index list,
            not the position of it's individual elements, i.e. the row index in a 2D
            matrix of mesh indices.
        @note
            The mesh index storage will be implicitly resized (if needed) to provide storage
            for the number of mesh name indices that is inferred from the specified index.
        @param meshIndices
            The source address from which the mesh name indices are to be copied.
        @note
            These indices can be used to access mesh names through DefinitionReader::getMeshName.
        @param count
            The number of mesh name indices to copy.
    */
    virtual void setMeshIndices(std::uint16_t index, const std::uint16_t* meshIndices, std::uint16_t count) = 0;
    /**
        @brief Delete all stored LOD to mesh list index mapping entries.
    */
    virtual void clearLODMeshMappings() = 0;
    /**
        @brief Set which meshes belong to which level of detail.
        @param lod
            The actual level of detail to which the meshes are being associated.
        @param index
            The index onto which mesh indices were assigned using setMeshIndices.
        @see setMeshIndices
    */
    virtual void setLODMeshMapping(std::uint16_t lod, std::uint16_t index) = 0;
    /**
        @brief Delete all stored mesh to blend shape channel mapping entries.
    */
    virtual void clearMeshBlendShapeChannelMappings() = 0;
    /**
        @brief Associate a blend shape channel with it's mesh.
        @param index
            A mapping's position in the zero-indexed array of mesh-blend shape channel mappings.
        @param meshIndex
            A mesh's position in the zero-indexed array of mesh names.
        @param blendShapeChannelIndex
            A blend shape channel's position in the zero-indexed array of blend shape channel names.
    */
    virtual void setMeshBlendShapeChannelMapping(std::uint32_t index,
                                                 std::uint16_t meshIndex,
                                                 std::uint16_t blendShapeChannelIndex) = 0;
    /**
        @brief A simple array describing the parent-child relationships between joints.
        @note
            Example:
            Joint names: [A, B, C, D, E, F, G, H]
            Hierarchy:   [0, 0, 0, 1, 1, 4, 2, 2]
            Describes the following hierarchy:
            A
            + B
            | + D
            | + E
            |   + F
            + C
              + G
              + H
        @param jointIndices
            The source address from which the joint indices are to be copied.
        @note
            These indices can be used to access joint names through DefinitionReader::getJointName.
        @param count
            The number of joint indices to copy.
    */
    virtual void setJointHierarchy(const std::uint16_t* jointIndices, std::uint16_t count) = 0;
    /**
        @param translations
            The source address from which the translations are to be copied.
        @param count
            The number of translation values to copy.
    */
    virtual void setNeutralJointTranslations(const Vector3* translations, std::uint16_t count) = 0;
    /**
        @param rotations
            The source address from which the rotations are to be copied.
        @param count
            The number of rotation values to copy.
    */
    virtual void setNeutralJointRotations(const Vector3* rotations, std::uint16_t count) = 0;
};

}  // namespace dna
