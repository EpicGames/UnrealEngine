// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Configuration.h"
#include "dna/types/Aliases.h"

namespace dna {

enum class Archetype {
    asian,
    black,
    caucasian,
    hispanic,
    alien,
    other
};

enum class Gender {
    male,
    female,
    other
};

}  // namespace dna
