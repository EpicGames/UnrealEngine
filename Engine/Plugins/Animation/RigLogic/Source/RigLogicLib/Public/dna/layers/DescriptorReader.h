// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/Descriptor.h"
#include "dna/layers/HeaderReader.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors for various metadata about the character and the rig.
    @warning
        Implementors should inherit from Reader itself and not this class.
    @see Reader
*/
class DNAAPI DescriptorReader : public HeaderReader {
protected:
    virtual ~DescriptorReader();

public:
    virtual StringView getName() const = 0;
    virtual Archetype getArchetype() const = 0;
    virtual Gender getGender() const = 0;
    virtual std::uint16_t getAge() const = 0;
    virtual std::uint32_t getMetaDataCount() const = 0;
    /**
        @param index
            A position in the zero-indexed array of key-value pairs.
        @warning
            The index must be less than the value returned by getMetaDataCount.
        @return View over the key name string.
    */
    virtual StringView getMetaDataKey(std::uint32_t index) const = 0;
    /**
        @brief Stored metadata value associated with the given key.
        @note
            If no value is associated with the given key, the returned view
            will contain nullptr and will have a size of 0.
        @param key
            A unique-known key that has a value associated to it.
        @warning
            The key must be null-terminated.
        @return View over the metadata value string.
    */
    virtual StringView getMetaDataValue(const char* key) const = 0;
    virtual TranslationUnit getTranslationUnit() const = 0;
    virtual RotationUnit getRotationUnit() const = 0;
    virtual CoordinateSystem getCoordinateSystem() const = 0;
    virtual RotationSequence getRotationSequence() const = 0;
    virtual RotationSign getRotationSign() const = 0;
    virtual FaceWindingOrder getFaceWindingOrder() const = 0;
    /**
        @brief Available levels of detail (e.g. 6 which means the following levels are available:
            [0,1,2,3,4,5], where 0 is the LOD with the highest details, and 5 is the LOD with
            lowest details).
    */
    virtual std::uint16_t getLODCount() const = 0;
    /**
        @brief The maximum level of detail stored in the DNA data for this character.
        @note
            The value is relative to LOD-0 from the database.
    */
    virtual std::uint16_t getDBMaxLOD() const = 0;
    /**
        @brief Name of the input control interface used to drive this character rig.
        @note
            This parameter denotes the character's input control complexity.
    */
    virtual StringView getDBComplexity() const = 0;
    /**
        @brief Name of the database from which the character originates.
        @note
            All characters from the same database must have the same Definition, but may
            have different complexity or LOD.
    */
    virtual StringView getDBName() const = 0;
};

}  // namespace dna
