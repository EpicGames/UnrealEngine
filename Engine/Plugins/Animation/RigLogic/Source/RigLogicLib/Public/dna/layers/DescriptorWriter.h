// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/Descriptor.h"
#include "dna/layers/HeaderWriter.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors to various metadata about the character and the rig.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI DescriptorWriter : public HeaderWriter {
protected:
    virtual ~DescriptorWriter();

public:
    /**
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an allocation.
    */
    virtual void setName(const char* name) = 0;
    virtual void setArchetype(Archetype archetype) = 0;
    virtual void setGender(Gender gender) = 0;
    virtual void setAge(std::uint16_t age) = 0;
    /**
        @brief Empties the metadata storage, delete all key-value pairs.
    */
    virtual void clearMetaData() = 0;
    /**
        @brief Associate the metadata value with the given key.
        @param key
            A unique, null-terminated key, to which the given value will be assigned.
        @param value
            A null-terminated, metadata value, which is to be assigned to the given key.
        @note
            Consecutive calls using the same key will overwrite any existing data.
        @note
            Passing nullptr as the value argument will cause the associated key to be deleted.
    */
    virtual void setMetaData(const char* key, const char* value) = 0;
    virtual void setTranslationUnit(TranslationUnit unit) = 0;
    virtual void setRotationUnit(RotationUnit unit) = 0;
    virtual void setCoordinateSystem(CoordinateSystem system) = 0;
    virtual void setRotationSequence(RotationSequence rotationSequence) = 0;
    virtual void setRotationSign(const RotationSign& rotationSign) = 0;
    virtual void setFaceWindingOrder(FaceWindingOrder faceWindingOrder) = 0;
    /**
        @brief Available levels of detail (e.g. 6 which means the following levels are available:
            [0,1,2,3,4,5], where 0 is the LOD with the highest details, and 5 is the LOD with
            lowest details).
        @param lodCount
            The number of levels available.
    */
    virtual void setLODCount(std::uint16_t lodCount) = 0;
    /**
        @brief The maximum level of detail stored in the DNA data for this character.
    */
    virtual void setDBMaxLOD(std::uint16_t lod) = 0;
    /**
        @brief Name of the input control interface used to drive this character rig.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setDBComplexity(const char* name) = 0;
    /**
        @brief Name of the database from which the character originates.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setDBName(const char* name) = 0;
};

}  // namespace dna
