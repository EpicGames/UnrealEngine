// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/types/Vector3.h"

#include <cstdint>

namespace dna {

struct TextureCoordinate {
    float u;
    float v;
};

using Position = Vector3;
using Normal = Vector3;
using Delta = Vector3;

struct VertexLayout {
    std::uint32_t position;
    std::uint32_t textureCoordinate;
    std::uint32_t normal;
};

}  // namespace dna
