// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionReader.h"
#include "dna/layers/Geometry.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors to the geometry data associated with a rig.
    @warning
        Implementors should inherit from Reader itself and not this class.
*/
class DNAAPI GeometryReader : public virtual DefinitionReader {
protected:
    virtual ~GeometryReader();

public:
    /**
        @brief Number of vertex positions in the entire mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint32_t getVertexPositionCount(std::uint16_t meshIndex) const = 0;
    /**
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param vertexIndex
            The index of the vertex position in the zero-indexed array of vertex positions.
        @warning
            vertexIndex must be less than the value returned by getVertexPositionCount.
        @note
            The vertices are sorted by the vertex ID.
        @return The vertex position.
    */
    virtual Position getVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const = 0;
    /**
        @brief List of all vertex position X values for the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexPosition.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all X values.
        @see getVertexPosition
    */
    virtual ConstArrayView<float> getVertexPositionXs(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of all vertex position Y values for the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexPosition.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all Y values.
        @see getVertexPosition
    */
    virtual ConstArrayView<float> getVertexPositionYs(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of all vertex position Z values for the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexPosition.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all Z values.
        @see getVertexPosition
    */
    virtual ConstArrayView<float> getVertexPositionZs(std::uint16_t meshIndex) const = 0;
    /**
        @brief Number of texture coordinates in the entire mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint32_t getVertexTextureCoordinateCount(std::uint16_t meshIndex) const = 0;
    /**
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param textureCoordinateIndex
            The index of the texture coordinate in the zero-indexed array of texture coordinates.
        @warning
            textureCoordinateIndex must be less than the value returned by getVertexTextureCoordinateCount.
        @return The texture coordinate.
    */
    virtual TextureCoordinate getVertexTextureCoordinate(std::uint16_t meshIndex, std::uint32_t textureCoordinateIndex) const = 0;
    /**
        @brief List of all texture coordinate U values for the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexTextureCoordinate.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all U values.
        @see getVertexTextureCoordinate
    */
    virtual ConstArrayView<float> getVertexTextureCoordinateUs(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of all texture coordinate V values for the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexTextureCoordinate.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all V values.
        @see getVertexTextureCoordinate
    */
    virtual ConstArrayView<float> getVertexTextureCoordinateVs(std::uint16_t meshIndex) const = 0;
    /**
        @brief Number of vertex normals in the entire mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint32_t getVertexNormalCount(std::uint16_t meshIndex) const = 0;
    /**
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param normalIndex
            The index of the vertex normal in the zero-indexed array of vertex normals.
        @warning
            normalIndex must be less than the value returned by getVertexNormalCount.
        @return The vertex normal.
    */
    virtual Normal getVertexNormal(std::uint16_t meshIndex, std::uint32_t normalIndex) const = 0;
    /**
        @brief List of all normal X values for the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexNormal.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all X values.
        @see getVertexNormal
    */
    virtual ConstArrayView<float> getVertexNormalXs(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of all normal Y value for the referenced meshs.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexNormal.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all Y values.
        @see getVertexNormal
    */
    virtual ConstArrayView<float> getVertexNormalYs(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of all normal Z values for the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexNormal.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all Z values.
        @see getVertexNormal
    */
    virtual ConstArrayView<float> getVertexNormalZs(std::uint16_t meshIndex) const = 0;
    /**
        @brief Number of vertex layouts in the entire mesh.
        @note
            A vertex layout is a collection of vertex attributes.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint32_t getVertexLayoutCount(std::uint16_t meshIndex) const = 0;
    /**
        @brief Vertex layouts contain only attribute indices which can be used to query
            the actual attributes, such as positions, texture coordinates and normals,
            which are associated with the vertex.
        @note
            The indices from a layout are usable with the above defined APIs.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param layoutIndex
            The index of the layout in the zero-indexed array of vertex layouts.
        @warning
            layoutIndex must be less than the value returned by getVertexLayoutCount.
        @see getVertexPosition
        @see getVertexTextureCoordinate
        @see getVertexNormal
    */
    virtual VertexLayout getVertexLayout(std::uint16_t meshIndex, std::uint32_t layoutIndex) const = 0;
    /**
        @brief Position indices for each vertex of the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexLayout.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all vertex position indices values.
        @see getVertexLayout
    */
    virtual ConstArrayView<std::uint32_t> getVertexLayoutPositionIndices(std::uint16_t meshIndex) const = 0;
    /**
        @brief Texture coordinate indices for each vertex of the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexLayout.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all vertex texture coordinate indices.
        @see getVertexLayout
    */
    virtual ConstArrayView<std::uint32_t> getVertexLayoutTextureCoordinateIndices(std::uint16_t meshIndex) const = 0;
    /**
        @brief Normal indices for each vertex of the referenced mesh.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getVertexLayout.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @return View over all vertex normal indices.
        @see getVertexLayout
    */
    virtual ConstArrayView<std::uint32_t> getVertexLayoutNormalIndices(std::uint16_t meshIndex) const = 0;
    /**
        @brief Number of faces that belong to the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint32_t getFaceCount(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of vertex layout indices the belong to a face on the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param faceIndex
            A face's position in the zero-indexed array of faces that belong to
            the above referenced mesh.
        @warning
            faceIndex must be less than the value returned by getFaceCount.
        @return View over the list of vertex layout indices.
        @see getVertexLayout
    */
    virtual ConstArrayView<std::uint32_t> getFaceVertexLayoutIndices(std::uint16_t meshIndex, std::uint32_t faceIndex) const = 0;
    /**
        @brief The maximum number of joints that may influence any single vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint16_t getMaximumInfluencePerVertex(std::uint16_t meshIndex) const = 0;
    /**
        @brief Number of skin weights associated with the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const = 0;
    /**
        @brief List of skin weights influencing the requested vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param vertexIndex
            A position in the zero-indexed array of vertices.
        @warning
            vertexIndex must be less than the value returned by getVertexPositionCount.
        @return View over the list of skin weights.
    */
    virtual ConstArrayView<float> getSkinWeightsValues(std::uint16_t meshIndex, std::uint32_t vertexIndex) const = 0;
    /**
        @brief List of joint indices associated with each skin weight for the specified vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param vertexIndex
            A position in the zero-indexed array of vertices.
        @warning
            vertexIndex must be less than the value returned by getVertexPositionCount.
        @note
            The joint indices are stored in the same order as the weights they
            are associated with.
        @return View over the list of joint indices.
    */
    virtual ConstArrayView<std::uint16_t> getSkinWeightsJointIndices(std::uint16_t meshIndex,
                                                                     std::uint32_t vertexIndex) const = 0;
    /**
        @brief Number of blend shapes that belong to the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual std::uint16_t getBlendShapeTargetCount(std::uint16_t meshIndex) const = 0;
    /** @brief The matching blend shape channel index of the requested blend shape target.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @warning
            blendShapeTargetIndex must be less than the value returned by getBlendShapeTargetCount.
        @see DefinitionReader::getBlendShapeChannelName
    */
    virtual std::uint16_t getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const = 0;
    /**
        @brief Number of deltas that belong to the specified blend shape.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @warning
            blendShapeTargetIndex must be less than the value returned by getBlendShapeTargetCount.
    */
    virtual std::uint32_t getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const = 0;
    /**
        @brief List of deltas for each affected vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @warning
            blendShapeTargetIndex must be less than the value returned by getBlendShapeTargetCount.
        @param deltaIndex
            A position in the zero-indexed array of blend shapes deltas.
        @warning
            deltaIndex must be less than the value returned by getBlendShapeTargetDeltaCount.
    */
    virtual Delta getBlendShapeTargetDelta(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           std::uint32_t deltaIndex) const = 0;
    /**
        @brief List of all delta X values for the referenced blend shape target.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getBlendShapeTargetDelta.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @warning
            blendShapeTargetIndex must be less than the value returned by getBlendShapeTargetCount.
        @return View over all X values.
        @see getBlendShapeTargetDelta
    */
    virtual ConstArrayView<float> getBlendShapeTargetDeltaXs(std::uint16_t meshIndex,
                                                             std::uint16_t blendShapeTargetIndex) const = 0;
    /**
        @brief List of all delta Y values for the referenced blend shape target.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getBlendShapeTargetDelta.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @warning
            blendShapeTargetIndex must be less than the value returned by getBlendShapeTargetCount.
        @return View over all Y values.
        @see getBlendShapeTargetDelta
    */
    virtual ConstArrayView<float> getBlendShapeTargetDeltaYs(std::uint16_t meshIndex,
                                                             std::uint16_t blendShapeTargetIndex) const = 0;
    /**
        @brief List of all delta Z values for the referenced blend shape target.
        @note
            This is an advanced API for performance critical access, for more convenient usage see getBlendShapeTargetDelta.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @warning
            blendShapeTargetIndex must be less than the value returned by getBlendShapeTargetCount.
        @return View over all Z values.
        @see getBlendShapeTargetDelta
    */
    virtual ConstArrayView<float> getBlendShapeTargetDeltaZs(std::uint16_t meshIndex,
                                                             std::uint16_t blendShapeTargetIndex) const = 0;
    /**
        @brief Vertex position indices affected by the referenced blend shape target.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @warning
            blendShapeTargetIndex must be less than the value returned by getBlendShapeTargetCount.
        @note
            The vertex position indices are stored in the same order as the deltas they
            are associated with.
            These indices can be used to query the associated vertices themselves through getVertexPosition.
        @see getVertexPosition
        @return View over the list of vertex position indices.
    */
    virtual ConstArrayView<std::uint32_t> getBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                                           std::uint16_t blendShapeTargetIndex) const = 0;
};

}  // namespace dna
