// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionWriter.h"
#include "dna/layers/Geometry.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors for the geometry data associated with a rig.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI GeometryWriter : public virtual DefinitionWriter {
protected:
    virtual ~GeometryWriter();

public:
    /**
        @brief Delete all meshes.
    */
    virtual void clearMeshes() = 0;
    /**
        @brief Delete the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual void deleteMesh(std::uint16_t meshIndex) = 0;
    /**
        @brief List of vertex positions.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param positions
            The source address from which the vertex positions are to be copied.
        @param count
            The number of vertex positions to copy.
        @note
            The mesh storage will be implicitly resized (if needed) to provide
            storage for the number of meshes that is inferred from the specified index.
    */
    virtual void setVertexPositions(std::uint16_t meshIndex, const Position* positions, std::uint32_t count) = 0;
    /**
        @brief List of vertex texture coordinates.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param textureCoordinates
            The source address from which the texture coordinates are to be copied.
        @param count
            The number of texture coordinates to copy.
        @note
            The mesh storage will be implicitly resized (if needed) to provide
            storage for the number of meshes that is inferred from the specified index.
    */
    virtual void setVertexTextureCoordinates(std::uint16_t meshIndex,
                                             const TextureCoordinate* textureCoordinates,
                                             std::uint32_t count) = 0;
    /**
        @brief List of vertex normals.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param normals
            The source address from which the normals are to be copied.
        @param count
            The number of normals to copy.
        @note
            The mesh storage will be implicitly resized (if needed) to provide
            storage for the number of meshes that is inferred from the specified index.
    */
    virtual void setVertexNormals(std::uint16_t meshIndex, const Normal* normals, std::uint32_t count) = 0;
    /**
        @brief List of vertex layouts the belong to the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param layouts
            The source address from which the layouts are to be copied.
        @param count
            The number of layouts to copy.
        @note
            The mesh storage will be implicitly resized (if needed) to provide
            storage for the number of meshes that is inferred from the specified index.
    */
    virtual void setVertexLayouts(std::uint16_t meshIndex, const VertexLayout* layouts, std::uint32_t count) = 0;
    /**
        @brief Delete all lists of vertex layout indices for the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual void clearFaceVertexLayoutIndices(std::uint16_t meshIndex) = 0;
    /**
        @brief Vertex layout indices that belong to the specified face.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param faceIndex
            A face's position in the zero-indexed array of faces that belong to
            the above referenced mesh.
        @param layoutIndices
            The source address from which the layout indices are to be copied.
        @note
            The layout indices point into the array that is set through setVertexLayouts
        @param count
            The number of vertices to copy.
        @note
            Both the mesh storage itself and it's face storage will be implicitly
            resized (if needed) to provide storage for the number of meshes and/or
            faces that are inferred from the specified indexes.
    */
    virtual void setFaceVertexLayoutIndices(std::uint16_t meshIndex,
                                            std::uint32_t faceIndex,
                                            const std::uint32_t* layoutIndices,
                                            std::uint32_t count) = 0;
    /**
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param maxInfluenceCount
            The maximum number of joints that may influence any single vertex.
    */
    virtual void setMaximumInfluencePerVertex(std::uint16_t meshIndex, std::uint16_t maxInfluenceCount) = 0;
    /**
        @brief Delete all skin weights for the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual void clearSkinWeights(std::uint16_t meshIndex) = 0;
    /**
        @brief List of skin weights influencing the referenced vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param vertexIndex
            A position in the zero-indexed array of vertex positions.
        @param weights
            The source address from which the weights are to be copied.
        @param count
            The number of weights to copy.
        @note
            Both the mesh storage itself and it's skin weight storage will be implicitly
            resized (if needed) to provide storage for the number of meshes and/or
            skin-weight lists that are inferred from the specified indexes.
        @warning
            The sum of weights must add up to 1.
    */
    virtual void setSkinWeightsValues(std::uint16_t meshIndex,
                                      std::uint32_t vertexIndex,
                                      const float* weights,
                                      std::uint16_t count) = 0;
    /**
        @brief List of joint indices associated with each skin weight for the specified vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param vertexIndex
            A position in the zero-indexed array of vertex positions.
        @param jointIndices
            The source address from which the joint indices are to be copied.
        @param count
            The number of joint indices to copy.
        @note
            Both the mesh storage itself and it's joint index list storage will be implicitly
            resized (if needed) to provide storage for the number of meshes and/or
            joint index lists that are inferred from the specified indexes.
        @warning
            The joint indices must be stored in the same order as the weights they
            are associated with.
    */
    virtual void setSkinWeightsJointIndices(std::uint16_t meshIndex,
                                            std::uint32_t vertexIndex,
                                            const std::uint16_t* jointIndices,
                                            std::uint16_t count) = 0;
    /**
        @brief Delete all blend shape targets for the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by getMeshCount.
    */
    virtual void clearBlendShapeTargets(std::uint16_t meshIndex) = 0;
    /** @brief The matching blend shape channel index of the specified blend shape target.
        @note
            Associate the mesh-local blend shape target index with the absolute blend shape channel
            index as found in the Definition layer.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @param blendShapeChannelIndex
            The index of the specified blend shape channel in the Definition layer.
        @note
            Both the mesh storage itself and it's blend shape target storage will be implicitly
            resized (if needed) to provide storage for the number of meshes and/or
            blend shape targets that are inferred from the specified indexes.
    */
    virtual void setBlendShapeChannelIndex(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           std::uint16_t blendShapeChannelIndex) = 0;
    /**
        @brief List of deltas for each affected vertex.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @param deltas
            The source address from which the blend shape target deltas are to be copied.
        @param count
            The number of blend shape target deltas to copy.
        @note
            Both the mesh storage itself and it's blend shape target storage will be implicitly
            resized (if needed) to provide storage for the number of meshes and/or
            blend shape targets that are inferred from the specified indexes.
    */
    virtual void setBlendShapeTargetDeltas(std::uint16_t meshIndex,
                                           std::uint16_t blendShapeTargetIndex,
                                           const Delta* deltas,
                                           std::uint32_t count) = 0;
    /**
        @brief Vertex position indices affected by the specified blend shape target.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param blendShapeTargetIndex
            A position in the zero-indexed array of blend shape targets within the specified mesh.
        @param vertexIndices
            The source address from which the vertex position indices are to be copied.
        @param count
            The number of vertex position indices to copy.
        @note
            Both the mesh storage itself and it's blend shape target storage will be implicitly
            resized (if needed) to provide storage for the number of meshes and/or
            blend shape targets that are inferred from the specified indexes.
        @warning
            The vertex position indices must be stored in the same order as the deltas
            they are associated with.
    */
    virtual void setBlendShapeTargetVertexIndices(std::uint16_t meshIndex,
                                                  std::uint16_t blendShapeTargetIndex,
                                                  const std::uint32_t* vertexIndices,
                                                  std::uint32_t count) = 0;
};

}  // namespace dna
