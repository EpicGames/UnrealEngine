// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors to the header data associated with a rig.
    @warning
        Implementors should inherit from Reader itself and not this class.
*/
class DNAAPI HeaderReader {
protected:
    virtual ~HeaderReader();

public:
    virtual std::uint16_t getFileFormatGeneration() const = 0;
    virtual std::uint16_t getFileFormatVersion() const = 0;
};

}  // namespace dna
