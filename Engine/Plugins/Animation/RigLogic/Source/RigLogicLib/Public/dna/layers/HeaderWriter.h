// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors for the header data associated with a rig.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI HeaderWriter {
protected:
    virtual ~HeaderWriter();

public:
    virtual void setFileFormatGeneration(std::uint16_t generation) = 0;
    virtual void setFileFormatVersion(std::uint16_t version) = 0;
};

}  // namespace dna
