// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

namespace dna {

enum class TranslationRepresentation {
    Vector
};

enum class RotationRepresentation {
    EulerAngles,
    Quaternion
};

enum class ScaleRepresentation {
    Vector
};

}  // namespace dna
