// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionReader.h"
#include "dna/layers/JointBehaviorMetadata.h"

namespace dna {

/**
    @brief Read-only accessors to the joint metadata that affects how are they evaluated at runtime.
    @warning
        Implementors should inherit from Reader itself and not this class.
    @see Reader
*/
class DNAAPI JointBehaviorMetadataReader : public virtual DefinitionReader {
protected:
    virtual ~JointBehaviorMetadataReader();

public:
    /**
        @brief Get the desired internal representation of a joint's translation component during rig evaluation.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @return The representation of the joint's translation component.
        @note
            This metadata affects how joints are calculated, as the given representation will be used internally
            by the implementation that evaluates them.
    */
    virtual TranslationRepresentation getJointTranslationRepresentation(std::uint16_t jointIndex) const = 0;
    /**
        @brief Get the desired internal representation of a joint's rotation component during rig evaluation.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @return The representation of the joint's rotation component.
        @note
            This metadata affects how joints are calculated, as the given representation will be used internally
            by the implementation that evaluates them.
    */
    virtual RotationRepresentation getJointRotationRepresentation(std::uint16_t jointIndex) const = 0;
    /**
        @brief Get the desired internal representation of a joint's scale component during rig evaluation.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @return The representation of the joint's scale component.
        @note
            This metadata affects how joints are calculated, as the given representation will be used internally
            by the implementation that evaluates them.
    */
    virtual ScaleRepresentation getJointScaleRepresentation(std::uint16_t jointIndex) const = 0;
};

}  // namespace dna
