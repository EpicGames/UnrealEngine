// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionWriter.h"
#include "dna/layers/JointBehaviorMetadata.h"

namespace dna {

/**
    @brief Write-only accessors to the joint metadata that affects how are they evaluated at runtime.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI JointBehaviorMetadataWriter : public virtual DefinitionWriter {
protected:
    virtual ~JointBehaviorMetadataWriter();

public:
    /**
        @brief Delete all representations.
    */
    virtual void clearJointRepresentations() = 0;
    /**
        @brief Set the desired internal representation of a joint's translation component during rig evaluation.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @param representation
            The desired representation of the joint's translation component.
        @note
            This metadata affects how joints are calculated, as the given representation will be used internally
            by the implementation that evaluates them.
    */
    virtual void setJointTranslationRepresentation(std::uint16_t jointIndex, TranslationRepresentation representation) = 0;
    /**
        @brief Set the desired internal representation of a joint's rotation component during rig evaluation.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @param representation
            The desired representation of the joint's rotation component.
        @note
            This metadata affects how joints are calculated, as the given representation will be used internally
            by the implementation that evaluates them.
    */
    virtual void setJointRotationRepresentation(std::uint16_t jointIndex, RotationRepresentation representation) = 0;
    /**
        @brief Set the desired internal representation of a joint's scale component during rig evaluation.
        @param jointIndex
            A joint's position in the zero-indexed array of joints.
        @warning
            The index must be less than the value returned by getJointCount.
        @param representation
            The desired representation of the joint's scale component.
        @note
            This metadata affects how joints are calculated, as the given representation will be used internally
            by the implementation that evaluates them.
    */
    virtual void setJointScaleRepresentation(std::uint16_t jointIndex, ScaleRepresentation representation) = 0;
};

}  // namespace dna
