// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace dna {

enum class ActivationFunction {
    linear,
    relu,
    leakyrelu,
    tanh,
    sigmoid
};

}  // namespace dna
