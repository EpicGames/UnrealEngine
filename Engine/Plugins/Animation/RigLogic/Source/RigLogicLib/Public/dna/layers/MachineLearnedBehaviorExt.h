// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

namespace dna {

enum class MachineLearnedBehaviorOperationType : std::uint16_t {
    Unspecified,
    Gather,
    Scatter,
    MLP,
    WeightedSum
};

enum class MachineLearnedBehaviorParameterKey : std::uint16_t {
    JointTranslationType,
    JointRotationType,
    JointScaleType,
    JointCoordinateSystemAxisX,
    JointCoordinateSystemAxisY,
    JointCoordinateSystemAxisZ,
    JointRotationSignAxisX,
    JointRotationSignAxisY,
    JointRotationSignAxisZ,
    JointRotationSequence
};

}  // namespace dna
