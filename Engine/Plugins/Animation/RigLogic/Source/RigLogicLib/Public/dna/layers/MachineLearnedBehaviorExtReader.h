// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/MachineLearnedBehaviorExt.h"
#include "dna/layers/MachineLearnedBehaviorReader.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors to the neural network extension data associated with a rig.
    @warning
        Implementors should inherit from Reader itself and not this class.
    @see Reader
*/
class DNAAPI MachineLearnedBehaviorExtReader : public virtual MachineLearnedBehaviorReader {
protected:
    virtual ~MachineLearnedBehaviorExtReader();

public:
    /**
        @brief Number of distinct ML models.
    */
    virtual std::uint16_t getMLTypeCount() const = 0;
    /**
        @brief Number of sets of operations within a single model.
        @note Operations within a single set are safe to execute in parallel, while each operation set itself is expected to be
              executed sequentially, one after the other.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
    */
    virtual std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const = 0;
    /**
        @brief Number of operation within a single operation set.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @warning
            The index must be less than the value returned by getMLOperationSetCount.
    */
    virtual std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const = 0;
    /**
        @brief Get the type of the selected operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @warning
            The index must be less than the value returned by getMLOperationSetCount.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @warning
            The index must be less than the value returned by getMLOperationCount.
        @return The operation type.
    */
    virtual MachineLearnedBehaviorOperationType getMLOperationType(std::uint16_t mlTypeIndex,
                                                                   std::uint16_t mlOperationSetIndex,
                                                                   std::uint16_t mlOperationIndex) const = 0;
    /**
        @brief List of optional static parameters that need to be passed to an operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @warning
            The index must be less than the value returned by getMLOperationSetCount.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @warning
            The index must be less than the value returned by getMLOperationCount.
        @return View over operation parameters
    */
    virtual ConstArrayView<std::uint32_t> getMLOperationParameters(std::uint16_t mlTypeIndex,
                                                                   std::uint16_t mlOperationSetIndex,
                                                                   std::uint16_t mlOperationIndex) const = 0;
    /**
        @brief List of operation set indices that are specified as dependencies of the current operation.
        @note This list needs to be paired with the list returned by getMLOperationDependencyOperationIndices, where each
       operation set index and operation index under the same index form a single pair, specifying the (operation-set, operation)
       index pair that points to a dependency of the current operation.
        @note As operation sets are evaluated in lock-step, each subsequent operation set will be able to access the results of
       previous operation sets, and these indices might reference operations whose results are needed as input for the current
       operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @warning
            The index must be less than the value returned by getMLOperationSetCount.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @warning
            The index must be less than the value returned by getMLOperationCount.
        @return View over operation set indices
    */
    virtual ConstArrayView<std::uint16_t> getMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                                      std::uint16_t mlOperationSetIndex,
                                                                                      std::uint16_t mlOperationIndex) const = 0;
    /**
        @brief List of operation indices that are specified as dependencies of the current operation.
        @note This list needs to be paired with the list returned by getMLOperationDependencyOperationSetIndices, where each
       operation set index and operation index under the same index form a single pair, specifying the (operation-set, operation)
       index pair that points to a dependency of the current operation.
        @note As operation sets are evaluated in lock-step, each subsequent operation set will be able to access the results of
       previous operation sets, and these indices might reference operations whose results are needed as input for the current
       operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @warning
            The index must be less than the value returned by getMLOperationSetCount.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @warning
            The index must be less than the value returned by getMLOperationCount.
        @return View over operation indices
    */
    virtual ConstArrayView<std::uint16_t> getMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                                   std::uint16_t mlOperationSetIndex,
                                                                                   std::uint16_t mlOperationIndex) const = 0;
    /**
        @brief Number of operation index lists.
        @note This value is useful only in the context of MachineLearnedBehaviorExtWriter.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @warning
            The index must be less than the value returned by getMLOperationSetCount.
    */
    virtual std::uint16_t getMLOperationIndexListCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const = 0;
    /**
        @brief List of operation indices for the specified LOD.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @warning
            The index must be less than the value returned by getMLTypeCount.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @warning
            The index must be less than the value returned by getMLOperationSetCount.
        @param lod
            The level of detail for which operations are being requested.
        @warning
            The lod index must be less than the value returned by getLODCount.
        @return View over the operation instance indices.
        @see DescriptorReader::getLODCount
    */
    virtual ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t mlTypeIndex,
                                                                      std::uint16_t mlOperationSetIndex,
                                                                      std::uint16_t lod) const = 0;
    /**
        @brief Input index list which point into the input buffer used by RigLogic.
        @note This buffer is partially populated by the ML model itself, providing the joint transformation values
              which the ML joints evaluator takes and forwards as joint outputs.
    */
    virtual ConstArrayView<std::uint16_t> getMLJointsInputIndices() const = 0;
    /**
        @brief Output index list which specifies the joint attribute index for each input index.
        @note Each output index is paired with an input index returned by getMLJointsInputIndices.
    */
    virtual ConstArrayView<std::uint16_t> getMLJointsOutputIndices() const = 0;
    /**
        @brief Optional metadata that can be assigned to ML joints.
        @note This is used to provide the joint attribute count that the ML model outputs.
    */
    virtual ConstArrayView<std::uint16_t> getMLJointsParameterKeys() const = 0;
    /**
        @brief Optional metadata that can be assigned to ML joints.
        @note This is used to provide the joint attribute count that the ML model outputs.
    */
    virtual ConstArrayView<std::uint16_t> getMLJointsParameterValues() const = 0;
};

}  // namespace dna
