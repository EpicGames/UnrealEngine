// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/MachineLearnedBehaviorExt.h"
#include "dna/layers/MachineLearnedBehaviorWriter.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors for the operation instance data associated with a rig.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI MachineLearnedBehaviorExtWriter : public virtual MachineLearnedBehaviorWriter {
protected:
    virtual ~MachineLearnedBehaviorExtWriter();

public:
    /**
        @brief Delete all stored ML extension data.
    */
    virtual void clearMLExtData() = 0;
    /**
        @brief Delete all data associated with a single ML type.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
    */
    virtual void clearMLOperationSets(std::uint16_t mlTypeIndex) = 0;
    /**
        @brief Delete all operations within a single operation set.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
    */
    virtual void clearMLOperations(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) = 0;
    /**
        @brief Set the type of the ML operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @param operationType
            The type of the operation.
        @note
            The storage will be implicitly resized (if needed) to provide
            storage for the new data to be stored.
    */
    virtual void setMLOperationType(std::uint16_t mlTypeIndex,
                                    std::uint16_t mlOperationSetIndex,
                                    std::uint16_t mlOperationIndex,
                                    MachineLearnedBehaviorOperationType operationType) = 0;
    /**
        @brief Set optional static parameters that need to be passed to an operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @note
            The storage will be implicitly resized (if needed) to provide
            storage for the new data to be stored.
        @param parameters
            The source address from which the parameters are to be copied.
        @param count
            The number of parameters to copy.
    */
    virtual void setMLOperationParameters(std::uint16_t mlTypeIndex,
                                          std::uint16_t mlOperationSetIndex,
                                          std::uint16_t mlOperationIndex,
                                          const std::uint32_t* parameters,
                                          std::uint16_t count) = 0;
    /**
        @brief Set the indices of operation sets that are dependencies of this operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @note
            The storage will be implicitly resized (if needed) to provide
            storage for the new data to be stored.
        @param indices
            The source address from which the dependency operation set indices are to be copied.
        @param count
            The number of dependency operation set indices to copy.
        @note This list needs to be paired with the list set by setMLOperationDependencyOperationIndices, where each operation
              set index and operation index under the same index form a single pair, specifying the (operation-set, operation)
       index pair that points to a dependency of the current operation.
    */
    virtual void setMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                             std::uint16_t mlOperationSetIndex,
                                                             std::uint16_t mlOperationIndex,
                                                             const std::uint16_t* indices,
                                                             std::uint16_t count) = 0;
    /**
        @brief Set the indices of operations that are dependencies of this operation.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @param mlOperationIndex
            The operation position in the zero-indexed array of operations within the selected set.
        @note
            The storage will be implicitly resized (if needed) to provide
            storage for the new data to be stored.
        @param indices
            The source address from which the dependency operation indices are to be copied.
        @param count
            The number of dependency operation indices to copy.
        @note This list needs to be paired with the list set by setMLOperationDependencyOperationSetIndices, where each operation
              set index and operation index under the same index form a single pair, specifying the (operation-set, operation)
       index pair that points to a dependency of the current operation.
    */
    virtual void setMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                          std::uint16_t mlOperationSetIndex,
                                                          std::uint16_t mlOperationIndex,
                                                          const std::uint16_t* indices,
                                                          std::uint16_t count) = 0;
    /**
        @brief Delete all stored operation indices and their corresponding LOD mappings.
    */
    virtual void clearMLOperationIndicesAndLODMappings() = 0;
    /**
        @brief Delete the operation indices used for operation-LOD mapping for a specific operation set.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
    */
    virtual void clearMLOperationIndices(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) = 0;
    /**
        @brief Delete the index-list-LOD mapping entries used for operation-LOD mapping for a specific operation set.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
    */
    virtual void clearLODMLOperationMappings(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) = 0;
    /**
        @brief Store a list of operation indices onto a specified index.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @param index
            A position in a zero-indexed array where operation indices are stored.
        @note
            The index denotes the position of an entire operation index list,
            not the position of its individual elements, i.e. the row index in a 2D
            matrix of operation indices.
        @note
            The operation index storage will be implicitly resized (if needed) to provide storage
            for the number of operation indices that is inferred from the specified index.
        @param mlOperationIndices
            The source address from which the operation indices are to be copied.
        @note
            These indices can be used to access operations through the above defined APIs.
        @param count
            The number of operation indices to copy.
        @note
            The storage will be implicitly resized (if needed) to provide
            storage for the new data to be stored.
    */
    virtual void setMLOperationIndices(std::uint16_t mlTypeIndex,
                                       std::uint16_t mlOperationSetIndex,
                                       std::uint16_t index,
                                       const std::uint16_t* mlOperationIndices,
                                       std::uint16_t count) = 0;
    /**
        @brief Set which operations belong to which level of detail.
        @param mlTypeIndex
            The ML model position in the zero-indexed array of ML models.
        @param mlOperationSetIndex
            The operation set position in the zero-indexed array of operation sets.
        @param lod
            The actual level of detail to which the operations are being associated.
        @param index
            The index onto which operation indices were assigned using setMLOperationIndices.
        @see setMLOperationIndices
    */
    virtual void setLODMLOperationMapping(std::uint16_t mlTypeIndex,
                                          std::uint16_t mlOperationSetIndex,
                                          std::uint16_t lod,
                                          std::uint16_t index) = 0;
    /**
        @brief Input index list which point into the input buffer used by RigLogic.
        @note This buffer is partially populated by the ML model itself, providing the joint transformation values
              which the ML joints evaluator takes and forwards as joint outputs.
        @param inputIndices
            The source address from which the input indices are to be copied.
        @param count
            The number of input indices to copy.
    */
    virtual void setMLJointsInputIndices(const std::uint16_t* inputIndices, std::uint16_t count) = 0;
    /**
        @brief Output index list which specifies the joint attribute index for each input index.
        @note Each output index is paired with an input index returned by getMLJointsInputIndices.
        @param outputIndices
            The source address from which the output indices are to be copied.
        @param count
            The number of output indices to copy.
    */
    virtual void setMLJointsOutputIndices(const std::uint16_t* outputIndices, std::uint16_t count) = 0;
    /**
        @brief Optional metadata that can be assigned to ML joints.
        @note This is used to provide the joint attribute count that the ML model outputs.
        @param parameterKeys
            The source address from which the parameter keys are to be copied.
        @param count
            The number of parameter keys to copy.
    */
    virtual void setMLJointsParameterKeys(const std::uint16_t* parameterKeys, std::uint16_t count) = 0;
    /**
        @brief Optional metadata that can be assigned to ML joints.
        @note This is used to provide the joint attribute count that the ML model outputs.
        @param parameterValues
            The source address from which the parameter values are to be copied.
        @param count
            The number of parameter values to copy.
    */
    virtual void setMLJointsParameterValues(const std::uint16_t* parameterValues, std::uint16_t count) = 0;
};

}  // namespace dna
