// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionReader.h"
#include "dna/layers/MachineLearnedBehavior.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors to the neural network data associated with a rig.
    @warning
        Implementors should inherit from Reader itself and not this class.
*/
class DNAAPI MachineLearnedBehaviorReader : public virtual DefinitionReader {
protected:
    virtual ~MachineLearnedBehaviorReader();

public:
    /**
        @brief Number of ML controls.
    */
    virtual std::uint16_t getMLControlCount() const = 0;
    /**
        @brief Name of the requested ML control.
        @param index
            A name's position in the zero-indexed array of ML control names.
        @warning
            The index must be less than the value returned by getMLControlCount.
        @return View over the control name string.
    */
    virtual StringView getMLControlName(std::uint16_t index) const = 0;
    /**
        @brief Number of neural networks.
    */
    virtual std::uint16_t getNeuralNetworkCount() const = 0;
    /**
        @brief Number of neural network index lists.
        @note
            This value is useful only in the context of MachineLearnedBehaviorWriter.
    */
    virtual std::uint16_t getNeuralNetworkIndexListCount() const = 0;
    /**
        @brief List of neural network indices for the specified LOD.
        @param lod
            The level of detail for which neural networks are being requested.
        @warning
            The lod index must be less than the value returned by getLODCount.
        @return View over the neural network indices.
        @see DescriptorReader::getLODCount
    */
    virtual ConstArrayView<std::uint16_t> getNeuralNetworkIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief Number of mesh regions.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by DefinitionReader::getMeshCount.
    */
    virtual std::uint16_t getMeshRegionCount(std::uint16_t meshIndex) const = 0;
    /**
        @brief Name of the requested region.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by DefinitionReader::getMeshCount.
        @param regionIndex
            A name's position in the zero-indexed array of region names.
        @warning
            The index must be less than the value returned by getMeshRegionCount.
        @return View over the region name string.
    */
    virtual StringView getMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex) const = 0;
    /**
        @brief List of neural network indices for the specified mesh and region.
        @param meshIndex
            The mesh for which neural networks are being requested.
        @warning
            The mesh index must be less than the value returned by getMeshCount.
        @param regionIndex
            The region for which neural networks are being requested.
        @warning
            The region index must be less than the value returned by getMeshRegionCount.
        @return View over the neural network indices.
        @see DefinitionReader::getMeshCount
        @see DefinitionReader::getMeshRegionCount
    */
    virtual ConstArrayView<std::uint16_t> getNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex,
                                                                               std::uint16_t regionIndex) const = 0;
    /**
        @brief List of input control indices that drive the referenced neural network.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by getNeuralNetworkCount.
        @return View over the input control indices.
        @see getNeuralNetworkCount
    */
    virtual ConstArrayView<std::uint16_t> getNeuralNetworkInputIndices(std::uint16_t netIndex) const = 0;
    /**
        @brief List of control indices whose weights are computed by the neural network.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by getNeuralNetworkCount.
        @return View over the control indices.
        @see getNeuralNetworkCount
    */
    virtual ConstArrayView<std::uint16_t> getNeuralNetworkOutputIndices(std::uint16_t netIndex) const = 0;
    /**
        @brief The number of neural network layers (excluding the input layer).
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by getNeuralNetworkCount.
        @see getNeuralNetworkCount
    */
    virtual std::uint16_t getNeuralNetworkLayerCount(std::uint16_t netIndex) const = 0;
    /**
        @brief The activation function type.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by getNeuralNetworkCount.
        @param layerIndex
            A name's position in the zero-indexed array of neural network layers.
        @warning
            The index must be less than the value returned by getNeuralNetworkLayerCount.
        @see getNeuralNetworkCount
        @see getNeuralNetworkLayerCount
    */
    virtual ActivationFunction getNeuralNetworkLayerActivationFunction(std::uint16_t netIndex,
                                                                       std::uint16_t layerIndex) const = 0;
    /**
        @brief Additional activation function parameters (if any).
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by getNeuralNetworkCount.
        @param layerIndex
            A name's position in the zero-indexed array of neural network layers.
        @warning
            The index must be less than the value returned by getNeuralNetworkLayerCount.
        @return View over the activation function parameters (defined in the context of the activation function).
        @see getNeuralNetworkCount
        @see getNeuralNetworkLayerCount
    */
    virtual ConstArrayView<float> getNeuralNetworkLayerActivationFunctionParameters(std::uint16_t netIndex,
                                                                                    std::uint16_t layerIndex) const = 0;
    /**
        @brief Neural network layer bias values.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by getNeuralNetworkCount.
        @param layerIndex
            A name's position in the zero-indexed array of neural network layers.
        @warning
            The index must be less than the value returned by getNeuralNetworkLayerCount.
        @return View over the bias values (for each unit in the layer).
        @see getNeuralNetworkCount
        @see getNeuralNetworkLayerCount
    */
    virtual ConstArrayView<float> getNeuralNetworkLayerBiases(std::uint16_t netIndex, std::uint16_t layerIndex) const = 0;
    /**
        @brief Neural network layer weights.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by getNeuralNetworkCount.
        @param layerIndex
            A name's position in the zero-indexed array of neural network layers.
        @warning
            The index must be less than the value returned by getNeuralNetworkLayerCount.
        @return View over the weight values (a vector for each unit in the layer).
        @see getNeuralNetworkCount
        @see getNeuralNetworkLayerCount
    */
    virtual ConstArrayView<float> getNeuralNetworkLayerWeights(std::uint16_t netIndex, std::uint16_t layerIndex) const = 0;
};

}  // namespace dna
