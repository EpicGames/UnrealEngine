// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionWriter.h"
#include "dna/layers/MachineLearnedBehavior.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors for the neural network data associated with a rig.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI MachineLearnedBehaviorWriter : public virtual DefinitionWriter {
protected:
    virtual ~MachineLearnedBehaviorWriter();

public:
    /**
        @brief Delete all stored ML control names.
    */
    virtual void clearMLControlNames() = 0;
    /**
        @brief Name of the specified ML control.
        @param index
            A name's position in the zero-indexed array of ML control names.
        @note
            The control name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setMLControlName(std::uint16_t index, const char* name) = 0;
    /**
        @brief Delete all neural networks.
    */
    virtual void clearNeuralNetworks() = 0;
    /**
        @brief Delete all stored neural network indices.
    */
    virtual void clearNeuralNetworkIndices() = 0;
    /**
        @brief Store a list of neural network indices onto a specified index.
        @param index
            A position in a zero-indexed array where neural network indices are stored.
        @note
            The index denotes the position of an entire neural network index list,
            not the position of its individual elements, i.e. the row index in a 2D
            matrix of neural network indices.
        @note
            The neural network index storage will be implicitly resized (if needed) to provide storage
            for the number of neural network indices that is inferred from the specified index.
        @param netIndices
            The source address from which the neural network indices are to be copied.
        @note
            These indices can be used to access neural networks through the below defined APIs.
        @param count
            The number of neural network indices to copy.
    */
    virtual void setNeuralNetworkIndices(std::uint16_t index, const std::uint16_t* netIndices, std::uint16_t count) = 0;
    /**
        @brief Delete all stored LOD to neural network list index mapping entries.
    */
    virtual void clearLODNeuralNetworkMappings() = 0;
    /**
        @brief Set which neural networks belong to which level of detail.
        @param lod
            The actual level of detail to which the neural networks are being associated.
        @param index
            The index onto which neural network indices were assigned using setNeuralNetworkIndices.
        @see setNeuralNetworkIndices
    */
    virtual void setLODNeuralNetworkMapping(std::uint16_t lod, std::uint16_t index) = 0;
    /**
        @brief Clear region information for all meshes.
    */
    virtual void clearMeshRegionNames() = 0;
    /**
        @brief Delete all region names for the specified mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @warning
            meshIndex must be less than the value returned by DefinitionReader::getMeshCount.
    */
    virtual void clearMeshRegionNames(std::uint16_t meshIndex) = 0;
    /**
        @brief Name of the specified region.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param regionIndex
            A name's position in the zero-indexed array of region names.
        @note
            The region name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex, const char* name) = 0;
    /**
        @brief Delete all stored neural network indices and accompanying data for each mesh and region.
    */
    virtual void clearNeuralNetworkIndicesPerMeshRegion() = 0;
    /**
        @brief Store a list of neural network indices for the referenced region of a mesh.
        @param meshIndex
            A mesh's position in the zero-indexed array of meshes.
        @param regionIndex
            A region's position in the zero-indexed array of regions.
        @note
            The storage will be implicitly resized (if needed) to provide
            storage for the new data to be stored.
        @param netIndices
            The source address from which the neural network indices are to be copied.
        @note
            These indices can be used to access neural networks through the below defined APIs.
        @param count
            The number of neural network indices to copy.
    */
    virtual void setNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex,
                                                      std::uint16_t regionIndex,
                                                      const std::uint16_t* netIndices,
                                                      std::uint16_t count) = 0;
    /**
        @brief Delete the specified neural network.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by MachineLearnedBehaviorReader::getNeuralNetworkCount.
        @see MachineLearnedBehaviorReader::getNeuralNetworkCount
    */
    virtual void deleteNeuralNetwork(std::uint16_t netIndex) = 0;
    /**
        @brief List of input control indices that drive the referenced neural network.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @param inputIndices
            The source address from which the control indices are to be copied.
        @param count
            The number of control indices to copy.
        @note
            The neural network storage will be implicitly resized (if needed) to provide
            storage for the new data.
    */
    virtual void setNeuralNetworkInputIndices(std::uint16_t netIndex, const std::uint16_t* inputIndices, std::uint16_t count) = 0;
    /**
        @brief List of control indices whose weights are computed by the neural network.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @param outputIndices
            The source address from which the control indices are to be copied.
        @param count
            The number of control indices to copy.
        @note
            The neural network storage will be implicitly resized (if needed) to provide
            storage for the new data.
    */
    virtual void setNeuralNetworkOutputIndices(std::uint16_t netIndex,
                                               const std::uint16_t* outputIndices,
                                               std::uint16_t count) = 0;
    /**
        @brief Delete all layers of the referenced neural network.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @warning
            netIndex must be less than the value returned by MachineLearnedBehaviorReader::getNeuralNetworkCount.
        @see MachineLearnedBehaviorReader::getNeuralNetworkCount
    */
    virtual void clearNeuralNetworkLayers(std::uint16_t netIndex) = 0;
    /**
        @brief Set the activation function type for the specified neural network layer.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @param layerIndex
            A layer's position in the zero-indexed array of neural network layers.
        @param function
            The activation function type.
        @note
            The neural network storage will be implicitly resized (if needed) to provide
            storage for the new data.
    */
    virtual void setNeuralNetworkLayerActivationFunction(std::uint16_t netIndex,
                                                         std::uint16_t layerIndex,
                                                         ActivationFunction function) = 0;
    /**
        @brief Set the additional activation function parameters for the specified neural network layer.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @param layerIndex
            A layer's position in the zero-indexed array of neural network layers.
        @param activationFunctionParameters
            The source address from which the activation function parameters are to be copied.
        @param count
            The number of activation function parameters to copy.
        @note
            The neural network storage will be implicitly resized (if needed) to provide
            storage for the new data.
    */
    virtual void setNeuralNetworkLayerActivationFunctionParameters(std::uint16_t netIndex,
                                                                   std::uint16_t layerIndex,
                                                                   const float* activationFunctionParameters,
                                                                   std::uint16_t count) = 0;
    /**
        @brief Set the bias values for each unit of the specified neural network layer.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @param layerIndex
            A layer's position in the zero-indexed array of neural network layers.
        @param biases
            The source address from which the bias values are to be copied.
        @param count
            The number of bias values to copy.
        @note
            The neural network storage will be implicitly resized (if needed) to provide
            storage for the new data.
    */
    virtual void setNeuralNetworkLayerBiases(std::uint16_t netIndex,
                                             std::uint16_t layerIndex,
                                             const float* biases,
                                             std::uint32_t count) = 0;
    /**
        @brief Set the weight matrix for the specified neural network layer.
        @param netIndex
            A neural network's position in the zero-indexed array of neural networks.
        @param layerIndex
            A layer's position in the zero-indexed array of neural network layers.
        @param weights
            The source address from which the weight matrix values are to be copied.
        @param count
            The number of weight matrix values to copy.
        @note
            The neural network storage will be implicitly resized (if needed) to provide
            storage for the new data.
    */
    virtual void setNeuralNetworkLayerWeights(std::uint16_t netIndex,
                                              std::uint16_t layerIndex,
                                              const float* weights,
                                              std::uint32_t count) = 0;
};

}  // namespace dna
