// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/layers/Twist.h"

namespace dna {

enum class RBFSolverType {
    /** The additive solver sums up contributions from each target. It's faster
        but may require more targets for a good coverage, and requires the
        normalization step to be performed for smooth results.
    */
    Additive,
    /** The interpolative solver interpolates the values from each target based
        on distance. As long as the input values are within the area bounded by
        the targets, the interpolation is well-behaved and returns weight values
        within the 0% - 100% limit with no normalization required.
        Interpolation also gives smoother results, with fewer targets than
        additive solver, but at a higher computational cost.
    */
    Interpolative
};

enum class RBFFunctionType {
    Gaussian,
    Exponential,
    Linear,
    Cubic,
    Quintic,
};

enum class RBFDistanceMethod {
    // Standard n-dimensional distance measure
    Euclidean,
    // Treat inputs as quaternion
    Quaternion,
    // Treat inputs as quaternion, and find distance between rotated TwistAxis direction
    SwingAngle,
    // Treat inputs as half quaternion, and find distance between rotations around the TwistAxis direction
    TwistAngle,
};

enum class RBFNormalizeMethod {
    OnlyNormalizeAboveOne,
    AlwaysNormalize
};

enum class AutomaticRadius {
    On,
    Off
};

}  // namespace dna
