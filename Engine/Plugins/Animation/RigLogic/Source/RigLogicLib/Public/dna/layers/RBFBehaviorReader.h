// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/BehaviorReader.h"
#include "dna/layers/DefinitionReader.h"
#include "dna/layers/RBFBehavior.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors to the RBF data associated with a rig.
    @warning
        Implementors should inherit from Reader itself and not this class.
    @see Reader
*/
class DNAAPI RBFBehaviorReader : public virtual BehaviorReader {
protected:
    virtual ~RBFBehaviorReader();

public:
    virtual std::uint16_t getRBFPoseCount() const = 0;
    /**
        @brief Name of the requested RBF pose.
        @param poseIndex
            A name's position in the zero-indexed array of RBF pose names.
        @warning
            The index must be less than the value returned by getRBFPoseCount.
        @return View over the control name string.
    */
    virtual StringView getRBFPoseName(std::uint16_t poseIndex) const = 0;
    /**
        @brief List of joint output indices of specified pose.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @return View over the array of row indices of raw joint matrix.
        @see BehaviorReader
    */
    virtual ConstArrayView<std::uint16_t> getRBFPoseJointOutputIndices(std::uint16_t poseIndex) const = 0;
    /**
        @brief List of blend shape channel indices of specifed pose.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @return View over the array of blend shape channel indices.
    */
    virtual ConstArrayView<std::uint16_t> getRBFPoseBlendShapeChannelOutputIndices(std::uint16_t poseIndex) const = 0;
    /**
        @brief List of animated map indices of specifed pose.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @return View over the array of animated map indices.
    */
    virtual ConstArrayView<std::uint16_t> getRBFPoseAnimatedMapOutputIndices(std::uint16_t poseIndex) const = 0;
    /**
        @brief Values of requested solver pose outputs.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
    */
    virtual ConstArrayView<float> getRBFPoseJointOutputValues(std::uint16_t poseIndex) const = 0;
    /**
        @brief Scale factor of requested solver's pose.
        @param poseIndex
             A poses's position in the zero-indexed array of poses.
        @warning
            poseIndex must be less than the value returned by getRBFPoseCount
    */
    virtual float getRBFPoseScale(std::uint16_t poseIndex) const = 0;
    /**
        @brief Number of distinct RBF controls.
        @note
            This is the number of unique output control indices for all RBF poses.
        @see getRBFPoseOutputControlIndices
    */
    virtual std::uint16_t getRBFPoseControlCount() const = 0;
    /**
        @brief Name of the requested RBF pose control.
        @param poseControlIndex
            A name's position in the zero-indexed array of RBF pose control names.
        @warning
            The index must be less than the value returned by getRBFPoseControlCount.
        @return View over the control name string.
    */
    virtual StringView getRBFPoseControlName(std::uint16_t poseControlIndex) const = 0;
    /**
        @brief List of control indices that additionally multiply the pose output (optional).
        @note
            When the final output weight of a pose is calculated (this is after it's already
            multiplied by the weights from @getRBFPoseOutputControlWeights), it can optionally be
            additionally multiplied by a list of input controls.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @return View over the array of control indices.
    */
    virtual ConstArrayView<std::uint16_t> getRBFPoseInputControlIndices(std::uint16_t poseIndex) const = 0;
    /**
        @brief List of control indices that the specified pose drives.
        @note
            This is an additional level of indirection, through which a single RBF pose is able to
            drive even multiple expressions (by providing multiple control indices for a pose).
            The solver calculates the output weight for a pose, and that output weight is distributed
            (and multiplied by the multiplier, @see getRBFPoseOutputControlWeights) to all the controls
            that are specified in this array.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @return View over the array of control indices.
    */
    virtual ConstArrayView<std::uint16_t> getRBFPoseOutputControlIndices(std::uint16_t poseIndex) const = 0;
    /**
        @brief List of weights that multiply the RBF outputs.
        @note
            When an RBF solver calculates the weights for each of its poses, those output weights are
            additionally multiplied by this fixed list of multipliers (one weight per control is given).
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @return View over the array of output control multipliers.
    */
    virtual ConstArrayView<float> getRBFPoseOutputControlWeights(std::uint16_t poseIndex) const = 0;
    /**
        @brief Number of RBF solvers.
    */
    virtual std::uint16_t getRBFSolverCount() const = 0;
    /**
        @brief Number of RBF solver index lists.
        @note
            This value is useful only in the context of RBFBehaviorWriter.
    */
    virtual std::uint16_t getRBFSolverIndexListCount() const = 0;
    /**
        @brief List of RBF solver indices for the specified LOD.
        @param lod
            The level of detail for which RBF solvers are being requested.
        @warning
            The lod index must be less than the value returned by getLODCount.
        @return View over the solver indices.
        @see DescriptorReader::getLODCount
    */
    virtual ConstArrayView<std::uint16_t> getRBFSolverIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief Name of the requested RBF solver.
        @param solverIndex
            A name's position in the zero-indexed array of RBF solver names.
        @warning
            The index must be less than the value returned by getRBFSolverCount.
        @return View over the solver name string.
    */
    virtual StringView getRBFSolverName(std::uint16_t solverIndex) const = 0;
    /**
        @brief List of raw body control indices that drive the referenced RBF solver.
            Depending on the distance method @see setRBFSolverDistanceMethod
            different number of inputs are expected.
            Quaternion, SwingAngle and TwistAngle:
                Inputs treated as quaternion, 4 raw controls per input.
                One for each attribute of quaternion (x, y, z, w)
                e.g. two driving joints will require 8 raw controls.
                Raw control indices:
                    11, 12, 13, 14, 18, 19, 20, 21
                Corresponds to:
                    0x, 0y, 0z, 0w, 1x, 1y, 1z, 1w
            Euclidean:
                Inputs treated as scalar, 1 raw control per input. Since inputs
                do not represent joints, they are just treated as arbitrary scalar
                values.
                Raw control indices:
                    33, 34, 41, 42, 56
                Corresponds to:
                    i0, i1, i2, i3, i4
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
            solverIndex must be less than the value returned by getRBFSolverCount.
        @return View over the array of raw control indices
    */
    virtual ConstArrayView<std::uint16_t> getRBFSolverRawControlIndices(std::uint16_t solverIndex) const = 0;
    /**
       @brief Pose indices of specified solver.
       @param solverIndex
           A RBF solver's position in the zero-indexed array of RBF solvers.
       @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
       @return View over the array of pose indices.
    */
    virtual ConstArrayView<std::uint16_t> getRBFSolverPoseIndices(std::uint16_t solverIndex) const = 0;
    /**
        @brief Raw control values for all poses of requested solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
        @return View over the array of the driver values for all poses.
    */
    virtual ConstArrayView<float> getRBFSolverRawControlValues(std::uint16_t solverIndex) const = 0;
    /**
        @brief Type of RBF solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
        @note
            The Interpolative solver boasts smoother blending, whereas the additive solver,
            although needing more targets, provides more precise control over each target's influence.
    */
    virtual RBFSolverType getRBFSolverType(std::uint16_t solverIndex) const = 0;
    /**
       @brief Radius of RBF solver.
       @param solverIndex
           A RBF solver's position in the zero-indexed array of RBF solvers.
       @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
    */
    virtual float getRBFSolverRadius(std::uint16_t solverIndex) const = 0;
    /**
        @brief State of automatic radius switch for specifed RBF solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
    */
    virtual AutomaticRadius getRBFSolverAutomaticRadius(std::uint16_t solverIndex) const = 0;
    /**
        @brief Weight threshold of RBF solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
    */
    virtual float getRBFSolverWeightThreshold(std::uint16_t solverIndex) const = 0;
    /**
        @brief Distance method of RBF solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
    */
    virtual RBFDistanceMethod getRBFSolverDistanceMethod(std::uint16_t solverIndex) const = 0;
    /**
        @brief Normalization method of RBF solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
        @note
            The additive solver requires normalization,
            whereas the Interpolative solver is not as reliant on it.
    */
    virtual RBFNormalizeMethod getRBFSolverNormalizeMethod(std::uint16_t solverIndex) const = 0;
    /**
        @brief Function type of RBF solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
    */
    virtual RBFFunctionType getRBFSolverFunctionType(std::uint16_t solverIndex) const = 0;
    /**
        @brief Twist axis of specified RBF solver when DistanceMethod is SwingAngle.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @warning
           solverIndex must be less than the value returned by getRBFSolverCount.
    */
    virtual TwistAxis getRBFSolverTwistAxis(std::uint16_t solverIndex) const = 0;
};

}  // namespace dna
