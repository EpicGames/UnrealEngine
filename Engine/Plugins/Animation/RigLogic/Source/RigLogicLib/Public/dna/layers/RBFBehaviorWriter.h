// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/BehaviorWriter.h"
#include "dna/layers/DefinitionWriter.h"
#include "dna/layers/RBFBehavior.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors for the RBF data associated with a rig.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI RBFBehaviorWriter : public virtual BehaviorWriter {
protected:
    virtual ~RBFBehaviorWriter();

public:
    /**
        @brief Delete all RBF poses.
    */
    virtual void clearRBFPoses() = 0;
    /**
        @brief Set name of pose.
        @param poseIndex
           A RBF pose's position in the zero-indexed array of poses.
        @note
            The pose storage will be implicitly resized (if needed) to provide storage
            for the number of poses that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setRBFPoseName(std::uint16_t poseIndex, const char* name) = 0;
    /**
        @brief Set scale factor for the specified pose of solver.
        @param poseIndex
           A RBF pose's position in the zero-indexed array of poses.
        @note
            The pose storage will be implicitly resized (if needed) to provide storage
            for the number of poses that is inferred from the specified index.
        @param scale
            The solver's pose scale.
    */
    virtual void setRBFPoseScale(std::uint16_t poseIndex, float scale) = 0;
    /**
        @brief Delete all RBF pose control names.
    */
    virtual void clearRBFPoseControlNames() = 0;
    /**
        @brief Name of the specified RBF pose control.
        @param index
            A name's position in the zero-indexed array of RBF pose control names.
        @note
            The control name storage will be implicitly resized (if needed) to provide
            storage for the number of names that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setRBFPoseControlName(std::uint16_t poseControlIndex, const char* name) = 0;
    /**
        @brief Set the list of control indices that additionally multiply the pose output (optional).
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @param controlIndices
            The source address from which the control indices are to be copied.
        @note
            When the final output weight of a pose is calculated (this is after it's already
            multiplied by the weights from @getRBFPoseOutputControlWeights), it can optionally be
            additionally multiplied by a list of input controls.
        @param controlCount
            The number of control indices to copy.
    */
    virtual void setRBFPoseInputControlIndices(std::uint16_t poseIndex,
                                               const std::uint16_t* controlIndices,
                                               std::uint16_t controlIndexCount) = 0;
    /**
        @brief Set the list of control indices that the specified pose drives.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @param controlIndices
            The source address from which the control indices are to be copied.
        @note
            These control indices
            This is an additional level of indirection, through which a single RBF pose is able to
            drive even multiple expressions (by providing multiple control indices for a pose).
            The solver calculates the output weight for a pose, and that output weight is distributed
            (and multiplied by the multiplier, @see setRBFPoseOutputControlWeights) to all the controls
            that are specified in this array.
        @param controlCount
            The number of control indices to copy.
    */
    virtual void setRBFPoseOutputControlIndices(std::uint16_t poseIndex,
                                                const std::uint16_t* controlIndices,
                                                std::uint16_t controlIndexCount) = 0;
    /**
        @brief Set the list of weights that multiply the RBF outputs.
        @param poseIndex
            A poses's position in the zero-indexed array of poses.
        @warning
           poseIndex must be less than the value returned by getRBFPoseCount
        @param controlWeights
            The source address from which the control weights are to be copied.
        @note
            When an RBF solver calculates the weights for each of its poses, those output weights are
            additionally multiplied by this fixed list of multipliers (one weight per control is given).
        @param controlWeightCount
            The number of control weights to copy.
    */
    virtual void setRBFPoseOutputControlWeights(std::uint16_t poseIndex,
                                                const float* controlWeights,
                                                std::uint16_t controlWeightCount) = 0;
    /**
        @brief Delete all RBF solvers.
    */
    virtual void clearRBFSolvers() = 0;
    /**
        @brief Delete all stored RBF solver indices.
    */
    virtual void clearRBFSolverIndices() = 0;
    /**
        @brief Store a list of RBF solver indices onto a specified index.
        @param index
            A position in a zero-indexed array where solver indices are stored.
        @note
            The index denotes the position of an entire solver index list,
            not the position of its individual elements, i.e. the row index in a 2D
            matrix of solver indices.
        @note
            The solver index storage will be implicitly resized (if needed) to provide storage
            for the number of solver indices that is inferred from the specified index.
        @param solverIndices
            The source address from which the solver indices are to be copied.
        @note
            These indices can be used to access RBF solver through the below defined APIs.
        @param count
            The number of solver indices to copy.
    */
    virtual void setRBFSolverIndices(std::uint16_t index, const std::uint16_t* solverIndices, std::uint16_t count) = 0;
    /**
        @brief Delete all stored LOD to RBF solver list index mapping entries.
    */
    virtual void clearLODRBFSolverMappings() = 0;
    /**
        @brief Set which solver belongs to which level of detail.
        @param lod
            The actual level of detail to which the solvers are being associated.
        @param index
            The index onto which solver indices were assigned using setRBFSolverIndices.
        @see setRBFSolverIndices
    */
    virtual void setLODRBFSolverMapping(std::uint16_t lod, std::uint16_t index) = 0;
    /**
        @brief Name of the specified RBF solver.
        @param solverIndex
            A name's position in the zero-indexed array of RBF solvers.
        @note
            The RBF solver storage will be implicitly resized (if needed) to provide
            storage for the number of solvers that is inferred from the specified index.
        @param name
            A null-terminated string.
        @note
            The passed in name is copied, which will involve an additional allocation.
    */
    virtual void setRBFSolverName(std::uint16_t solverIndex, const char* name) = 0;
    /**
        @brief Store a list of raw control indices of the specified RBF solver.
            Depending on the distance method @see setRBFSolverDistanceMethod
            different number of inputs are expected.
            Quaternion, SwingAngle and TwistAngle:
                Inputs treated as quaternion, 4 raw controls per input.
                One for each attribute of quaternion (x, y, z, w)
                e.g. two driving joints will require 8 raw controls.
                Raw control indices:
                    11, 12, 13, 14, 18, 19, 20, 21
                Corresponds to:
                    0x, 0y, 0z, 0w, 1x, 1y, 1z, 1w
            Euclidean:
                Inputs treated as scalar. 1 raw control per input. Since inputs
                do not represent joints, they are just treated as arbitrary scalar
                values.
                Raw control indices:
                    33, 34, 41, 42, 56
                Corresponds to:
                    i0, i1, i2, i3, i4
        @param solverIndex
            A position in a zero-indexed array where solvers are stored.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param rawControlIndices
            The source address from which the raw control indices are to be copied.
        @param count
             The number of raw control indices to copy.
    */
    virtual void setRBFSolverRawControlIndices(std::uint16_t solverIndex,
                                               const std::uint16_t* rawControlIndices,
                                               std::uint16_t count) = 0;
    /**
        @brief Store a list of pose indices of specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
    */
    virtual void setRBFSolverPoseIndices(std::uint16_t solverIndex, const std::uint16_t* poseIndices, std::uint16_t count) = 0;
    /**
        @brief Set raw control values for all poses of requested solver, value for each raw control per pose.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param values
            The source address from which the values are to be copied.
        @param count
            The number of values to copy.
    */
    virtual void setRBFSolverRawControlValues(std::uint16_t solverIndex, const float* values, std::uint16_t count) = 0;
    /**
        @brief Set solver type for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param type
            The solver type.
    */
    virtual void setRBFSolverType(std::uint16_t solverIndex, RBFSolverType type) = 0;
    /**
        @brief Set radius for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param radius
            The solver's radius.
    */
    virtual void setRBFSolverRadius(std::uint16_t solverIndex, float radius) = 0;
    /**
        @brief Set automatic radius for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param automaticRadius
            The solver's automatic radius.
    */
    virtual void setRBFSolverAutomaticRadius(std::uint16_t solverIndex, AutomaticRadius automaticRadius) = 0;
    /**
        @brief Set weight threshold for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param weightThreshold
            The solver's weight threshold.
    */
    virtual void setRBFSolverWeightThreshold(std::uint16_t solverIndex, float weightThreshold) = 0;
    /**
        @brief Set distance method for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param distanceMethod
            The solver's distance method.
    */
    virtual void setRBFSolverDistanceMethod(std::uint16_t solverIndex, RBFDistanceMethod distanceMethod) = 0;
    /**
        @brief Set normalization method for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param normalizeMethod
            The solver's normalization method.
    */
    virtual void setRBFSolverNormalizeMethod(std::uint16_t solverIndex, RBFNormalizeMethod normalizeMethod) = 0;
    /**
        @brief Set function type for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param functionType
            The solver's function type.
    */
    virtual void setRBFSolverFunctionType(std::uint16_t solverIndex, RBFFunctionType functionType) = 0;
    /**
        @brief Set twist axis for the specified solver.
        @param solverIndex
            A RBF solver's position in the zero-indexed array of RBF solvers.
        @note
            The solver storage will be implicitly resized (if needed) to provide storage
            for the number of solvers that is inferred from the specified index.
        @param TwistAxis
            The solver's twist axis.
    */
    virtual void setRBFSolverTwistAxis(std::uint16_t solverIndex, TwistAxis twistAxis) = 0;
};

}  // namespace dna
