// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionReader.h"
#include "dna/layers/Twist.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Read-only accessors to the Swing and Twist data associated with a rig.
    @warning
        Implementors should inherit from Reader itself and not this class.
    @see Reader
*/
class DNAAPI TwistSwingBehaviorReader : public virtual DefinitionReader {
protected:
    virtual ~TwistSwingBehaviorReader();

public:
    /**
        @brief The number of twist parameter groups.
    */
    virtual std::uint16_t getTwistCount() const = 0;
    /**
        @brief Twist axis of the specified twist parameter group.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getTwistCount.
    */
    virtual TwistAxis getTwistSetupTwistAxis(std::uint16_t twistIndex) const = 0;
    /**
        @brief Get the raw control indices representing the joint rotation that drives the twist transformation.
        @note
            The number of these raw control indices is always 4, representing a single quaternion [x, y, z, w] attributes.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getTwistCount.
        @return View over the raw control indices representing the twist driver joint rotation.
    */
    virtual ConstArrayView<std::uint16_t> getTwistInputControlIndices(std::uint16_t twistIndex) const = 0;
    /**
        @brief Get the joint indices whose rotations are driven by the twist transformation.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getTwistCount.
        @return View over the twist driven joint indices.
    */
    virtual ConstArrayView<std::uint16_t> getTwistOutputJointIndices(std::uint16_t twistIndex) const = 0;
    /**
        @brief Get the weights that define which portions of the driver joint rotation will be applied to the driven joints.
        @note
            There is one weight per output joint in this list.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getTwistCount.
        @return View over the twist blend weights.
    */
    virtual ConstArrayView<float> getTwistBlendWeights(std::uint16_t twistIndex) const = 0;
    /**
        @brief The number of swing parameter groups.
    */
    virtual std::uint16_t getSwingCount() const = 0;
    /**
        @brief Twist axis of the specified swing parameter group.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getSwingCount.
    */
    virtual TwistAxis getSwingSetupTwistAxis(std::uint16_t swingIndex) const = 0;
    /**
        @brief Get the raw control indices representing the joint rotation that drives the swing transformation.
        @note
            The number of these raw control indices is always 4, representing a single quaternion [x, y, z, w] attributes.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getSwingCount.
        @return View over the raw control indices representing the swing driver joint rotation.
    */
    virtual ConstArrayView<std::uint16_t> getSwingInputControlIndices(std::uint16_t swingIndex) const = 0;
    /**
        @brief Get the joint indices whose rotations are driven by the swing transformation.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getSwingCount.
        @return View over the swing driven joint indices.
    */
    virtual ConstArrayView<std::uint16_t> getSwingOutputJointIndices(std::uint16_t swingIndex) const = 0;
    /**
        @brief Get the weights that define which portions of the driver joint rotation will be applied to the driven joints.
        @note
            There is one weight per output joint in this list.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @warning
            The index must be less than the value returned by getSwingCount.
        @return View over the swing blend weights.
    */
    virtual ConstArrayView<float> getSwingBlendWeights(std::uint16_t swingIndex) const = 0;
};

}  // namespace dna
