// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/layers/DefinitionWriter.h"
#include "dna/layers/Twist.h"
#include "dna/types/Aliases.h"

#include <cstdint>

namespace dna {

/**
    @brief Write-only accessors to the Swing and Twist data associated with a rig.
    @warning
        Implementors should inherit from Writer itself and not this class.
    @see Writer
*/
class DNAAPI TwistSwingBehaviorWriter : public virtual DefinitionWriter {
protected:
    virtual ~TwistSwingBehaviorWriter();

public:
    /**
        @brief Delete all twist parameter groups.
    */
    virtual void clearTwists() = 0;
    /**
        @brief Delete the specified twist parameter group.
        @param twistIndex
            The index of the parameter group to be deleted.
    */
    virtual void deleteTwist(std::uint16_t twistIndex) = 0;
    /**
        @brief Set twist axis for the specified twist parameter group.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param TwistAxis
            The twist axis of the parameter group.
    */
    virtual void setTwistSetupTwistAxis(std::uint16_t twistIndex, TwistAxis twistAxis) = 0;
    /**
        @brief Set the raw control indices representing the joint rotation that drives the twist transformation.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param controlIndices
            Pointer to raw control indices representing the joint rotation that drives the twist transformation.
        @note
            The number of these raw control indices is always 4, representing a single quaternion [x, y, z, w] attributes.
        @param controlIndexCount
            The number of raw control indices to copy.
    */
    virtual void setTwistInputControlIndices(std::uint16_t twistIndex,
                                             const std::uint16_t* controlIndices,
                                             std::uint16_t controlIndexCount) = 0;
    /**
        @brief Set the joint indices whose rotations are driven by the twist transformation.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param jointIndices
            Pointer to joint indices of the twist driven joints.
        @param jointIndexCount
            The number of joint indices to copy.
    */
    virtual void setTwistOutputJointIndices(std::uint16_t twistIndex,
                                            const std::uint16_t* jointIndices,
                                            std::uint16_t jointIndexCount) = 0;
    /**
        @brief Set the weights that define which portions of the driver joint rotation will be applied to the driven joints.
        @param twistIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param blendWeights
            The source address from which the twist blend weights are to be copied.
        @note
            There is one weight per output joint in this list.
        @param blendWeightCount
            The number of blend weights to copy.
    */
    virtual void setTwistBlendWeights(std::uint16_t twistIndex, const float* blendWeights, std::uint16_t blendWeightCount) = 0;
    /**
        @brief Delete all swing parameter groups.
    */
    virtual void clearSwings() = 0;
    /**
        @brief Delete the specified swing parameter group.
        @param swingIndex
            The index of the parameter group to be deleted.
    */
    virtual void deleteSwing(std::uint16_t swingIndex) = 0;
    /**
        @brief Set twist axis for the specified swing parameter group.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param TwistAxis
            The twist axis of the parameter group.
    */
    virtual void setSwingSetupTwistAxis(std::uint16_t swingIndex, TwistAxis twistAxis) = 0;
    /**
        @brief Set the raw control indices representing the joint rotation that drives the swing transformation.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param controlIndices
            View over the raw control indices representing the swing driver joint rotation.
        @note
            The number of these raw control indices is always 4, representing a single quaternion [x, y, z, w] attributes.
        @param controlIndexCount
            The number of raw control indices to copy.
    */
    virtual void setSwingInputControlIndices(std::uint16_t swingIndex,
                                             const std::uint16_t* controlIndices,
                                             std::uint16_t controlIndexCount) = 0;
    /**
        @brief Set the joint indices whose rotations are driven by the swing transformation.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param jointIndices
            Pointer to joint indices of the swing driven joints.
        @param jointIndexCount
            The number of joint indices to copy.
    */
    virtual void setSwingOutputJointIndices(std::uint16_t swingIndex,
                                            const std::uint16_t* jointIndices,
                                            std::uint16_t jointIndexCount) = 0;
    /**
        @brief Set the weights that define which portions of the driver joint rotation will be applied to the driven joints.
        @param swingIndex
            The index of the parameter group whose data is being accessed.
        @note
            The parameter group storage will be implicitly resized (if needed) to provide
            storage for the number of groups that is inferred from the specified index.
        @param blendWeights
            The source address from which the swing blend weights are to be copied.
        @note
            There is one weight per output joint in this list.
        @param blendWeightCount
            The number of blend weights to copy.
    */
    virtual void setSwingBlendWeights(std::uint16_t swingIndex, const float* blendWeights, std::uint16_t blendWeightCount) = 0;
};

}  // namespace dna
