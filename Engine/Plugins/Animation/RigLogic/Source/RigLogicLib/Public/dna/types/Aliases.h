// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <arrayview/ArrayView.h>
#include <arrayview/StringView.h>
#include <pma/MemoryResource.h>
#include <pma/ScopedPtr.h>
#include <status/Status.h>
#include <status/StatusCode.h>
#include <tdm/TDM.h>
#include <trio/Stream.h>
#include <trio/streams/FileStream.h>
#include <trio/streams/MemoryMappedFileStream.h>
#include <trio/streams/MemoryStream.h>

namespace dna {

using sc::Status;
using trio::BoundedIOStream;
using trio::FileStream;
using trio::MemoryMappedFileStream;
using trio::MemoryStream;

using namespace av;
using namespace pma;

}  // namespace dna
