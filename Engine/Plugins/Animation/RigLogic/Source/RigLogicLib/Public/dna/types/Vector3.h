// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace dna {

struct Vector3 {
    float x;
    float y;
    float z;

    Vector3& operator+=(const Vector3& rhs) {
        x += rhs.x;
        y += rhs.y;
        z += rhs.z;
        return *this;
    }

    Vector3& operator-=(const Vector3& rhs) {
        x -= rhs.x;
        y -= rhs.y;
        z -= rhs.z;
        return *this;
    }

    Vector3& operator*=(const Vector3& rhs) {
        x *= rhs.x;
        y *= rhs.y;
        z *= rhs.z;
        return *this;
    }

    Vector3& operator/=(const Vector3& rhs) {
        x /= rhs.x;
        y /= rhs.y;
        z /= rhs.z;
        return *this;
    }

    Vector3& operator+=(float rhs) {
        x += rhs;
        y += rhs;
        z += rhs;
        return *this;
    }

    Vector3& operator-=(float rhs) {
        x -= rhs;
        y -= rhs;
        z -= rhs;
        return *this;
    }

    Vector3& operator*=(float rhs) {
        x *= rhs;
        y *= rhs;
        z *= rhs;
        return *this;
    }

    Vector3& operator/=(float rhs) {
        x /= rhs;
        y /= rhs;
        z /= rhs;
        return *this;
    }
};

inline Vector3 operator+(Vector3 lhs, const Vector3& rhs) {
    return (lhs += rhs);
}

inline Vector3 operator-(Vector3 lhs, const Vector3& rhs) {
    return (lhs -= rhs);
}

inline Vector3 operator*(Vector3 lhs, const Vector3& rhs) {
    return (lhs *= rhs);
}

inline Vector3 operator/(Vector3 lhs, const Vector3& rhs) {
    return (lhs /= rhs);
}

inline Vector3 operator+(Vector3 lhs, float rhs) {
    return (lhs += rhs);
}

inline Vector3 operator-(Vector3 lhs, float rhs) {
    return (lhs -= rhs);
}

inline Vector3 operator*(Vector3 lhs, float rhs) {
    return (lhs *= rhs);
}

inline Vector3 operator/(Vector3 lhs, float rhs) {
    return (lhs /= rhs);
}

inline bool operator==(const Vector3& lhs, const Vector3& rhs) {
    return (lhs.x == rhs.x && lhs.y == rhs.y && lhs.z == rhs.z);
}

inline bool operator!=(const Vector3& lhs, const Vector3& rhs) {
    return !(lhs == rhs);
}

}  // namespace dna
