// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define DNA_MAJOR_VERSION 10
#define DNA_MINOR_VERSION 1
#define DNA_PATCH_VERSION 3
#define DNA_VERSION_STRING "10.1.3"
