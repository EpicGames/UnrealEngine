// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dna/Defs.h"
#include "dna/types/Aliases.h"

namespace dna {

struct DNAAPI VersionInfo {
    static int getMajorVersion();
    static int getMinorVersion();
    static int getPatchVersion();
    static StringView getVersionString();
};

}  // namespace dna
