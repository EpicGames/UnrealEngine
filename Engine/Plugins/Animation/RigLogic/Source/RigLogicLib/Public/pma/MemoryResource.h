// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "pma/Defs.h"

#include <cstddef>

namespace pma {

/**
    @brief MemoryResource is an abstract class that allows the implementation of polymorphic allocators.
    @note
        It's purpose is to allow passing arbitrary allocators through API boundaries, without requiring changes in the
        signatures and types involved.
*/
class PMAAPI MemoryResource {
public:
    virtual ~MemoryResource();
    virtual void* allocate(std::size_t size, std::size_t alignment) = 0;
    virtual void deallocate(void* ptr, std::size_t size, std::size_t alignment) = 0;
};

}  // namespace pma
