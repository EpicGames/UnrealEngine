// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "pma/MemoryResource.h"
#include "pma/resources/DefaultMemoryResource.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cassert>
#include <cstddef>
#include <memory>
#include <scoped_allocator>
#include <type_traits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
    #pragma warning(disable : 4068)
#endif

namespace pma {

namespace impl {

using min_align_t = void*;

template<typename T, typename U>
struct max_align_of {
    using type = typename std::conditional<(alignof(T) > alignof(U)), T, U>::type;
};

template<typename T, std::size_t Alignment, class TDefaultMemoryResource>
class PolyAllocator {
public:
    using value_type = T;
    using traits_type = std::allocator_traits<PolyAllocator>;

    template<typename U, std::size_t UAlignment, class UDefaultMemoryResource>
    friend class PolyAllocator;

public:
    PolyAllocator() {
#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif
        static TDefaultMemoryResource dmr;
#ifdef __clang__
    #pragma clang diagnostic pop
#endif
        pMemRes = &dmr;
    }

    PolyAllocator(MemoryResource* memRes) :
        PolyAllocator{} {
        if (memRes != nullptr) {
            pMemRes = memRes;
        }
    }

    PolyAllocator(std::nullptr_t /*unused*/) :
        PolyAllocator{} {
    }

    template<class U, std::size_t UAlignment, class UDefaultMemoryResource>
    PolyAllocator(const PolyAllocator<U, UAlignment, UDefaultMemoryResource>& rhs) :
        pMemRes{rhs.pMemRes} {
    }

    void* allocateBytes(std::size_t size, std::size_t alignment = Alignment) {
        return pMemRes->allocate(size, alignment);
    }

    void deallocateBytes(void* ptr, std::size_t size, std::size_t alignment = Alignment) {
        pMemRes->deallocate(ptr, size, alignment);
    }

    template<typename U = value_type>
    typename std::enable_if<!std::is_void<U>::value, U*>::type allocateObject(std::size_t count,
                                                                              std::size_t alignment = Alignment) {
        return static_cast<U*>(allocateBytes(count * sizeof(U), alignment));
    }

    template<typename U = value_type>
    typename std::enable_if<!std::is_void<U>::value>::type deallocateObject(U* ptr,
                                                                            std::size_t count,
                                                                            std::size_t alignment = Alignment) {
        deallocateBytes(static_cast<void*>(ptr), count * sizeof(U), alignment);
    }

    template<typename U = value_type, typename... Args>
    U* newObject(Args&&... args) {
        auto ptr = traits_type::allocate(*this, 1ul);
        assert(ptr != nullptr);
        traits_type::construct(*this, ptr, std::forward<Args>(args)...);
        return ptr;
    }

    template<typename U = value_type>
    void deleteObject(U* ptr) {
        traits_type::destroy(*this, ptr);
        traits_type::deallocate(*this, ptr, 1ul);
    }

    // Allocation function as requested by standard-library containers
    value_type* allocate(std::size_t count) {
        return allocateObject(count);
    }

    // Deallocation function as requested by standard-library containers
    void deallocate(value_type* ptr, std::size_t count) {
        deallocateObject(ptr, count);
    }

    static std::size_t getAlignment() {
        return Alignment;
    }

    MemoryResource* getMemoryResource() const {
        return pMemRes;
    }

private:
    MemoryResource* pMemRes;
};

}  // namespace impl

template<typename T,
         std::size_t Alignment = alignof(typename impl::max_align_of<T, impl::min_align_t>::type),
         class TDefaultMemoryResource = DefaultMemoryResource>
class PolyAllocator : public std::scoped_allocator_adaptor<impl::PolyAllocator<T, Alignment, TDefaultMemoryResource>> {
private:
    using Impl = impl::PolyAllocator<T, Alignment, TDefaultMemoryResource>;
    using Base = std::scoped_allocator_adaptor<Impl>;

public:
    template<typename U>
    struct rebind {
        using other = PolyAllocator<U, Alignment, TDefaultMemoryResource>;
    };

    PolyAllocator() = default;

    PolyAllocator(MemoryResource* memRes) :
        Base{Impl{memRes}} {
    }

    template<class U, std::size_t UAlignment, class UDefaultMemoryResource>
    PolyAllocator(const PolyAllocator<U, UAlignment, UDefaultMemoryResource>& rhs) :
        Base{rhs} {
    }

    template<class U, std::size_t UAlignment, class UDefaultMemoryResource>
    PolyAllocator(const impl::PolyAllocator<U, UAlignment, UDefaultMemoryResource>& rhs) :
        Base{rhs} {
    }
};

template<typename T,
         std::size_t TAlignment,
         class TDefaultMemoryResource,
         typename U,
         std::size_t UAlignment,
         class UDefaultMemoryResource>
bool operator==(const PolyAllocator<T, TAlignment, TDefaultMemoryResource>& lhs,
                const PolyAllocator<U, UAlignment, UDefaultMemoryResource>& rhs) {
    return (TAlignment == UAlignment && lhs.getMemoryResource() == rhs.getMemoryResource());
}

template<typename T,
         std::size_t TAlignment,
         class TDefaultMemoryResource,
         typename U,
         std::size_t UAlignment,
         class UDefaultMemoryResource>
bool operator!=(const PolyAllocator<T, TAlignment, TDefaultMemoryResource>& lhs,
                const PolyAllocator<U, UAlignment, UDefaultMemoryResource>& rhs) {
    return !(lhs == rhs);
}

}  // namespace pma
