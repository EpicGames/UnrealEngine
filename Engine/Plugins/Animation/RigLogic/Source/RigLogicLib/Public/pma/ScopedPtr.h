// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace pma {

template<class T, class B = T>
struct New {
    template<typename... Args>
    B* operator()(Args&&... args) {
        return new T{std::forward<Args>(args)...};
    }
};

template<class T, class B = T>
struct Delete {
    void operator()(B* ptr) {
// Calling delete on an incomplete type is undefined behavior.
// This check will result in a compile error for incomplete types, rather than allow UB.
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wsign-conversion"
#endif
        using complete_type_checker = char[sizeof(T) ? 1 : -1];
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        static_cast<void>(sizeof(complete_type_checker));
        delete ptr;
    }
};

template<class T>
struct New<T[]> {
    T* operator()(std::size_t size) {
        return new T[size]{};
    }
};

template<class T>
struct Delete<T[]> {
    void operator()(T* ptr) {
// Calling delete on an incomplete type is undefined behavior.
// This check will result in a compile error for incomplete types, rather than allow UB.
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wsign-conversion"
#endif
        using complete_type_checker = char[sizeof(T) ? 1 : -1];
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        static_cast<void>(sizeof(complete_type_checker));
        delete[] ptr;
    }
};

template<class T, class B = T>
struct FactoryCreate {
    template<typename... Args>
    B* operator()(Args&&... args) {
        return T::create(std::forward<Args>(args)...);
    }
};

template<class T, class B = T>
struct FactoryDestroy {
    void operator()(B* ptr) {
        T::destroy(static_cast<T*>(ptr));
    }
};

template<class T>
struct DefaultInstanceCreator {
    using type = New<T>;
};

template<class T>
struct DefaultInstanceDestroyer {
    using type = Delete<T>;
};

/**
    @brief Takes ownership over the given pointer and handles it's lifetime.
    @note
        As ScopedPtr inherits the specified destroyer type, stateless lifetime
        managers are zero-cost, but it's also possible to use stateful lifetime
        managers (such as lambdas with captures and what-not).
        For stateful lifetime managers, a dedicated constructor exists that
        receives the destroyer instance and initializes the inherited destroyer
        type with it.
    @see makeScoped
    @see New
    @see Delete
    @see FactoryCreate
    @see FactoryDestroy
    @see DefaultInstanceCreator
    @see DefaultInstanceDestroyer
*/
template<class T, class TDestroyer = typename DefaultInstanceDestroyer<T>::type>
class ScopedPtr : private TDestroyer {
private:
    template<typename U>
    struct inspect {
        using element_type = U;
        using pointer_type = element_type*;
        using is_array = std::false_type;
    };

    template<typename U>
    struct inspect<U[]> {
        using element_type = U;
        using pointer_type = element_type*;
        using is_array = std::true_type;
    };

public:
    using pointer = typename inspect<T>::pointer_type;
    using element_type = typename inspect<T>::element_type;
    using destroyer_type = TDestroyer;

    template<typename U, class UDestroyer>
    friend class ScopedPtr;

public:
    ScopedPtr() :
        ptr{nullptr} {
    }

    explicit ScopedPtr(pointer ptr_) :
        ptr{ptr_} {
    }

    ScopedPtr(pointer ptr_, destroyer_type&& destroyer) :
        destroyer_type{std::move(destroyer)},
        ptr{ptr_} {
    }

    ~ScopedPtr() {
        if (ptr) {
            destroyer_type::operator()(ptr);
            ptr = pointer{};
        }
    }

    ScopedPtr(std::nullptr_t) :
        ptr{nullptr} {
    }

    ScopedPtr& operator=(std::nullptr_t) {
        reset();
        return *this;
    }

    ScopedPtr(const ScopedPtr&) = delete;
    ScopedPtr& operator=(const ScopedPtr&) = delete;

    ScopedPtr(ScopedPtr&& rhs) noexcept :
        ptr{nullptr} {
        rhs.swap(*this);
    }

    ScopedPtr& operator=(ScopedPtr&& rhs) noexcept {
        rhs.swap(*this);
        return *this;
    }

    template<typename U, class UDestroyer>
    ScopedPtr(ScopedPtr<U, UDestroyer>&& rhs) noexcept :
        ptr{nullptr} {
        ScopedPtr<T, destroyer_type> tmp{rhs.release(), static_cast<UDestroyer&&>(rhs)};
        tmp.swap(*this);
    }

    template<typename U, class UDestroyer>
    ScopedPtr& operator=(ScopedPtr<U, UDestroyer>&& rhs) noexcept {
        ScopedPtr<T, destroyer_type> tmp{rhs.release(), static_cast<UDestroyer&&>(rhs)};
        tmp.swap(*this);
        return *this;
    }

    template<typename U = T, typename IA = typename inspect<U>::is_array>
    typename std::enable_if<IA::value, element_type&>::type operator[](std::size_t index) const noexcept {
        return ptr[index];
    }

    template<typename U = T, typename IA = typename inspect<U>::is_array>
    typename std::enable_if<!IA::value, element_type&>::type operator*() const noexcept {
        return *ptr;
    }

    pointer operator->() const noexcept {
        return ptr;
    }

    operator bool() const noexcept {
        return ptr != nullptr;
    }

    pointer get() const noexcept {
        return ptr;
    }

    pointer release() noexcept {
        pointer result = nullptr;
        std::swap(result, ptr);
        return result;
    }

    void reset(pointer rhs = pointer()) noexcept {
        pointer old = release();
        ptr = rhs;
        if (old) {
            destroyer_type::operator()(old);
        }
    }

    void swap(ScopedPtr& rhs) noexcept {
        std::swap(static_cast<destroyer_type&>(*this), static_cast<destroyer_type&>(rhs));
        std::swap(ptr, rhs.ptr);
    }

private:
    pointer ptr;
};

/**
    @brief Syntactic sugar for creating instances wrapped in a ScopedPtr.
    @note
        The default behavior is to rely on the New / Delete pair of lifetime
        managers, because it's sensible to do so.
        However, because a significant portion of our abstractions follow
        the convention of exposing a create / destroy pair of factory functions
        (where create always returns a raw pointer), there also exists a dedicated
        FactoryCreate / FactoryDestroy pair of lifetime managers.
        To change the default behavior in order to utilize a specific lifetime
        manager pair, specialize the DefaultInstanceCreator and DefaultInstanceDestroyer
        traits for the types that need different handling.
        Alternately, it's also possible to pass a custom creator / destroyer on each
        invocation.
 */
template<class T,
         class TCreator,
         class TDestroyer,
         typename... Args,
         typename Base = typename std::remove_pointer<decltype(TCreator{}(std::declval<Args>()...))>::type>
ScopedPtr<Base, TDestroyer> makeScoped(Args&&... args) {
    static_assert(std::is_same<Base, T>::value || std::is_base_of<Base, T>::value ||
                      std::is_convertible<T, typename std::add_pointer<Base>::type>::value,
                  "Incompatible types.");
    return ScopedPtr<Base, TDestroyer>{TCreator{}(std::forward<Args>(args)...)};
}

template<class T, template<class...> class TCreatorTemplate, template<class...> class TDestroyerTemplate, typename... Args>
ScopedPtr<T, TDestroyerTemplate<T>> makeScoped(Args&&... args) {
    using TCreator = TCreatorTemplate<T>;
    using TDestroyer = TDestroyerTemplate<T>;
    return makeScoped<T, TCreator, TDestroyer>(std::forward<Args>(args)...);
}

template<class T, typename... Args>
ScopedPtr<T, typename DefaultInstanceDestroyer<T>::type> makeScoped(Args&&... args) {
    using TCreator = typename DefaultInstanceCreator<T>::type;
    using TDestroyer = typename DefaultInstanceDestroyer<T>::type;
    return makeScoped<T, TCreator, TDestroyer>(std::forward<Args>(args)...);
}

}  // namespace pma
