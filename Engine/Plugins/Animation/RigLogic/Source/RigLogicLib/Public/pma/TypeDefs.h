// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "pma/PolyAllocator.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365)
#endif
#include <cstddef>
#include <list>
#include <map>
#include <set>
#include <string>
#include <unordered_map>
#include <unordered_set>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace pma {

template<typename T, typename Allocator = PolyAllocator<T>>
using String = std::basic_string<T, std::char_traits<T>, Allocator>;

template<typename T, typename Allocator = PolyAllocator<T>>
using Vector = std::vector<T, Allocator>;

template<typename T, typename Allocator = PolyAllocator<Vector<T>>>
using Matrix = Vector<Vector<T>, Allocator>;

template<typename T, typename Allocator = PolyAllocator<T>>
using List = std::list<T, Allocator>;

template<typename T, typename Allocator = PolyAllocator<T>>
using Set = std::set<T, std::less<T>, Allocator>;

template<typename T, typename Allocator = PolyAllocator<T>>
using UnorderedSet = std::unordered_set<T, std::hash<T>, std::equal_to<T>, Allocator>;

template<typename K, typename V, typename Allocator = PolyAllocator<std::pair<const K, V>>>
using Map = std::map<K, V, std::less<K>, Allocator>;

template<typename K, typename V, typename Allocator = PolyAllocator<std::pair<const K, V>>>
using UnorderedMap = std::unordered_map<K, V, std::hash<K>, std::equal_to<K>, Allocator>;

}  // namespace pma
