// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "pma/PolyAllocator.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <functional>
#include <memory>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace pma {

class MemoryResource;

namespace impl {

template<class TPointer, class TTarget, class TBase = TTarget>
class ManagedInstance {
public:
    using PointerType = TPointer;

private:
    explicit ManagedInstance(MemoryResource* memRes) :
        pMemRes{memRes} {
    }

public:
    static ManagedInstance with(MemoryResource* memRes) {
        return ManagedInstance{memRes};
    }

    template<typename... Args>
    PointerType create(Args&&... args) {
        pma::PolyAllocator<TTarget> alloc{pMemRes};
        auto deleter = [alloc](TBase* ptr) mutable { alloc.deleteObject(static_cast<TTarget*>(ptr)); };
        return {alloc.newObject(std::forward<Args>(args)...), deleter};
    }

private:
    MemoryResource* pMemRes;
};

}  // namespace impl

template<class TTarget, class TBase = TTarget>
using UniqueInstance = impl::ManagedInstance<std::unique_ptr<TBase, std::function<void(TBase*)>>, TTarget, TBase>;

template<class TTarget, class TBase = TTarget>
using SharedInstance = impl::ManagedInstance<std::shared_ptr<TBase>, TTarget, TBase>;

}  // namespace pma
