// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define PMA_MAJOR_VERSION 1
#define PMA_MINOR_VERSION 3
#define PMA_PATCH_VERSION 18
#define PMA_VERSION_STRING "1.3.18"
