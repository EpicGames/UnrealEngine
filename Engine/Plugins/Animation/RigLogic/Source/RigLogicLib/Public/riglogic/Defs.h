// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(FORCE_DLL_ATTRIBUTE_EXPORT_IMPORT)
    #define DLL_EXPORT __attribute__((dllexport))
    #define DLL_IMPORT __attribute__((dllimport))
#elif defined(FORCE_DLL_DECLSPEC_EXPORT_IMPORT)
    #define DLL_EXPORT __declspec(dllexport)
    #define DLL_IMPORT __declspec(dllimport)
#elif defined(FORCE_DLL_VISIBILITY_EXPORT_IMPORT)
    #define DLL_EXPORT __attribute__((visibility("default")))
    #define DLL_IMPORT DLL_EXPORT
#else
    #if defined(_WIN32) || defined(__CYGWIN__)
        #if defined(__GNUC__)
            #define DLL_EXPORT __attribute__((dllexport))
            #define DLL_IMPORT __attribute__((dllimport))
        #else
            #define DLL_EXPORT __declspec(dllexport)
            #define DLL_IMPORT __declspec(dllimport)
        #endif
    #elif defined(__GNUC__)
        #define DLL_EXPORT __attribute__((visibility("default")))
        #define DLL_IMPORT DLL_EXPORT
    #endif
#endif

#if defined(RL_BUILD_SHARED)
    // Build shared library
    #define RLAPI DLL_EXPORT
#elif defined(RL_SHARED)
    // Use shared library
    #define RLAPI DLL_IMPORT
#else
    // Build or use static library
    #define RLAPI
#endif
