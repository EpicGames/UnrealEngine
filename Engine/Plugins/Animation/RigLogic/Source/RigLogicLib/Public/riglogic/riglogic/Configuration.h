// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

namespace rl4 {

/**
    @brief Implementation type for RigLogic calculations.
*/
enum class CalculationType : std::uint8_t {
    Scalar,    ///< scalar CPU algorithm
    SSE,       ///< vectorized (SSE) CPU algorithm
    AVX,       ///< vectorized (AVX) CPU algorithm (RigLogic must be built with AVX support,
               ///< otherwise it falls back to using the Scalar version)
    NEON,      ///< vectorized (NEON) CPU algorithm (RigLogic must be built with NEON support,
               ///< otherwise it falls back to using the Scalar version)
    AnyVector  ///< Pick any available vectorization
};

/**
    @brief Floating point type used in vectorized calculations.
*/
enum class FloatingPointType : std::uint8_t {
    Float,
    HalfFloat
};

/**
    @brief Translation type to be used by RigLogic.
*/
enum class TranslationType : std::uint8_t {
    Vector = 3,
};

/**
    @brief Rotation type to be used by RigLogic.
*/
enum class RotationType : std::uint8_t {
    EulerAngles = 3,
    Quaternions = 4
};

/**
    @brief Scale type to be used by RigLogic.
*/
enum class ScaleType : std::uint8_t {
    Vector = 3,
};

struct Configuration {
    CalculationType calculationType = CalculationType::AnyVector;
    FloatingPointType floatingPointType = FloatingPointType::HalfFloat;
    bool loadJoints = true;
    bool loadBlendShapes = true;
    bool loadAnimatedMaps = true;
    bool loadMachineLearnedBehavior = true;
    bool loadRBFBehavior = true;
    bool loadTwistSwingBehavior = true;
    TranslationType translationType = TranslationType::Vector;
    RotationType rotationType = RotationType::EulerAngles;
    ScaleType scaleType = ScaleType::Vector;
    float translationPruningThreshold = 0.0f;  // Reasonably safe to try 0.0001f;
    float rotationPruningThreshold = 0.0f;     // Reasonably safe to try 0.1f
    float scalePruningThreshold = 0.0f;        // Reasonably safe to try 0.001f;
};

}  // namespace rl4
