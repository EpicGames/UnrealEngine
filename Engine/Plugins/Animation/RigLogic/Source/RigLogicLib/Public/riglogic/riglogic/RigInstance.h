// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/Defs.h"
#include "riglogic/types/Aliases.h"

#include <cstdint>

namespace rl4 {

class RigLogic;

/**
    @brief RigInstance contains the instance specific data of a rig.
    @note
        The input / output buffers are all contained within a rig instance.
        It provides functions for setting the level of detail and input control values of a single instance,
        as well as accessors to the output values of joints, blend shapes and animated maps.
        To evaluate / drive the rig instance, it must be passed to the RigLogic::calculate function.
    @see RigLogic
*/
class RLAPI RigInstance {
protected:
    virtual ~RigInstance();

public:
    /**
        @brief Factory method for the creation of rig instances.
        @note
            Rig instances are all tied to their parent RigLogic instance.
            All rig instances created through a particular RigLogic instance are based on the same DNA.
        @warning
            It is not possible to evaluate a rig instance created through one RigLogic instance by another RigLogic instance.
        @param rigLogic
            The RigLogic instance upon which this RigInstance should be based.
        @param memRes
            A custom memory resource to be used for the allocation of the rig instance resources.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static RigInstance* create(RigLogic* rigLogic, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing rig instances.
        @see create
    */
    static void destroy(RigInstance* instance);

    virtual std::uint16_t getGUIControlCount() const = 0;
    virtual float getGUIControl(std::uint16_t index) const = 0;
    virtual void setGUIControl(std::uint16_t index, float value) = 0;
    virtual ArrayView<float> getGUIControlValues() = 0;
    virtual ConstArrayView<float> getGUIControlValues() const = 0;

    virtual std::uint16_t getRawControlCount() const = 0;
    virtual float getRawControl(std::uint16_t index) const = 0;
    virtual void setRawControl(std::uint16_t index, float value) = 0;
    virtual ArrayView<float> getRawControlValues() = 0;
    virtual ConstArrayView<float> getRawControlValues() const = 0;

    virtual std::uint16_t getPSDControlCount() const = 0;
    virtual float getPSDControl(std::uint16_t index) const = 0;
    virtual ConstArrayView<float> getPSDControlValues() const = 0;

    virtual std::uint16_t getMLControlCount() const = 0;
    virtual float getMLControl(std::uint16_t index) const = 0;
    virtual ConstArrayView<float> getMLControlValues() const = 0;

    virtual std::uint16_t getMLTypeCount() const = 0;
    virtual std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const = 0;
    virtual std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const = 0;
    virtual ConstArrayView<float> getMLMaskValues() const = 0;
    virtual ArrayView<float> getMLMaskValues() = 0;

    virtual std::uint16_t getRBFControlCount() const = 0;
    virtual float getRBFControl(std::uint16_t index) const = 0;
    virtual ConstArrayView<float> getRBFControlValues() const = 0;

    /**
        @brief Calculated values for joint transformations.
        @note
            This is just a primitive array of floats, providing access to all the values of all transformations.
        @return View over the array of values.
    */
    virtual ConstArrayView<float> getJointOutputs() const = 0;
    /**
        @brief Calculated values for blend shape deformations.
        @return View over the array of floats.
    */
    virtual ConstArrayView<float> getBlendShapeOutputs() const = 0;
    /**
        @brief Calculated values for animated map deformations.
        @return View over the array of floats.
    */
    virtual ConstArrayView<float> getAnimatedMapOutputs() const = 0;
    /**
        @brief The current level of details of this instance.
    */
    virtual std::uint16_t getLOD() const = 0;
    /**
        @brief The current level of details of this instance.
        @param level
            Value to set for current level of details.
    */
    virtual void setLOD(std::uint16_t level) = 0;
};

}  // namespace rl4

namespace pma {

template<>
struct DefaultInstanceCreator<rl4::RigInstance> {
    using type = FactoryCreate<rl4::RigInstance>;
};

template<>
struct DefaultInstanceDestroyer<rl4::RigInstance> {
    using type = FactoryDestroy<rl4::RigInstance>;
};

}  // namespace pma
