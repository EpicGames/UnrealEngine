// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/Defs.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/Stats.h"
#include "riglogic/types/Aliases.h"

#include <cstdint>

namespace rl4 {

class RigInstance;

/**
    @brief RigLogic calculates rig output values based on input control values.
    @note
        A single RigLogic instance can create and drive an arbitrary number of rig instances.
        Since the rig instance specific data is located on the RigInstance class, RigLogic
        itself may be considered stateless, and all of it's functions thread-safe.
    @see RigInstance
*/
class RLAPI RigLogic {
public:
    using Configuration = rl4::Configuration;

protected:
    virtual ~RigLogic();

public:
    /**
        @brief Factory method for the creation of RigLogic.
        @param reader
            Source from which to copy and optimize DNA data, which is used for rig evaluation
        @param config
            Determines which algorithm implementation is used for rig evaluation and which submodules to load (affects memory
       allocations)
        @param memRes
            A custom memory resource to be used for allocations.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static RigLogic* create(const dna::Reader* reader, const Configuration& config = {}, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing RigLogic.
        @param instance
            Instance of RigLogic to be freed.
        @see create
    */
    static void destroy(RigLogic* instance);
    /**
        @brief Factory method for restoring an instance of RigLogic from a memory dump.
        @note
            This form of creation allows faster instantiation of RigLogic, as it doesn't need to
            go through the storage optimization phase, it just loads an earlier dumped state of
            a RigLogic instance back into memory.
        @param source
            Source stream from which to restore the state, obtained by calling dump.
        @param memRes
            A custom memory resource to be used for allocations.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see dump
        @see destroy
    */
    static RigLogic* restore(BoundedIOStream* source, MemoryResource* memRes = nullptr);
    /**
        @brief Create a snapshot of an initialized RigLogic instance.
        @param destination
            The output stream into which the current state of RigLogic is going to be written.
        @see restore
    */
    virtual void dump(BoundedIOStream* destination) const = 0;
    /**
        @brief Retrieve the configuration that RigLogic was initialized with.
    */
    virtual const Configuration& getConfiguration() const = 0;
    /**
        @brief Available levels of detail (e.g. 6 which means the following levels are available:
            [0,1,2,3,4,5], where 0 is the LOD with the highest details, and 5 is the LOD with
            lowest details).
    */
    virtual std::uint16_t getLODCount() const = 0;
    /**
        @brief List of RBF solver indices that need to be computed on the specified LOD.
        @param lod
            The LOD for which the indices are requested.
    */
    virtual ConstArrayView<std::uint16_t> getRBFSolverIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief List of ML operation indices that need to be computed on the specified LOD.
        @param lod
            The LOD for which the indices are requested.
    */
    virtual ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t lod,
                                                                      std::uint16_t mlTypeIndex,
                                                                      std::uint16_t mlOperationSetIndex) const = 0;
    /**
        @brief List of blend shape channel indices that need to be computed on the specified LOD.
        @param lod
            The LOD for which the indices are requested.
    */
    virtual ConstArrayView<std::uint16_t> getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief List of animated map indices that need to be computed on the specified LOD.
        @param lod
            The LOD for which the indices are requested.
    */
    virtual ConstArrayView<std::uint16_t> getAnimatedMapIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief List of joint indices that need to be computed on the specified LOD.
        @param lod
            The LOD for which the indices are requested.
    */
    virtual ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const = 0;
    /**
        @brief Neutral values of joint transformations.
        @note
            These is just a primitive array of floats, providing access to all the values of all transformations.
        @return View over the array of values.
    */
    virtual ConstArrayView<float> getNeutralJointValues() const = 0;
    /**
        @brief All joint output indices concatenated into a single chunk per each LOD.
    */
    virtual ConstArrayView<std::uint16_t> getJointVariableAttributeIndices(std::uint16_t lod) const = 0;
    /**
        @brief Number of joint groups present in the entire joint matrix.
        @see calculateJoints
    */
    virtual std::uint16_t getJointGroupCount() const = 0;
    /**
        @brief Number of RBF solvers for driving RBF behavior.
        @see calculateRBFBehavior
    */
    virtual std::uint16_t getRBFSolverCount() const = 0;
    /**
        @brief Number of twist setups in body behavior.
    */
    virtual std::uint16_t getTwistCount() const = 0;
    /**
        @brief Number of swing setups in body behavior.
    */
    virtual std::uint16_t getSwingCount() const = 0;
    /**
        @brief Number of meshes.
    */
    virtual std::uint16_t getMeshCount() const = 0;
    /**
        @brief Number of regions within the specified mesh.
    */
    virtual std::uint16_t getMeshRegionCount(std::uint16_t meshIndex) const = 0;
    /**
        @brief Number of ML model types.
        @see calculateMLControls
    */
    virtual std::uint16_t getMLTypeCount() const = 0;
    /**
        @brief Number of ML operation sets for the specified type.
    */
    virtual std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const = 0;
    /**
        @brief Number of ML operation for the specified type.
        @note This can be thought of as parallel instances of a series of operations.
    */
    virtual std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const = 0;
    /**
        @brief Maps GUI controls to raw controls.
        @note
            This method may be called after either RigInstance::setGUIControlValues or RigInstance::setGUIControl was used.
    */
    virtual void mapGUIToRawControls(RigInstance* instance) const = 0;
    /**
        @brief Maps raw controls to GUI controls.
        @note
            This method may be called after either RigInstance::setRawControlValues or RigInstance::setRawControl was used.
    */
    virtual void mapRawToGUIControls(RigInstance* instance) const = 0;
    /**
        @brief Calculate controls driving the machine learned behavior of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose controls for driving machine learned behavior are to be calculated.
        @see calculate
    */
    virtual void calculateMLControls(RigInstance* instance) const = 0;
    /**
        @brief Calculate controls driving the machine learned behavior of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose controls for driving machine learned behavior are to be calculated.
        @see calculate
    */
    virtual void calculateMLControls(RigInstance* instance,
                                     std::uint16_t mlTypeIndex,
                                     std::uint16_t mlOperationSetIndex,
                                     std::uint16_t mlOperationIndex) const = 0;
    /**
        @brief Calculate controls driving the RBF behavior of the of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose controls for driving machine learned behavior are to be calculated.
        @see calculate
    */
    virtual void calculateRBFControls(RigInstance* instance) const = 0;
    /**
        @brief Calculate controls driving the RBF behavior of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose controls for RBF behavior are to be calculated.
        @param solverIndex
            The RBF solver whose outputs need to be calculated.
        @warning
            The index must be less than the value returned by getRBFSolverCount.
        @see calculate
    */
    virtual void calculateRBFControls(RigInstance* instance, std::uint16_t solverIndex) const = 0;
    /**
        @brief Calculate only the PSD control values of the rig.
        @note
            Based on the set or computed raw control values, calculate the PSD outputs.
            The raw control values, followed by the PSD, ML and RBF values represent the input values of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose outputs are to be calculated.
        @see calculate
    */
    virtual void calculatePSDControls(RigInstance* instance) const = 0;
    /**
        @brief Calculate only the joint outputs of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose outputs are to be calculated.
        @see calculate
    */
    virtual void calculateJoints(RigInstance* instance) const = 0;
    /**
        @brief Calculate individual joint groups.
        @note
            It's intended to be used in a multi-threaded environment, such that the calculatin of all joint groups
            is performed by several threads, instead of computing all at once.
        @note
            This is considered as an expert usage use case.
        @param instance
            The rig instance whose outputs are to be calculated.
        @param jointGroupIndex
            A joint group's position in the zero-indexed array of joint groups.
        @warning
            The index must be less than the value returned by getJointGroupCount.
        @see calculate
    */
    virtual void calculateJoints(RigInstance* instance, std::uint16_t jointGroupIndex) const = 0;
    /**
        @brief Calculate only the blend shape channel weights of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose blend shape channel weights are to be calculated.
        @see calculate
    */
    virtual void calculateBlendShapes(RigInstance* instance) const = 0;
    /**
        @brief Calculate only the animated map outputs of the rig.
        @note
            This is considered as an advanced usage use case.
        @param instance
            The rig instance whose outputs are to be calculated.
        @see calculate
    */
    virtual void calculateAnimatedMaps(RigInstance* instance) const = 0;
    /**
        @brief Calculate outputs for joints, blend shapes, and animated maps.
        @note
            It performs a complete evaluation of the rig, computing all the output values in the following order:
              - Calculate input values (raw controls + PSDs)
              - Calculate joint output values
              - Calculate blend shape output values
              - Calculate animated map output values
        @param instance
            The rig instance whose outputs are to be calculated.
    */
    virtual void calculate(RigInstance* instance) const = 0;
    /**
        @brief Collect stats that summarize the overall amount of data that participate in computation for the current LOD.
        @param instance
            The rig instance which provides the current LOD to compute the necessary stats.
        @param stats
            The object into which the collected data will be written.
    */
    virtual void collectCalculationStats(const RigInstance* instance, Stats* stats) const = 0;
};

}  // namespace rl4

namespace pma {

template<>
struct DefaultInstanceCreator<rl4::RigLogic> {
    using type = FactoryCreate<rl4::RigLogic>;
};

template<>
struct DefaultInstanceDestroyer<rl4::RigLogic> {
    using type = FactoryDestroy<rl4::RigLogic>;
};

}  // namespace pma
