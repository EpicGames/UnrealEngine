// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/riglogic/Configuration.h"

#include <cstdint>

namespace rl4 {

struct Stats {
    CalculationType calculationType;
    FloatingPointType floatingPointType;
    std::uint16_t rbfSolverCount;
    std::uint16_t mlOperationCount;
    std::uint16_t psdCount;
    std::uint16_t blendShapeChannelCount;
    std::uint16_t animatedMapCount;
    std::uint16_t jointCount;
    std::uint32_t jointDeltaValueCount;
};

}  // namespace rl4
