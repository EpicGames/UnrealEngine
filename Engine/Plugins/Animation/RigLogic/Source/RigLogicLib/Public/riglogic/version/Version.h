// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define RL_MAJOR_VERSION 13
#define RL_MINOR_VERSION 2
#define RL_PATCH_VERSION 5
#define RL_VERSION_STRING "13.2.5"
