// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/Defs.h"
#include "riglogic/types/Aliases.h"

namespace rl4 {

struct RLAPI VersionInfo {
    static int getMajorVersion();
    static int getMinorVersion();
    static int getPatchVersion();
    static StringView getVersionString();
};

}  // namespace rl4
