// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "status/Defs.h"
#include "status/StatusCode.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <stdio.h>

#include <algorithm>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace sc {

namespace impl {

template<std::size_t... Is>
struct ISeq {
    static constexpr std::size_t size() {
        return sizeof...(Is);
    }
};

template<std::size_t Size, std::size_t Offset, std::size_t... Is>
struct Make : Make<(Size - 1), Offset, (Offset + Size) - 1, Is...> {};

template<std::size_t Offset, std::size_t... Is>
struct Make<0, Offset, Is...> {
    using Type = ISeq<Is...>;
};

template<std::size_t Size, std::size_t Offset = std::size_t{}>
using MakeISeq = typename Make<Size, Offset>::Type;

}  // namespace impl

class SCAPI StatusProvider {
public:
    explicit StatusProvider(std::initializer_list<StatusCode> statuses);

    static void reset();
    static StatusCode get();
    static bool isOk();
    static void set(StatusCode status);

    template<std::size_t... Is, typename... Args>
    static void set(StatusCode status, impl::ISeq<Is...> /*unused*/, Args&&... args) {
        constexpr std::size_t bufferSize = 512ul;
        char buffer[bufferSize] = {};
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wformat-security"
#endif
        // Invoke the hook with index = 0, denoting that this is the status message itself being hooked
        // The return value from the hook will override the original message
        status.message = execHook(status, 0ul, status.message);
        // The returned number of bytes to be written does not include the null terminator
        const auto neededSize = snprintf(nullptr, 0ul, status.message, execHook(status, Is, args)...) + 1;
        const auto size = std::min(bufferSize, static_cast<std::size_t>(neededSize));
        // Invoke the hook with the index denoting the argument position [1..nargs] (for each const char* argument passed to
        // `set`). The return value from the hook will override the original argument. Arguments of type other than const char*
        // remain untouched and just pass through.
        snprintf(buffer, size, status.message, execHook(status, Is, args)...);
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        status.message = buffer;
        execSet(status);
    }

    template<typename... Args>
    static void set(StatusCode status, Args&&... args) {
        // Generate a compile-time integer sequence for each argument [1..nargs]
        using ArgIndices = impl::MakeISeq<sizeof...(Args), static_cast<std::size_t>(1)>;
        set(status, ArgIndices{}, args...);
    }

private:
    static void execSet(StatusCode status);
    static const char* execHook(StatusCode status, std::size_t index, const char* data);

    template<typename T>
    static T&& execHook(StatusCode /*unused*/, std::size_t /*unused*/, T&& data) {
        return data;
    }
};

}  // namespace sc
