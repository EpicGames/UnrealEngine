// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "status/Defs.h"
#include "status/StatusCode.h"

namespace sc {

/**
 * @brief Hook function that allows overriding the status message itself or any of the string arguments used for interpolation of
 * the final status message.
 *
 * @param StatusCode
 *     The StatusCode itself whose message or argument is being hooked at the time of call.
 * @param std::size_t
 *     The index denoting which argument is being processed by the hook.
 * @note
 *     The value will be 0 when the hook is invoked for the status message itself, and from 1..N for any of the arguments used for
 * interpolating the status message. Although the hook is invoked only for const char* arguments, the indices will still denote
 * their actual position in the argument list.
 * @param const char*
 *     The original status message or interpolation argument.
 * @return
 *     Another message that will be used as status message or interpolation argument instead of the passed in const char*.
 */
using HookFunction = const char* (*)(StatusCode, std::size_t, const char*);

class SCAPI Status {
public:
    static bool isOk();
    static StatusCode get();
    static HookFunction getHook();
    static void setHook(HookFunction hook);
};

}  // namespace sc
