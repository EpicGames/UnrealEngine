// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "status/Defs.h"

#include <cstddef>

namespace sc {

struct SCAPI StatusCode {
    int code;
    const char* message;
};

inline bool operator==(const StatusCode& lhs, const StatusCode& rhs) {
    return (lhs.code == rhs.code);
}

inline bool operator!=(const StatusCode& lhs, const StatusCode& rhs) {
    return !(lhs == rhs);
}

}  // namespace sc
