// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define SC_MAJOR_VERSION 1
#define SC_MINOR_VERSION 2
#define SC_PATCH_VERSION 12
#define SC_VERSION_STRING "1.2.12"
