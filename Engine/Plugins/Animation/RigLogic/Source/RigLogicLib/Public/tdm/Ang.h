// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "tdm/Types.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace tdm {

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4521)
#endif

inline constexpr double pi() {
    return 3.14159265358979323846;
}

template<typename T>
inline typename std::enable_if<std::is_floating_point<T>::value, T>::type degrees(T radians) {
    static constexpr const auto c = static_cast<T>(180.0 / pi());
    return c * radians;
}

template<typename T>
inline typename std::enable_if<std::is_floating_point<T>::value, T>::type radians(T degrees) {
    static constexpr const auto c = static_cast<T>(pi() / 180.0);
    return c * degrees;
}

template<typename T, typename TUnit>
struct ang {
    using value_type = T;

    value_type value;

    ang() :
        value{} {
    }

    explicit constexpr ang(value_type val) :
        value{val} {
    }

    constexpr ang(const ang<value_type, TUnit>& other) :
        value{other.value} {
    }

    ang<value_type, TUnit>& operator=(const ang<value_type, TUnit>& other) {
        value = other.value;
        return *this;
    }

    template<typename U,
             typename TTUnit = TUnit,
             typename std::enable_if<std::is_same<TTUnit, impl::degrees>::value>::type* = nullptr>
    explicit ang(const ang<U, impl::radians>& other) :
        value{static_cast<value_type>(180.0 / tdm::pi()) * static_cast<value_type>(other.value)} {
    }

    template<typename U,
             typename TTUnit = TUnit,
             typename std::enable_if<std::is_same<TTUnit, impl::radians>::value>::type* = nullptr>
    explicit ang(const ang<U, impl::degrees>& other) :
        value{static_cast<value_type>(tdm::pi() / 180.0) * static_cast<value_type>(other.value)} {
    }

    ang& operator+=(const ang& rhs) {
        value += rhs.value;
        return *this;
    }

    ang& operator-=(const ang& rhs) {
        value -= rhs.value;
        return *this;
    }

    ang& operator*=(value_type val) {
        value *= val;
        return *this;
    }

    ang& operator/=(value_type val) {
        value /= val;
        return *this;
    }

    ang operator-() const {
        return ang{-value};
    }
};

#ifdef _MSC_VER
    #pragma warning(pop)
#endif

template<typename T, typename TUnit>
inline bool operator==(const ang<T, TUnit>& lhs, const ang<T, TUnit>& rhs) {
    return lhs.value == rhs.value;
}

template<typename T, typename TUnit>
inline bool operator!=(const ang<T, TUnit>& lhs, const ang<T, TUnit>& rhs) {
    return !(lhs == rhs);
}

template<typename T, typename TUnit>
inline ang<T, TUnit> operator+(const ang<T, TUnit>& lhs, const ang<T, TUnit>& rhs) {
    return ang<T, TUnit>(lhs) += rhs;
}

template<typename T, typename TUnit>
inline ang<T, TUnit> operator-(const ang<T, TUnit>& lhs, const ang<T, TUnit>& rhs) {
    return ang<T, TUnit>(lhs) -= rhs;
}

template<typename T, typename TUnit>
inline ang<T, TUnit> operator*(const ang<T, TUnit>& lhs, T rhs) {
    return ang<T, TUnit>(lhs) *= rhs;
}

template<typename T, typename TUnit>
inline ang<T, TUnit> operator*(T lhs, const ang<T, TUnit>& rhs) {
    return ang<T, TUnit>(rhs) *= lhs;
}

template<typename T, typename TUnit>
inline ang<T, TUnit> operator/(const ang<T, TUnit>& lhs, T rhs) {
    return ang<T, TUnit>(lhs) /= rhs;
}

namespace ang_literals {

constexpr fdeg operator""_fdeg(long double angle) {
    return fdeg{static_cast<float>(angle)};
}

constexpr frad operator""_frad(long double angle) {
    return frad{static_cast<float>(angle)};
}

}  // namespace ang_literals

}  // namespace tdm
