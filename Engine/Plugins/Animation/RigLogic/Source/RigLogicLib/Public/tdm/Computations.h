// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "tdm/Mat.h"
#include "tdm/Types.h"
#include "tdm/Vec.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cmath>
#include <limits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace tdm {

template<typename T>
inline vec3<T> cross(const vec3<T>& lhs, const vec3<T>& rhs) {
    return vec3<T>{lhs[1] * rhs[2] - lhs[2] * rhs[1], lhs[2] * rhs[0] - lhs[0] * rhs[2], lhs[0] * rhs[1] - lhs[1] * rhs[0]};
}

template<dim_t L, typename T>
inline T dot(const vec<L, T>& lhs, const vec<L, T>& rhs) {
    return (lhs * rhs).sum();
}

template<typename T>
inline T dot(const quat<T>& q1, const quat<T>& q2) {
    return q1.x * q2.x + q1.y * q2.y + q1.z * q2.z + q1.w * q2.w;
}

template<dim_t L, typename T>
inline vec<L, T> negate(vec<L, T> v) {
    return v.negate();
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> negate(mat<R, C, T> m) {
    return m.negate();
}

template<typename T>
inline quat<T> negate(quat<T> q) {
    return q.negate();
}

template<dim_t L, typename T>
inline typename std::enable_if<std::is_floating_point<T>::value, T>::type length(const vec<L, T>& v) {
    return v.length();
}

template<typename T>
inline typename std::enable_if<std::is_floating_point<T>::value, T>::type length(const quat<T>& q) {
    return q.length();
}

template<dim_t L, typename T>
inline typename std::enable_if<std::is_floating_point<T>::value, vec<L, T>>::type normalize(vec<L, T> v) {
    return v.normalize();
}

template<typename T>
inline typename std::enable_if<std::is_floating_point<T>::value, quat<T>>::type normalize(quat<T> q) {
    return q.normalize();
}

template<typename T>
inline quat<T> conjugate(const quat<T>& q) {
    return quat<T>{-q.x, -q.y, -q.z, q.w};
}

template<typename T>
inline quat<T> inverse(const quat<T>& q) {
    return conjugate(q) * (static_cast<T>(1.0) / q.length2());
}

template<typename T>
inline quat<T> lerp(const quat<T>& q1, const quat<T>& q2, T t) {
    return q1 * (static_cast<T>(1.0) - t) + q2 * t;
}

template<typename T>
inline quat<T> slerp(const quat<T>& q1, const quat<T>& q2, T t) {
    T costheta = dot(q1, q2);
    quat<T> tmp = q2;
    if (costheta < static_cast<T>(0.0)) {
        tmp.negate();
        costheta = -costheta;
    }

    T t1;
    T t2;
    if (costheta > (static_cast<T>(1.0) - std::numeric_limits<T>::epsilon())) {
        t1 = static_cast<T>(1.0) - t;
        t2 = t;
    } else {
        const T theta = std::acos(costheta);
        const T sintheta = std::sin(theta);
        const T rsintheta = static_cast<T>(1.0) / sintheta;
        t1 = std::sin((static_cast<T>(1.0) - t) * theta) * rsintheta;
        t2 = std::sin(t * theta) * rsintheta;
    }
    return q1 * t1 + tmp * t2;
}

template<dim_t R, dim_t C, typename T>
inline mat<C, R, T> transpose(const mat<R, C, T>& m) {
    using row_type = typename mat<C, R, T>::row_type;
    mat<C, R, T> ret;
    ret.apply([&m](row_type& row, dim_t i) { row = m.column(i); });
    return ret;
}

namespace impl {

#pragma push_macro("minor")
#undef minor

template<dim_t N, typename T>
inline void minor(const mat<N, N, T>& input, dim_t dimensions, dim_t i, dim_t j, mat<N, N, T>& output) {
    for (dim_t outRow{}, inRow{}; inRow < dimensions; ++inRow) {
        for (dim_t outCol{}, inCol{}; inCol < dimensions; ++inCol) {
            if ((inRow != i) && (inCol != j)) {
                output(outRow, outCol) = input(inRow, inCol);
                ++outCol;
                if (outCol == (dimensions - static_cast<dim_t>(1))) {
                    outCol = {};
                    ++outRow;
                }
            }
        }
    }
}

template<dim_t N, typename T>
inline T determinant(const mat<N, N, T>& m, dim_t dimensions) {
    if (dimensions == static_cast<dim_t>(1)) {
        return m(0, 0);
    }

    T result{};
    mat<N, N, T> temp;
    auto sign = static_cast<T>(1);
    const dim_t i{};
    for (dim_t j{}; j < dimensions; ++j) {
        minor(m, dimensions, i, j, temp);
        result += (sign * m(i, j) * determinant(temp, dimensions - 1));
        sign = -sign;
    }

    return result;
}

template<dim_t N, typename T>
inline mat<N, N, T> adjoint(const mat<N, N, T>& m) {
    if (m.rows() == static_cast<dim_t>(1)) {
        return mat<N, N, T>{static_cast<T>(1)};
    }

    mat<N, N, T> result;
    mat<N, N, T> temp;
    for (dim_t row{}; row < m.rows(); ++row) {
        for (dim_t col{}; col < m.columns(); ++col) {
            minor(m, N, row, col, temp);
            const T sign = static_cast<T>((row + col) % 2u == 0u ? 1 : -1);
            result(col, row) = (sign * determinant(temp, N - 1));
        }
    }
    return result;
}

#pragma pop_macro("minor")

}  // namespace impl

template<dim_t N, typename T>
inline T determinant(const mat<N, N, T>& m) {
    return impl::determinant(m, N);
}

template<dim_t N, typename T>
inline mat<N, N, T> inverse(const mat<N, N, T>& m) {
    T det = determinant(m);
    if (det == T{}) {
        return {};
    }

    mat<N, N, T> adj = impl::adjoint(m);
    mat<N, N, T> inv;
    for (dim_t row{}; row < m.rows(); ++row) {
        for (dim_t col{}; col < m.columns(); ++col) {
            inv(row, col) = adj(row, col) / det;
        }
    }
    return inv;
}

template<dim_t N, typename T>
inline T trace(const mat<N, N, T>& m) {
    T trace{0};
    for (dim_t row{}; row < m.rows(); ++row) {
        trace += m(row, row);
    }
    return trace;
}

namespace lu {

// Based on the algorithms found in :
// Numerical Recipes in C - The Art of Scientific Computing - Second Edition

template<dim_t N, typename T>
inline bool decompose(mat<N, N, T>& a, vec<N, dim_t>& permute) {
    constexpr T absmin = static_cast<T>(1.0e-20);

    vec<N, T> scale;

    for (dim_t i = {}; i < N; ++i) {
        T rowMax = {};
        for (dim_t j = {}; j < N; ++j) {
            rowMax = std::max(rowMax, std::abs(a(i, j)));
        }

        if (rowMax == static_cast<T>(0.0)) {
            return false;
        }
        scale[i] = static_cast<T>(1.0) / rowMax;
    }

    for (dim_t j = {}; j < N; ++j) {
        for (dim_t i = {}; i < j; ++i) {
            T sum = a(i, j);
            for (dim_t k = {}; k < i; ++k) {
                sum -= a(i, k) * a(k, j);
            }
            a(i, j) = sum;
        }

        dim_t iMax = {};
        T colMax = {};
        for (dim_t i = j; i < N; ++i) {
            T sum = a(i, j);
            for (dim_t k = {}; k < j; ++k) {
                sum -= a(i, k) * a(k, j);
            }
            a(i, j) = sum;

            const T temp = scale[i] * std::abs(sum);
            if (temp >= colMax) {
                colMax = temp;
                iMax = i;
            }
        }

        if (j != iMax) {
            for (dim_t k = {}; k < N; ++k) {
                std::swap(a(iMax, k), a(j, k));
            }
            scale[iMax] = scale[j];
        }

        permute[j] = iMax;

        if (a[j][j] == static_cast<T>(0.0)) {
            a[j][j] = absmin;
        }

        if (j != (N - 1)) {
            const T temp = static_cast<T>(1.0) / a(j, j);
            for (dim_t i = j + 1; i < N; ++i) {
                a(i, j) *= temp;
            }
        }
    }

    return true;
}

template<dim_t N, typename T>
inline void substitute(const mat<N, N, T>& a, const vec<N, dim_t>& permute, vec<N, T>& b) {
    dim_t ii = {};
    for (dim_t i = {}; i < N; ++i) {
        const dim_t ip = permute[i];
        T sum = b[ip];
        b[ip] = b[i];
        if (ii) {
            for (dim_t j = (ii - 1); j < i; ++j) {
                sum -= a(i, j) * b[j];
            }
        } else if (sum != static_cast<T>(0.0)) {
            ii = i + 1;
        }
        b[i] = sum;
    }

    for (dim_t ipo = N; ipo > 0; --ipo) {
        const dim_t i = ipo - 1;
        T sum = b[i];
        for (dim_t j = i + 1; j < N; ++j) {
            sum -= a(i, j) * b[j];
        }
        b[i] = sum / a(i, i);
    }
}

template<dim_t N, typename T>
inline mat<N, N, T> inverse(mat<N, N, T> m) {
    vec<N, dim_t> permute;
    if (!decompose(m, permute)) {
        return {};
    }

    mat<N, N, T> inv;
    for (dim_t j = {}; j < N; ++j) {
        vec<N, T> col;
        col[j] = static_cast<T>(1.0);
        substitute(m, permute, col);
        for (dim_t i = {}; i < N; ++i) {
            inv[i][j] = col[i];
        }
    }
    return inv;
}

}  // namespace lu

}  // namespace tdm
