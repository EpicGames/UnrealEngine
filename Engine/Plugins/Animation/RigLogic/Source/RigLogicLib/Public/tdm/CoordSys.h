// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "tdm/Types.h"

namespace tdm {

namespace impl {

template<typename T>
inline vec3<T> axis_vector(axis_dir axis) {
    switch (axis) {
    case axis_dir::right:
        return {static_cast<T>(1.0), static_cast<T>(0.0), static_cast<T>(0.0)};
    case axis_dir::left:
        return {static_cast<T>(-1.0), static_cast<T>(0.0), static_cast<T>(0.0)};
    case axis_dir::up:
        return {static_cast<T>(0.0), static_cast<T>(1.0), static_cast<T>(0.0)};
    case axis_dir::down:
        return {static_cast<T>(0.0), static_cast<T>(-1.0), static_cast<T>(0.0)};
    case axis_dir::front:
        return {static_cast<T>(0.0), static_cast<T>(0.0), static_cast<T>(1.0)};
    case axis_dir::back:
        return {static_cast<T>(0.0), static_cast<T>(0.0), static_cast<T>(-1.0)};
    }
    return {static_cast<T>(0.0), static_cast<T>(0.0), static_cast<T>(0.0)};
}

}  // namespace impl

struct coord_sys {
    axis_dir x;
    axis_dir y;
    axis_dir z;

    template<typename T>
    mat3<T> basis() const {
        // Rows of B are local axes expressed in canonical coordinates.
        // Row-vector: v_dst = v_src * C, with C = B_src * B_dst^T
        const vec3<T> vx = impl::axis_vector<T>(x);
        const vec3<T> vy = impl::axis_vector<T>(y);
        const vec3<T> vz = impl::axis_vector<T>(z);
        return mat3<T>{vx, vy, vz};
    }

    template<typename T>
    chirality handedness() const {
        // The naming convention we use is that "front" is into screen, and "back" is from screen (towards the viewer). However,
        // the basis coordinate system, for a right handed system, would, for a X = right, Y = up, point Z towards the viewer,
        // i.e. it would make Z+ (0,0,1) point toward the viewer ("back") (the opposite of our convention). Because of legacy DNAs
        // that already have hard coded coordinate system description following the existing naming convention, we cannot simply
        // change the canonical basis to have front be "into screen", and "back" be towards the viewer. That is why we have a
        // clash with the naming and the mathematical basis representation and are forced to invert the logic for handedness, so
        // it would return a correct physical handedness for coordinate systems that follow our naming convention.
        const T det = determinant(basis<T>());
        return (det < static_cast<T>(0)) ? chirality::right : chirality::left;
    }

    template<typename T>
    bool valid() const {
        const T det = determinant(basis<T>());
        return std::fabs(std::fabs(det) - static_cast<T>(1)) < static_cast<T>(1e-6);
    }
};

inline bool operator==(const coord_sys& lhs, const coord_sys& rhs) {
    return lhs.x == rhs.x && lhs.y == rhs.y && lhs.z == rhs.z;
}

inline bool operator!=(const coord_sys& lhs, const coord_sys& rhs) {
    return !(lhs == rhs);
}

inline bool operator==(const rot_sign& lhs, const rot_sign& rhs) {
    return lhs.x == rhs.x && lhs.y == rhs.y && lhs.z == rhs.z;
}

inline bool operator!=(const rot_sign& lhs, const rot_sign& rhs) {
    return !(lhs == rhs);
}

}  // namespace tdm
