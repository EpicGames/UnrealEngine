// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "tdm/Types.h"
#include "tdm/Vec.h"

namespace tdm {

namespace impl {

template<typename...>
struct is_all_scalar : std::false_type {};

template<>
struct is_all_scalar<> : std::false_type {};

template<typename T>
struct is_all_scalar<T> : std::is_scalar<T> {};

template<typename T>
struct is_all_scalar<T, T> : std::is_scalar<T> {};

template<typename T, typename... Us>
struct is_all_scalar<T, T, Us...> : is_all_scalar<T, Us...> {};

}  // namespace impl

template<dim_t R, dim_t C, typename T>
inline mat<C, R, T> transpose(const mat<R, C, T>& m);

template<dim_t N, typename T>
inline mat<N, N, T> inverse(const mat<N, N, T>& m);

template<dim_t R, dim_t C, typename T>
struct mat {
    using value_type = T;
    using row_type = vec<C, value_type>;
    using column_type = vec<R, value_type>;

    static constexpr dim_t rows() {
        return column_type::dimensions();
    }

    static constexpr dim_t columns() {
        return row_type::dimensions();
    }

private:
    row_type values[R];

public:
    mat() :
        values{} {
    }

    ~mat() = default;

    mat(const mat&) = default;
    mat& operator=(const mat&) = default;

    mat(mat&&) = default;
    mat& operator=(mat&&) = default;

    template<typename U, typename std::enable_if<std::is_scalar<U>::value>::type* = nullptr>
    explicit mat(U scalar) {
        apply([scalar](row_type& row, dim_t /*unused*/) { row = row_type{scalar}; });
    }

    template<typename... Us,
             typename std::enable_if<(sizeof...(Us) == R * C) && impl::is_all_scalar<Us...>::value>::type* = nullptr>
    mat(Us... scalars) {
        T tmp[sizeof...(Us)] = {static_cast<T>(scalars)...};
        apply([&tmp](row_type& row, dim_t ri) {
            row.apply([&tmp, ri](value_type& value, dim_t ci) { value = tmp[ri * columns() + ci]; });
        });
    }

    template<typename U>
    mat(const mat<R, C, U>& rhs) {
        apply([&rhs](row_type& row, dim_t ri) { row = rhs[ri]; });
    }

    template<typename U>
    mat& operator=(const mat<R, C, U>& rhs) {
        return operator=(mat<R, C, T>{rhs});
    }

    template<typename... Us, typename std::enable_if<sizeof...(Us) == R>::type* = nullptr>
    mat(const vec<C, Us>&... vs) :
        values{row_type{vs}...} {
    }

    template<typename... Us, typename std::enable_if<sizeof...(Us) == R>::type* = nullptr>
    static mat from_rows(const vec<C, Us>&... vs) {
        return mat{vs...};
    }

    template<typename... Us, typename std::enable_if<sizeof...(Us) == C>::type* = nullptr>
    static mat from_columns(const vec<R, Us>&... vs) {
        mat<sizeof...(vs), R, T> tmp{vs...};
        return tdm::transpose(tmp);
    }

    template<typename U, typename..., dim_t H = R, dim_t W = C>
    static typename std::enable_if<H == W, mat>::type diagonal(U scalar) {
        mat ret;
        ret.apply([scalar](row_type& row, dim_t ri) { row[ri] = scalar; });
        return ret;
    }

    template<typename U, typename..., dim_t H = R, dim_t W = C>
    static typename std::enable_if<H == W, mat>::type diagonal(const vec<W, U>& scalars) {
        mat ret;
        ret.apply([&scalars](row_type& row, dim_t ri) { row[ri] = scalars[ri]; });
        return ret;
    }

    template<typename... Us, dim_t H = R, dim_t W = C>
    static typename std::enable_if<(H == W) && (sizeof...(Us) == H), mat>::type diagonal(Us... scalars) {
        return diagonal(vec<R, T>{scalars...});
    }

    template<typename..., dim_t H = R, dim_t W = C>
    static typename std::enable_if<H == W, mat>::type identity() {
        return diagonal(static_cast<T>(1));
    }

    row_type& operator[](dim_t index) {
        // NOLINTNEXTLINE(cppcoreguidelines-pro-bounds-array-to-pointer-decay,hicpp-no-array-decay)
        assert(index < rows());
        // NOLINTNEXTLINE(cppcoreguidelines-pro-bounds-constant-array-index)
        return values[index];
    }

    const row_type& operator[](dim_t index) const {
        // NOLINTNEXTLINE(cppcoreguidelines-pro-bounds-array-to-pointer-decay,hicpp-no-array-decay)
        assert(index < rows());
        // NOLINTNEXTLINE(cppcoreguidelines-pro-bounds-constant-array-index)
        return values[index];
    }

    T& operator()(dim_t rowIndex, dim_t colIndex) {
        return operator[](rowIndex)[colIndex];
    }

    const T& operator()(dim_t rowIndex, dim_t colIndex) const {
        return operator[](rowIndex)[colIndex];
    }

    template<typename F>
    mat& apply(F func) {
        for (dim_t ri{}; ri != rows(); ++ri) {
            func(values[ri], ri);
        }
        return *this;
    }

    template<typename F>
    const mat& apply(F func) const {
        for (dim_t ri{}; ri != rows(); ++ri) {
            func(values[ri], ri);
        }
        return *this;
    }

    mat& operator++() {
        return apply([](row_type& row, dim_t /*unused*/) { ++row; });
    }

    mat& operator--() {
        return apply([](row_type& row, dim_t /*unused*/) { --row; });
    }

    template<typename U>
    mat& operator+=(U rhs) {
        return apply([rhs](row_type& row, dim_t /*unused*/) { row += rhs; });
    }

    template<typename U>
    mat& operator+=(const mat<R, C, U>& rhs) {
        return apply([&rhs](row_type& row, dim_t ri) { row += rhs[ri]; });
    }

    mat& operator+=(const mat& rhs) {
        return operator+= <T>(rhs);
    }

    template<typename U>
    mat& operator-=(U rhs) {
        return apply([rhs](row_type& row, dim_t /*unused*/) { row -= rhs; });
    }

    template<typename U>
    mat& operator-=(const mat<R, C, U>& rhs) {
        return apply([&rhs](row_type& row, dim_t ri) { row -= rhs[ri]; });
    }

    mat& operator-=(const mat& rhs) {
        return operator-= <T>(rhs);
    }

    template<typename U>
    mat& operator*=(U rhs) {
        return apply([rhs](row_type& row, dim_t /*unused*/) { row *= rhs; });
    }

    template<typename U>
    mat& operator*=(const mat<R, C, U>& rhs) {
        return (*this = *this * rhs);
    }

    mat& operator*=(const mat& rhs) {
        return operator*= <T>(rhs);
    }

    template<typename U>
    mat& operator/=(U rhs) {
        return apply([rhs](row_type& row, dim_t /*unused*/) { row /= rhs; });
    }

    template<typename U>
    mat& operator/=(const mat<R, C, U>& rhs) {
        return operator*=(inverse(rhs));
    }

    mat& operator/=(const mat& rhs) {
        return operator/= <T>(rhs);
    }

    template<typename..., dim_t H = R, dim_t W = C>
    typename std::enable_if<H == W, mat&>::type transpose() {
        return (*this = tdm::transpose(*this));
    }

    mat& negate() {
        apply([](row_type& row, dim_t /*unused*/) { row.negate(); });
        return *this;
    }

    row_type row(dim_t index) const {
        return operator[](index);
    }

    column_type column(dim_t index) const {
        column_type col;
        apply([&col, index](const row_type& row, dim_t ri) { col[ri] = row[index]; });
        return col;
    }

    template<dim_t H, dim_t W>
    typename std::enable_if<(H > 1 && W > 1 && H <= R && W <= C), mat<H, W, T>>::type submat(dim_t y, dim_t x) const {
        assert(H + y <= rows());
        assert(W + x <= columns());
        mat<H, W, T> ret;
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wattributes"
#endif
        ret.apply([this, y, x](typename mat<H, W, T>::row_type& row, dim_t ri) {
            row.apply([this, y, x, ri](value_type& value, dim_t ci) { value = values[y + ri][x + ci]; });
        });
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        return ret;
    }
};

template<dim_t R, dim_t C, typename T>
inline bool operator==(const mat<R, C, T>& lhs, const mat<R, C, T>& rhs) {
    using row_type = typename mat<R, C, T>::row_type;
    bool retval = true;
    lhs.apply([&rhs, &retval](const row_type& row, dim_t ri) { retval = retval && (row == rhs[ri]); });
    return retval;
}

template<dim_t R, dim_t C, typename T>
inline bool operator!=(const mat<R, C, T>& lhs, const mat<R, C, T>& rhs) {
    return !(lhs == rhs);
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator+(const mat<R, C, T>& m) {
    return m;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator-(const mat<R, C, T>& m) {
    mat<R, C, T> ret{m};
    ret.negate();
    return ret;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator+(const mat<R, C, T>& lhs, T rhs) {
    return mat<R, C, T>(lhs) += rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator+(T lhs, const mat<R, C, T>& rhs) {
    return mat<R, C, T>(lhs) += rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator+(const mat<R, C, T>& lhs, const mat<R, C, T>& rhs) {
    return mat<R, C, T>(lhs) += rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator-(const mat<R, C, T>& lhs, T rhs) {
    return mat<R, C, T>(lhs) -= rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator-(T lhs, const mat<R, C, T>& rhs) {
    return mat<R, C, T>(lhs) -= rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator-(const mat<R, C, T>& lhs, const mat<R, C, T>& rhs) {
    return mat<R, C, T>(lhs) -= rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator*(const mat<R, C, T>& lhs, T rhs) {
    return mat<R, C, T>(lhs) *= rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator*(T lhs, const mat<R, C, T>& rhs) {
    return mat<R, C, T>(rhs) *= lhs;
}

template<dim_t R, dim_t C, typename T>
inline typename mat<R, C, T>::row_type operator*(const typename mat<R, C, T>::column_type& lhs, const mat<R, C, T>& rhs) {
    using row_type = typename mat<R, C, T>::row_type;
    row_type ret;
    rhs.apply([&ret, &lhs](const row_type& row, dim_t ri) { ret += (row * row_type{lhs[ri]}); });
    return ret;
}

template<dim_t R, dim_t C, typename T>
inline typename mat<R, C, T>::column_type operator*(const mat<R, C, T>& lhs, const typename mat<R, C, T>::row_type& rhs) {
    using column_type = typename mat<R, C, T>::column_type;
    using value_type = typename column_type::value_type;
    column_type ret;
    rhs.apply([&ret, &lhs](value_type value, dim_t ci) { ret += (lhs.column(ci) * column_type{value}); });
    return ret;
}

template<dim_t R, dim_t S, dim_t C, typename T>
inline mat<R, C, T> operator*(const mat<R, S, T>& lhs, const mat<S, C, T>& rhs) {
    using row_type = typename mat<R, C, T>::row_type;
    mat<R, C, T> ret;
    ret.apply([&lhs, &rhs](row_type& row, dim_t ri) { row = (lhs[ri] * rhs); });
    return ret;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator/(const mat<R, C, T>& lhs, T rhs) {
    return mat<R, C, T>(lhs) /= rhs;
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator/(T lhs, const mat<R, C, T>& rhs) {
    using row_type = typename mat<R, C, T>::row_type;
    mat<R, C, T> tmp{rhs};
    tmp.apply([lhs](row_type& row, dim_t /*unused*/) { row = lhs / row; });
    return tmp;
}

template<dim_t R, dim_t C, typename T>
inline typename mat<R, C, T>::row_type operator/(const typename mat<R, C, T>::column_type& lhs, const mat<R, C, T>& rhs) {
    return (lhs * inverse(rhs));
}

template<dim_t R, dim_t C, typename T>
inline typename mat<R, C, T>::column_type operator/(const mat<R, C, T>& lhs, const typename mat<R, C, T>::row_type& rhs) {
    return (inverse(lhs) * rhs);
}

template<dim_t R, dim_t C, typename T>
inline mat<R, C, T> operator/(const mat<R, C, T>& lhs, const mat<R, C, T>& rhs) {
    return mat<R, C, T>(lhs) /= rhs;
}

template<dim_t R, dim_t C, typename T, typename F>
inline mat<R, C, T> applied(const mat<R, C, T>& lhs, F func) {
    mat<R, C, T> m{lhs};
    m.apply(func);
    return m;
}

}  // namespace tdm
