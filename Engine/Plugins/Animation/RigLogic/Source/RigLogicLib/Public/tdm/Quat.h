// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "tdm/Ang.h"
#include "tdm/Transforms.h"
#include "tdm/Types.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cmath>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace tdm {

namespace impl {

template<typename T, rot_seq order>
struct euler_to_quat;

template<typename T>
struct euler_to_quat<T, rot_seq::xyz> {

    quat<T> operator()(const rad3<T>& rot, rot_sign signs) {
        // Apply per-axis rotation direction sign
        const rad3<T> angles{rad<T>{rot[0].value * static_cast<T>(signs.x)},
                             rad<T>{rot[1].value * static_cast<T>(signs.y)},
                             rad<T>{rot[2].value * static_cast<T>(signs.z)}};
        rad3<T> h{angles * static_cast<T>(0.5)};
        rad3<T> c = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::cos(angle.value)}; });
        rad3<T> s = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::sin(angle.value)}; });

        const auto sx = s[0].value;
        const auto sy = s[1].value;
        const auto sz = s[2].value;

        const auto cx = c[0].value;
        const auto cy = c[1].value;
        const auto cz = c[2].value;

        // Intrinsic XYZ: Rx * Ry * Rz
        const auto x = sx * cy * cz - cx * sy * sz;
        const auto y = cx * sy * cz + sx * cy * sz;
        const auto z = cx * cy * sz - sx * sy * cz;
        const auto w = cx * cy * cz + sx * sy * sz;

        return quat<T>{x, y, z, w};
    }
};

template<typename T>
struct euler_to_quat<T, rot_seq::xzy> {

    quat<T> operator()(const rad3<T>& rot, rot_sign signs) {
        const rad3<T> angles{rad<T>{rot[0].value * static_cast<T>(signs.x)},
                             rad<T>{rot[1].value * static_cast<T>(signs.y)},
                             rad<T>{rot[2].value * static_cast<T>(signs.z)}};
        rad3<T> h{angles * static_cast<T>(0.5)};
        rad3<T> c = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::cos(angle.value)}; });
        rad3<T> s = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::sin(angle.value)}; });

        const auto sx = s[0].value;
        const auto sy = s[1].value;
        const auto sz = s[2].value;

        const auto cx = c[0].value;
        const auto cy = c[1].value;
        const auto cz = c[2].value;

        // Intrinsic XZY: Rx * Rz * Ry
        const auto x = sx * cy * cz + cx * sy * sz;
        const auto y = cx * sy * cz + sx * cy * sz;
        const auto z = cx * cy * sz - sx * sy * cz;
        const auto w = cx * cy * cz - sx * sy * sz;

        return quat<T>{x, y, z, w};
    }
};

template<typename T>
struct euler_to_quat<T, rot_seq::yxz> {

    quat<T> operator()(const rad3<T>& rot, rot_sign signs) {
        const rad3<T> angles{rad<T>{rot[0].value * static_cast<T>(signs.x)},
                             rad<T>{rot[1].value * static_cast<T>(signs.y)},
                             rad<T>{rot[2].value * static_cast<T>(signs.z)}};
        rad3<T> h{angles * static_cast<T>(0.5)};
        rad3<T> c = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::cos(angle.value)}; });
        rad3<T> s = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::sin(angle.value)}; });

        const auto sx = s[0].value;
        const auto sy = s[1].value;
        const auto sz = s[2].value;

        const auto cx = c[0].value;
        const auto cy = c[1].value;
        const auto cz = c[2].value;

        // Intrinsic YXZ: Ry * Rx * Rz
        const auto x = sx * cy * cz - cx * sy * sz;
        const auto y = cx * sy * cz + sx * cy * sz;
        const auto z = cx * cy * sz + sx * sy * cz;
        const auto w = cx * cy * cz - sx * sy * sz;

        return quat<T>{x, y, z, w};
    }
};

template<typename T>
struct euler_to_quat<T, rot_seq::yzx> {

    quat<T> operator()(const rad3<T>& rot, rot_sign signs) {
        const rad3<T> angles{rad<T>{rot[0].value * static_cast<T>(signs.x)},
                             rad<T>{rot[1].value * static_cast<T>(signs.y)},
                             rad<T>{rot[2].value * static_cast<T>(signs.z)}};
        rad3<T> h{angles * static_cast<T>(0.5)};
        rad3<T> c = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::cos(angle.value)}; });
        rad3<T> s = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::sin(angle.value)}; });

        const auto sx = s[0].value;
        const auto sy = s[1].value;
        const auto sz = s[2].value;

        const auto cx = c[0].value;
        const auto cy = c[1].value;
        const auto cz = c[2].value;

        // Intrinsic YZX: Ry * Rz * Rx
        const auto x = sx * cy * cz - cx * sy * sz;
        const auto y = cx * sy * cz - sx * cy * sz;
        const auto z = cx * cy * sz + sx * sy * cz;
        const auto w = cx * cy * cz + sx * sy * sz;

        return quat<T>{x, y, z, w};
    }
};

template<typename T>
struct euler_to_quat<T, rot_seq::zxy> {

    quat<T> operator()(const rad3<T>& rot, rot_sign signs) {
        const rad3<T> angles{rad<T>{rot[0].value * static_cast<T>(signs.x)},
                             rad<T>{rot[1].value * static_cast<T>(signs.y)},
                             rad<T>{rot[2].value * static_cast<T>(signs.z)}};
        rad3<T> h{angles * static_cast<T>(0.5)};
        rad3<T> c = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::cos(angle.value)}; });
        rad3<T> s = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::sin(angle.value)}; });

        const auto sx = s[0].value;
        const auto sy = s[1].value;
        const auto sz = s[2].value;

        const auto cx = c[0].value;
        const auto cy = c[1].value;
        const auto cz = c[2].value;

        // Intrinsic ZXY: Rz * Rx * Ry
        const auto x = sx * cy * cz + cx * sy * sz;
        const auto y = cx * sy * cz - sx * cy * sz;
        const auto z = cx * cy * sz - sx * sy * cz;
        const auto w = cx * cy * cz + sx * sy * sz;

        return quat<T>{x, y, z, w};
    }
};

template<typename T>
struct euler_to_quat<T, rot_seq::zyx> {

    quat<T> operator()(const rad3<T>& rot, rot_sign signs) {
        const rad3<T> angles{rad<T>{rot[0].value * static_cast<T>(signs.x)},
                             rad<T>{rot[1].value * static_cast<T>(signs.y)},
                             rad<T>{rot[2].value * static_cast<T>(signs.z)}};
        rad3<T> h{angles * static_cast<T>(0.5)};
        rad3<T> c = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::cos(angle.value)}; });
        rad3<T> s = applied(h, [](rad<T>& angle, dim_t /*unused*/) { angle = rad<T>{std::sin(angle.value)}; });

        const auto sx = s[0].value;
        const auto sy = s[1].value;
        const auto sz = s[2].value;

        const auto cx = c[0].value;
        const auto cy = c[1].value;
        const auto cz = c[2].value;

        // Intrinsic ZYX: Rz * Ry * Rx
        const auto x = sx * cy * cz + cx * sy * sz;
        const auto y = cx * sy * cz - sx * cy * sz;
        const auto z = cx * cy * sz + sx * sy * cz;
        const auto w = cx * cy * cz - sx * sy * sz;

        return quat<T>{x, y, z, w};
    }
};

template<typename T, rot_seq order>
struct quat_to_euler;

template<typename T>
struct quat_to_euler<T, rot_seq::xyz> {

    rad3<T> operator()(const quat<T>& q, rot_sign signs) {
        const T x2 = q.x + q.x;
        const T y2 = q.y + q.y;
        const T z2 = q.z + q.z;

        const T xx2 = q.x * x2;
        const T xy2 = q.x * y2;
        const T xz2 = q.x * z2;

        const T yy2 = q.y * y2;
        const T yz2 = q.y * z2;

        const T zz2 = q.z * z2;

        const T wx2 = q.w * x2;
        const T wy2 = q.w * y2;
        const T wz2 = q.w * z2;

        // Intrinsic XYZ: extract from Rx * Ry * Rz
        rad3<T> angles;
        const T sy = xz2 - wy2;
        angles[1] = rad<T>(fastasin(-sy));
        if (std::abs(sy) < static_cast<T>(0.99999999999)) {
            angles[0] = rad<T>(std::atan2(yz2 + wx2, static_cast<T>(1.0) - (xx2 + yy2)));
            angles[2] = rad<T>(std::atan2(xy2 + wz2, static_cast<T>(1.0) - (yy2 + zz2)));
        } else {
            angles[0] = rad<T>(static_cast<T>(0.0));
            angles[2] = rad<T>(std::atan2(wz2 - xy2, static_cast<T>(1.0) - (xx2 + zz2)));
        }
        // Apply per-axis sign to match rotation direction convention
        angles[0].value *= static_cast<T>(signs.x);
        angles[1].value *= static_cast<T>(signs.y);
        angles[2].value *= static_cast<T>(signs.z);
        return angles;
    }
};

template<typename T>
struct quat_to_euler<T, rot_seq::xzy> {

    rad3<T> operator()(const quat<T>& q, rot_sign signs) {
        const T x2 = q.x + q.x;
        const T y2 = q.y + q.y;
        const T z2 = q.z + q.z;

        const T xx2 = q.x * x2;
        const T xy2 = q.x * y2;
        const T xz2 = q.x * z2;

        const T yy2 = q.y * y2;
        const T yz2 = q.y * z2;

        const T zz2 = q.z * z2;

        const T wx2 = q.w * x2;
        const T wy2 = q.w * y2;
        const T wz2 = q.w * z2;

        // Intrinsic XZY: extract from Rx * Rz * Ry
        rad3<T> angles;
        const T sz = xy2 + wz2;
        angles[2] = rad<T>(fastasin(sz));
        if (std::abs(sz) < static_cast<T>(0.99999999999)) {
            angles[0] = rad<T>(std::atan2(wx2 - yz2, static_cast<T>(1.0) - (xx2 + zz2)));
            angles[1] = rad<T>(std::atan2(wy2 - xz2, static_cast<T>(1.0) - (yy2 + zz2)));
        } else {
            angles[0] = rad<T>(static_cast<T>(0.0));
            angles[1] = rad<T>(std::atan2(xz2 + wy2, static_cast<T>(1.0) - (xx2 + yy2)));
        }
        // Apply per-axis sign to match rotation direction convention
        angles[0].value *= static_cast<T>(signs.x);
        angles[1].value *= static_cast<T>(signs.y);
        angles[2].value *= static_cast<T>(signs.z);
        return angles;
    }
};

template<typename T>
struct quat_to_euler<T, rot_seq::yxz> {

    rad3<T> operator()(const quat<T>& q, rot_sign signs) {
        const T x2 = q.x + q.x;
        const T y2 = q.y + q.y;
        const T z2 = q.z + q.z;

        const T xx2 = q.x * x2;
        const T xy2 = q.x * y2;
        const T xz2 = q.x * z2;

        const T yy2 = q.y * y2;
        const T yz2 = q.y * z2;

        const T zz2 = q.z * z2;

        const T wx2 = q.w * x2;
        const T wy2 = q.w * y2;
        const T wz2 = q.w * z2;

        // Intrinsic YXZ: extract from Ry * Rx * Rz
        rad3<T> angles;
        const T sx = yz2 + wx2;
        angles[0] = rad<T>(fastasin(sx));
        if (std::abs(sx) < static_cast<T>(0.99999999999)) {
            angles[1] = rad<T>(std::atan2(wy2 - xz2, static_cast<T>(1.0) - (xx2 + yy2)));
            angles[2] = rad<T>(std::atan2(wz2 - xy2, static_cast<T>(1.0) - (xx2 + zz2)));
        } else {
            angles[1] = rad<T>(static_cast<T>(0.0));
            angles[2] = rad<T>(std::atan2(xy2 + wz2, static_cast<T>(1.0) - (yy2 + zz2)));
        }
        // Apply per-axis sign to match rotation direction convention
        angles[0].value *= static_cast<T>(signs.x);
        angles[1].value *= static_cast<T>(signs.y);
        angles[2].value *= static_cast<T>(signs.z);
        return angles;
    }
};

template<typename T>
struct quat_to_euler<T, rot_seq::yzx> {

    rad3<T> operator()(const quat<T>& q, rot_sign signs) {
        const T x2 = q.x + q.x;
        const T y2 = q.y + q.y;
        const T z2 = q.z + q.z;

        const T xx2 = q.x * x2;
        const T xy2 = q.x * y2;
        const T xz2 = q.x * z2;

        const T yy2 = q.y * y2;
        const T yz2 = q.y * z2;

        const T zz2 = q.z * z2;

        const T wx2 = q.w * x2;
        const T wy2 = q.w * y2;
        const T wz2 = q.w * z2;

        // Intrinsic YZX: extract from Ry * Rz * Rx
        rad3<T> angles;
        const T sz = xy2 - wz2;
        angles[2] = rad<T>(fastasin(-sz));
        if (std::abs(sz) < static_cast<T>(0.99999999999)) {
            angles[0] = rad<T>(std::atan2(yz2 + wx2, static_cast<T>(1.0) - (xx2 + zz2)));
            angles[1] = rad<T>(std::atan2(xz2 + wy2, static_cast<T>(1.0) - (yy2 + zz2)));
        } else {
            angles[0] = rad<T>(std::atan2(wx2 - yz2, static_cast<T>(1.0) - (xx2 + yy2)));
            angles[1] = rad<T>(static_cast<T>(0.0));
        }
        // Apply per-axis sign to match rotation direction convention
        angles[0].value *= static_cast<T>(signs.x);
        angles[1].value *= static_cast<T>(signs.y);
        angles[2].value *= static_cast<T>(signs.z);
        return angles;
    }
};

template<typename T>
struct quat_to_euler<T, rot_seq::zxy> {

    rad3<T> operator()(const quat<T>& q, rot_sign signs) {
        const T x2 = q.x + q.x;
        const T y2 = q.y + q.y;
        const T z2 = q.z + q.z;

        const T xx2 = q.x * x2;
        const T xy2 = q.x * y2;
        const T xz2 = q.x * z2;

        const T yy2 = q.y * y2;
        const T yz2 = q.y * z2;

        const T zz2 = q.z * z2;

        const T wx2 = q.w * x2;
        const T wy2 = q.w * y2;
        const T wz2 = q.w * z2;

        // Intrinsic ZXY: extract from Rz * Rx * Ry
        rad3<T> angles;
        const T sx = yz2 - wx2;
        angles[0] = rad<T>(fastasin(-sx));
        if (std::abs(sx) < static_cast<T>(0.99999999999)) {
            angles[1] = rad<T>(std::atan2(xz2 + wy2, static_cast<T>(1.0) - (xx2 + yy2)));
            angles[2] = rad<T>(std::atan2(xy2 + wz2, static_cast<T>(1.0) - (xx2 + zz2)));
        } else {
            angles[1] = rad<T>(std::atan2(wy2 - xz2, static_cast<T>(1.0) - (yy2 + zz2)));
            angles[2] = rad<T>(static_cast<T>(0.0));
        }
        // Apply per-axis sign to match rotation direction convention
        angles[0].value *= static_cast<T>(signs.x);
        angles[1].value *= static_cast<T>(signs.y);
        angles[2].value *= static_cast<T>(signs.z);
        return angles;
    }
};

template<typename T>
struct quat_to_euler<T, rot_seq::zyx> {

    rad3<T> operator()(const quat<T>& q, rot_sign signs) {
        const T x2 = q.x + q.x;
        const T y2 = q.y + q.y;
        const T z2 = q.z + q.z;

        const T xx2 = q.x * x2;
        const T xy2 = q.x * y2;
        const T xz2 = q.x * z2;

        const T yy2 = q.y * y2;
        const T yz2 = q.y * z2;

        const T zz2 = q.z * z2;

        const T wx2 = q.w * x2;
        const T wy2 = q.w * y2;
        const T wz2 = q.w * z2;

        // Intrinsic ZYX: extract from Rz * Ry * Rx
        rad3<T> angles;
        const T sy = xz2 + wy2;
        angles[1] = rad<T>(fastasin(sy));
        if (std::abs(sy) < static_cast<T>(0.99999999999)) {
            angles[0] = rad<T>(std::atan2(wx2 - yz2, static_cast<T>(1.0) - (xx2 + yy2)));
            angles[2] = rad<T>(std::atan2(wz2 - xy2, static_cast<T>(1.0) - (yy2 + zz2)));
        } else {
            angles[0] = rad<T>(std::atan2(yz2 + wx2, static_cast<T>(1.0) - (xx2 + zz2)));
            angles[2] = rad<T>(static_cast<T>(0.0));
        }
        // Apply per-axis sign to match rotation direction convention
        angles[0].value *= static_cast<T>(signs.x);
        angles[1].value *= static_cast<T>(signs.y);
        angles[2].value *= static_cast<T>(signs.z);
        return angles;
    }
};

template<typename T>
quat<T> euler2quat(const rad3<T>& rot, rot_seq order, rot_sign signs) {
    switch (order) {
    case rot_seq::xyz:
        return euler_to_quat<T, rot_seq::xyz>()(rot, signs);
    case rot_seq::xzy:
        return euler_to_quat<T, rot_seq::xzy>()(rot, signs);
    case rot_seq::yxz:
        return euler_to_quat<T, rot_seq::yxz>()(rot, signs);
    case rot_seq::yzx:
        return euler_to_quat<T, rot_seq::yzx>()(rot, signs);
    case rot_seq::zxy:
        return euler_to_quat<T, rot_seq::zxy>()(rot, signs);
    case rot_seq::zyx:
        return euler_to_quat<T, rot_seq::zyx>()(rot, signs);
    }
    return {};
}

template<typename T>
rad3<T> quat2euler(const quat<T>& q, rot_seq order, rot_sign signs) {
    switch (order) {
    case rot_seq::xyz:
        return quat_to_euler<T, rot_seq::xyz>()(q, signs);
    case rot_seq::xzy:
        return quat_to_euler<T, rot_seq::xzy>()(q, signs);
    case rot_seq::yxz:
        return quat_to_euler<T, rot_seq::yxz>()(q, signs);
    case rot_seq::yzx:
        return quat_to_euler<T, rot_seq::yzx>()(q, signs);
    case rot_seq::zxy:
        return quat_to_euler<T, rot_seq::zxy>()(q, signs);
    case rot_seq::zyx:
        return quat_to_euler<T, rot_seq::zyx>()(q, signs);
    }
    return {};
}

}  // namespace impl

template<typename T>
struct quat {
    using value_type = T;

    value_type x;
    value_type y;
    value_type z;
    value_type w;

    quat() :
        x{},
        y{},
        z{},
        w{static_cast<value_type>(1.0)} {
    }

    quat(value_type x_, value_type y_, value_type z_, value_type w_) :
        x{x_},
        y{y_},
        z{z_},
        w{w_} {
    }

    template<rot_seq order>
    static quat<T> from_euler(const rad3<T>& rot, rot_sign signs) {
        return impl::euler_to_quat<value_type, order>()(rot, signs);
    }

    explicit quat(const rad3<value_type>& rot, rot_seq order, rot_sign signs) :
        quat{impl::euler2quat(rot, order, signs)} {
    }

    quat& operator+=(const quat& rhs) {
        x += rhs.x;
        y += rhs.y;
        z += rhs.z;
        w += rhs.w;
        return *this;
    }

    quat& operator-=(const quat& rhs) {
        x -= rhs.x;
        y -= rhs.y;
        z -= rhs.z;
        w -= rhs.w;
        return *this;
    }

    quat& operator*=(const quat& rhs) {
        const quat lhs{*this};
        x = lhs.w * rhs.x + lhs.x * rhs.w + lhs.y * rhs.z - lhs.z * rhs.y;
        y = lhs.w * rhs.y + lhs.y * rhs.w + lhs.z * rhs.x - lhs.x * rhs.z;
        z = lhs.w * rhs.z + lhs.z * rhs.w + lhs.x * rhs.y - lhs.y * rhs.x;
        w = lhs.w * rhs.w - lhs.x * rhs.x - lhs.y * rhs.y - lhs.z * rhs.z;
        return *this;
    }

    quat& operator*=(value_type val) {
        x *= val;
        y *= val;
        z *= val;
        w *= val;
        return *this;
    }

    quat& operator/=(value_type val) {
        x /= val;
        y /= val;
        z /= val;
        w /= val;
        return *this;
    }

    quat operator-() const {
        return quat{-x, -y, -z, -w};
    }

    template<rot_seq order>
    rad3<value_type> euler(rot_sign signs) const {
        return impl::quat_to_euler<value_type, order>()(*this, signs);
    }

    rad3<value_type> euler(rot_seq order, rot_sign signs) const {
        return impl::quat2euler(*this, order, signs);
    }

    quat& negate() {
        x = -x;
        y = -y;
        z = -z;
        w = -w;
        return *this;
    }

    value_type length2() const {
        return x * x + y * y + z * z + w * w;
    }

    value_type length() const {
        return std::sqrt(length2());
    }

    quat& normalize() {
        const value_type il = static_cast<T>(1.0) / length();
        x *= il;
        y *= il;
        z *= il;
        w *= il;
        return *this;
    }
};

template<typename T>
inline bool operator==(const quat<T>& lhs, const quat<T>& rhs) {
    return (lhs.x == rhs.x) && (lhs.y == rhs.y) && (lhs.z == rhs.z) && (lhs.w == rhs.w);
}

template<typename T>
inline bool operator!=(const quat<T>& lhs, const quat<T>& rhs) {
    return !(lhs == rhs);
}

template<typename T>
inline quat<T> operator+(const quat<T>& lhs, const quat<T>& rhs) {
    return quat<T>(lhs) += rhs;
}

template<typename T>
inline quat<T> operator-(const quat<T>& lhs, const quat<T>& rhs) {
    return quat<T>(lhs) -= rhs;
}

template<typename T>
inline quat<T> operator*(const quat<T>& lhs, const quat<T>& rhs) {
    return quat<T>(lhs) *= rhs;
}

template<typename T>
inline quat<T> operator*(const quat<T>& lhs, T rhs) {
    return quat<T>(lhs) *= rhs;
}

template<typename T>
inline quat<T> operator*(T lhs, const quat<T>& rhs) {
    return rhs * lhs;
}

template<typename T>
inline quat<T> operator/(const quat<T>& lhs, T rhs) {
    return quat<T>(lhs) /= rhs;
}

template<typename T>
inline quat<T> operator/(T lhs, const quat<T>& rhs) {
    return rhs / lhs;
}

}  // namespace tdm
