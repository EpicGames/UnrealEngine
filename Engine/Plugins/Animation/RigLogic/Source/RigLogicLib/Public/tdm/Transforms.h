// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "tdm/Ang.h"
#include "tdm/Computations.h"
#include "tdm/CoordSys.h"

namespace tdm {

template<typename T>
inline typename std::enable_if<std::is_floating_point<T>::value, T>::type fastasin(T value) {
    constexpr T FASTASIN_HALF_PI = static_cast<T>(1.5707963050);
    // Clamp input to [-1,1].
    const bool nonnegative = (value >= static_cast<T>(0.0));
    value = std::abs(value);
    T omx = static_cast<T>(1.0) - value;
    if (omx < static_cast<T>(0.0)) {
        omx = static_cast<T>(0.0);
    }
    const T root = std::sqrt(omx);
    // 7-degree minimax approximation
    // clang-format off
    T result =
        ((((((-static_cast<T>(0.0012624911)
              * value + static_cast<T>(0.0066700901))
             * value - static_cast<T>(0.0170881256))
            * value + static_cast<T>(0.0308918810))
           * value - static_cast<T>(0.0501743046))
          * value + static_cast<T>(0.0889789874))
         * value - static_cast<T>(0.2145988016))
        * value + FASTASIN_HALF_PI;
    // clang-format on

    result *= root;  // acos(|x|)
    // acos(x) = pi - acos(-x) when x < 0, asin(x) = pi/2 - acos(x)
    return (nonnegative ? FASTASIN_HALF_PI - result : result - FASTASIN_HALF_PI);
}

namespace affine {

template<dim_t L, typename T>
inline mat<L, L, T> scale(const vec<L, T>& factors) {
    return mat<L, L, T>::diagonal(factors);
}

template<dim_t L, typename T>
inline mat<L, L, T> scale(const mat<L, L, T>& m, const vec<L, T>& factors) {
    return m * scale(factors);
}

template<dim_t L, typename T>
inline mat<L, L, T> scale(T factor) {
    return scale(vec<L, T>{factor});
}

template<dim_t L, typename T>
inline mat<L, L, T> scale(const mat<L, L, T>& m, T factor) {
    return scale(m, vec<L, T>{factor});
}

}  // namespace affine

namespace impl {

template<typename T>
inline mat3<T> rotx(rad<T> x, rot_dir dir) {
    const T sx = std::sin(x.value) * static_cast<T>(dir);
    const T cx = std::cos(x.value);
    mat3<T> m = mat3<T>::identity();
    m(1, 1) = cx;
    m(1, 2) = sx;
    m(2, 1) = -sx;
    m(2, 2) = cx;
    return m;
}

template<typename T>
inline mat3<T> roty(rad<T> y, rot_dir dir) {
    const T sy = std::sin(y.value) * static_cast<T>(dir);
    const T cy = std::cos(y.value);
    mat3<T> m = mat3<T>::identity();
    m(0, 0) = cy;
    m(0, 2) = -sy;
    m(2, 0) = sy;
    m(2, 2) = cy;
    return m;
}

template<typename T>
inline mat3<T> rotz(rad<T> z, rot_dir dir) {
    const T sz = std::sin(z.value) * static_cast<T>(dir);
    const T cz = std::cos(z.value);
    mat3<T> m = mat3<T>::identity();
    m(0, 0) = cz;
    m(0, 1) = sz;
    m(1, 0) = -sz;
    m(1, 1) = cz;
    return m;
}

template<typename T, rot_seq order>
struct rot_mat;

template<typename T>
struct rot_mat<T, rot_seq::xyz> {
    mat3<T> operator()(mat3<T> x, mat3<T> y, mat3<T> z) {
        return x * y * z;
    }
};

template<typename T>
struct rot_mat<T, rot_seq::xzy> {
    mat3<T> operator()(mat3<T> x, mat3<T> y, mat3<T> z) {
        return x * z * y;
    }
};

template<typename T>
struct rot_mat<T, rot_seq::yxz> {
    mat3<T> operator()(mat3<T> x, mat3<T> y, mat3<T> z) {
        return y * x * z;
    }
};

template<typename T>
struct rot_mat<T, rot_seq::yzx> {
    mat3<T> operator()(mat3<T> x, mat3<T> y, mat3<T> z) {
        return y * z * x;
    }
};

template<typename T>
struct rot_mat<T, rot_seq::zxy> {
    mat3<T> operator()(mat3<T> x, mat3<T> y, mat3<T> z) {
        return z * x * y;
    }
};

template<typename T>
struct rot_mat<T, rot_seq::zyx> {
    mat3<T> operator()(mat3<T> x, mat3<T> y, mat3<T> z) {
        return z * y * x;
    }
};

template<typename T, rot_seq order>
inline mat3<T> euler_to_mat(const rad3<T>& euler, rot_sign signs) {
    const mat3<T> rx = rotx(euler[0], signs.x);
    const mat3<T> ry = roty(euler[1], signs.y);
    const mat3<T> rz = rotz(euler[2], signs.z);
    return rot_mat<T, order>()(rx, ry, rz);
}

template<typename T>
inline mat3<T> euler2mat(const rad3<T>& euler, rot_seq seq, rot_sign signs) {
    switch (seq) {
    case rot_seq::xyz:
        return euler_to_mat<T, rot_seq::xyz>(euler, signs);
    case rot_seq::xzy:
        return euler_to_mat<T, rot_seq::xzy>(euler, signs);
    case rot_seq::yxz:
        return euler_to_mat<T, rot_seq::yxz>(euler, signs);
    case rot_seq::yzx:
        return euler_to_mat<T, rot_seq::yzx>(euler, signs);
    case rot_seq::zxy:
        return euler_to_mat<T, rot_seq::zxy>(euler, signs);
    case rot_seq::zyx:
        return euler_to_mat<T, rot_seq::zyx>(euler, signs);
    }
    return mat3<T>{};
}

template<typename T, rot_seq order>
struct mat_to_euler;

// Row-major XYZ intrinsic (R = Rx * Ry * Rz):
template<typename T>
struct mat_to_euler<T, rot_seq::xyz> {
    rad3<T> operator()(const mat3<T>& m, rot_sign signs) {
        rad3<T> result;
        // R[0][2] = -sin(y)
        // R[1][2] = sin(x)*cos(y)
        // R[2][2] = cos(x)*cos(y)
        // R[0][0] = cos(y)*cos(z)
        // R[0][1] = cos(y)*sin(z)
        const T sy = m(0, 2);

        if (sy < static_cast<T>(1)) {
            if (sy > static_cast<T>(-1)) {
                // Normal case: cos(y) != 0
                result[0] = rad<T>{std::atan2(m(1, 2), m(2, 2))};  // x = atan2(sin(x)*cos(y), cos(x)*cos(y))
                result[1] = rad<T>{std::asin(-sy)};                // y = asin(-(-sin(y))) = asin(sin(y)) - wait, sy = -sin(y)?
                result[2] = rad<T>{std::atan2(m(0, 1), m(0, 0))};  // z = atan2(cos(y)*sin(z), cos(y)*cos(z))
            } else {
                // Gimbal lock: sy == -1, means sin(y) = 1, y = 90 degrees
                result[0] = rad<T>{std::atan2(-m(2, 1), m(1, 1))};
                result[1] = rad<T>{static_cast<T>(0.5) * static_cast<T>(pi())};
                result[2] = rad<T>{static_cast<T>(0)};
            }
        } else {
            // Gimbal lock: sy == 1, means sin(y) = -1, y = -90 degrees
            result[0] = rad<T>{std::atan2(-m(2, 1), m(1, 1))};
            result[1] = rad<T>{static_cast<T>(-0.5) * static_cast<T>(pi())};
            result[2] = rad<T>{static_cast<T>(0)};
        }

        result[0].value *= static_cast<T>(signs.x);
        result[1].value *= static_cast<T>(signs.y);
        result[2].value *= static_cast<T>(signs.z);
        return result;
    }
};

// Row major XZY intrinsic (R = Rx * Rz * Ry)
template<typename T>
struct mat_to_euler<T, rot_seq::xzy> {
    rad3<T> operator()(const mat3<T>& m, rot_sign signs) {
        rad3<T> result;
        // R(0,1) = sin(z), gimbal lock when cos(z) = 0
        const T sz = m(0, 1);

        if (sz < static_cast<T>(1)) {
            if (sz > static_cast<T>(-1)) {
                result[0] = rad<T>{std::atan2(-m(2, 1), m(1, 1))};
                result[1] = rad<T>{std::atan2(-m(0, 2), m(0, 0))};
                result[2] = rad<T>{std::asin(sz)};
            } else {
                // Gimbal lock: sz == -1, z = -90 degrees
                result[0] = rad<T>{-std::atan2(m(2, 0), m(2, 2))};
                result[1] = rad<T>{static_cast<T>(0)};
                result[2] = rad<T>{static_cast<T>(-0.5) * static_cast<T>(pi())};
            }
        } else {
            // Gimbal lock: sz == 1, z = 90 degrees
            result[0] = rad<T>{std::atan2(m(2, 0), m(2, 2))};
            result[1] = rad<T>{static_cast<T>(0)};
            result[2] = rad<T>{static_cast<T>(0.5) * static_cast<T>(pi())};
        }

        result[0].value *= static_cast<T>(signs.x);
        result[1].value *= static_cast<T>(signs.y);
        result[2].value *= static_cast<T>(signs.z);
        return result;
    }
};

// Row major YXZ intrinsic (R = Ry * Rx * Rz)
template<typename T>
struct mat_to_euler<T, rot_seq::yxz> {
    rad3<T> operator()(const mat3<T>& m, rot_sign signs) {
        rad3<T> result;
        // R(1,2) = sin(x), gimbal lock when cos(x) = 0
        // R(0,2) = -sy*cx, R(2,2) = cy*cx, so y = atan2(-R(0,2), R(2,2))
        // R(1,0) = -cx*sz, R(1,1) = cx*cz, so z = atan2(-R(1,0), R(1,1))
        const T sx = m(1, 2);

        if (sx < static_cast<T>(1)) {
            if (sx > static_cast<T>(-1)) {
                result[0] = rad<T>{std::asin(sx)};
                result[1] = rad<T>{std::atan2(-m(0, 2), m(2, 2))};
                result[2] = rad<T>{std::atan2(-m(1, 0), m(1, 1))};
            } else {
                // Gimbal lock: sx == -1, x = -90 degrees
                result[0] = rad<T>{static_cast<T>(-0.5) * static_cast<T>(pi())};
                result[1] = rad<T>{-std::atan2(m(0, 1), m(0, 0))};
                result[2] = rad<T>{static_cast<T>(0)};
            }
        } else {
            // Gimbal lock: sx == 1, x = 90 degrees
            result[0] = rad<T>{static_cast<T>(0.5) * static_cast<T>(pi())};
            result[1] = rad<T>{std::atan2(m(0, 1), m(0, 0))};
            result[2] = rad<T>{static_cast<T>(0)};
        }

        result[0].value *= static_cast<T>(signs.x);
        result[1].value *= static_cast<T>(signs.y);
        result[2].value *= static_cast<T>(signs.z);
        return result;
    }
};

// Row major YZX intrinsic (R = Ry * Rz * Rx)
template<typename T>
struct mat_to_euler<T, rot_seq::yzx> {
    rad3<T> operator()(const mat3<T>& m, rot_sign signs) {
        rad3<T> result;
        // R(1,0) = -sin(z), gimbal lock when cos(z) = 0
        const T sz = m(1, 0);

        if (sz < static_cast<T>(1)) {
            if (sz > static_cast<T>(-1)) {
                result[0] = rad<T>{std::atan2(m(1, 2), m(1, 1))};
                result[1] = rad<T>{std::atan2(m(2, 0), m(0, 0))};
                result[2] = rad<T>{std::asin(-sz)};
            } else {
                // Gimbal lock: sz == -1, z = 90 degrees
                result[0] = rad<T>{static_cast<T>(0)};
                result[1] = rad<T>{std::atan2(-m(0, 2), m(2, 2))};
                result[2] = rad<T>{static_cast<T>(0.5) * static_cast<T>(pi())};
            }
        } else {
            // Gimbal lock: sz == 1, z = -90 degrees
            result[0] = rad<T>{static_cast<T>(0)};
            result[1] = rad<T>{std::atan2(-m(0, 2), m(2, 2))};
            result[2] = rad<T>{static_cast<T>(-0.5) * static_cast<T>(pi())};
        }

        result[0].value *= static_cast<T>(signs.x);
        result[1].value *= static_cast<T>(signs.y);
        result[2].value *= static_cast<T>(signs.z);
        return result;
    }
};

// Row major ZXY intrinsic (R = Rz * Rx * Ry)
template<typename T>
struct mat_to_euler<T, rot_seq::zxy> {
    rad3<T> operator()(const mat3<T>& m, rot_sign signs) {
        rad3<T> result;
        // R(2,1) = -sin(x), gimbal lock when cos(x) = 0
        const T sx = m(2, 1);

        if (sx < static_cast<T>(1)) {
            if (sx > static_cast<T>(-1)) {
                result[0] = rad<T>{std::asin(-sx)};
                result[1] = rad<T>{std::atan2(m(2, 0), m(2, 2))};
                result[2] = rad<T>{std::atan2(m(0, 1), m(1, 1))};
            } else {
                // Gimbal lock: sx == -1, x = 90 degrees
                result[0] = rad<T>{static_cast<T>(0.5) * static_cast<T>(pi())};
                result[1] = rad<T>{static_cast<T>(0)};
                result[2] = rad<T>{-std::atan2(-m(0, 2), m(0, 0))};
            }
        } else {
            // Gimbal lock: sx == 1, x = -90 degrees
            result[0] = rad<T>{static_cast<T>(-0.5) * static_cast<T>(pi())};
            result[1] = rad<T>{static_cast<T>(0)};
            result[2] = rad<T>{std::atan2(-m(0, 2), m(0, 0))};
        }

        result[0].value *= static_cast<T>(signs.x);
        result[1].value *= static_cast<T>(signs.y);
        result[2].value *= static_cast<T>(signs.z);
        return result;
    }
};

// Row major ZYX intrinsic (R = Rz * Ry * Rx)
template<typename T>
struct mat_to_euler<T, rot_seq::zyx> {
    rad3<T> operator()(const mat3<T>& m, rot_sign signs) {
        rad3<T> result;
        // R(2,0) = sin(y), gimbal lock when cos(y) = 0
        const T sy = m(2, 0);

        if (sy < static_cast<T>(1)) {
            if (sy > static_cast<T>(-1)) {
                result[0] = rad<T>{std::atan2(-m(2, 1), m(2, 2))};
                result[1] = rad<T>{std::asin(sy)};
                result[2] = rad<T>{std::atan2(-m(1, 0), m(0, 0))};
            } else {
                // Gimbal lock: sy == -1, y = -90 degrees
                result[0] = rad<T>{-std::atan2(m(0, 1), m(1, 1))};
                result[1] = rad<T>{static_cast<T>(-0.5) * static_cast<T>(pi())};
                result[2] = rad<T>{static_cast<T>(0)};
            }
        } else {
            // Gimbal lock: sy == 1, y = 90 degrees
            result[0] = rad<T>{std::atan2(m(0, 1), m(1, 1))};
            result[1] = rad<T>{static_cast<T>(0.5) * static_cast<T>(pi())};
            result[2] = rad<T>{static_cast<T>(0)};
        }

        result[0].value *= static_cast<T>(signs.x);
        result[1].value *= static_cast<T>(signs.y);
        result[2].value *= static_cast<T>(signs.z);
        return result;
    }
};

// Dispatcher function for Euler extraction.
// The rot_sign parameter indicates the rotation direction convention per axis.
// Negative rotation direction means the sine terms were negated when building the matrix.
// The sign is applied in the mat_to_euler structs to match the original convention.
template<typename T>
inline rad3<T> mat2euler(const mat3<T>& m, rot_seq seq, rot_sign signs) {
    switch (seq) {
    case rot_seq::xyz:
        return mat_to_euler<T, rot_seq::xyz>()(m, signs);
    case rot_seq::xzy:
        return mat_to_euler<T, rot_seq::xzy>()(m, signs);
    case rot_seq::yxz:
        return mat_to_euler<T, rot_seq::yxz>()(m, signs);
    case rot_seq::yzx:
        return mat_to_euler<T, rot_seq::yzx>()(m, signs);
    case rot_seq::zxy:
        return mat_to_euler<T, rot_seq::zxy>()(m, signs);
    case rot_seq::zyx:
        return mat_to_euler<T, rot_seq::zyx>()(m, signs);
    }
    return rad3<T>{};
}

}  // namespace impl

inline namespace projective {

template<typename T>
inline mat4<T> rotate(const vec3<T>& axis, rad<T> angle, rot_dir dir) {
    const rad<T> c{std::cos(angle.value)};
    const rad<T> s{std::sin(angle.value) * static_cast<T>(dir)};
    const rad<T> one_minus_c = rad<T>{static_cast<T>(1)} - c;
    const vec3<T> n = normalize(axis);
    return mat4<T>{n[0] * n[0] * one_minus_c.value + c.value,
                   n[1] * n[0] * one_minus_c.value - n[2] * s.value,
                   n[2] * n[0] * one_minus_c.value + n[1] * s.value,
                   static_cast<T>(0),
                   n[0] * n[1] * one_minus_c.value + n[2] * s.value,
                   n[1] * n[1] * one_minus_c.value + c.value,
                   n[2] * n[1] * one_minus_c.value - n[0] * s.value,
                   static_cast<T>(0),
                   n[0] * n[2] * one_minus_c.value - n[1] * s.value,
                   n[1] * n[2] * one_minus_c.value + n[0] * s.value,
                   n[2] * n[2] * one_minus_c.value + c.value,
                   static_cast<T>(0),
                   static_cast<T>(0),
                   static_cast<T>(0),
                   static_cast<T>(0),
                   static_cast<T>(1)};
}

template<typename T>
inline mat4<T> rotate(const mat4<T>& m, const vec3<T>& axis, rad<T> angle, rot_dir dir) {
    return m * rotate(axis, angle, dir);
}

template<typename T>
inline mat4<T> rotate(rad<T> x, rad<T> y, rad<T> z, rot_seq order, rot_sign signs) {
    const mat3<T> mat3x3 = impl::euler2mat<T>(rad3<T>{x, y, z}, order, signs);
    mat4<T> m = mat4<T>::identity();
    for (dim_t ri = 0; ri < mat3x3.rows(); ++ri) {
        for (dim_t ci = 0; ci < mat3x3.columns(); ++ci) {
            m[ri][ci] = mat3x3[ri][ci];
        }
    }
    return m;
}

template<typename T>
inline mat4<T> rotate(const mat4<T>& m, rad<T> x, rad<T> y, rad<T> z, rot_seq order, rot_sign signs) {
    return m * rotate(x, y, z, order, signs);
}

template<typename T>
inline mat4<T> rotate(const rad3<T>& rotation, rot_seq order, rot_sign signs) {
    return rotate(rotation[0], rotation[1], rotation[2], order, signs);
}

template<typename T>
inline mat4<T> rotate(const mat4<T>& m, const rad3<T>& rotation, rot_seq order, rot_sign signs) {
    return m * rotate(rotation[0], rotation[1], rotation[2], order, signs);
}

template<dim_t L, typename T>
inline mat<L + 1, L + 1, T> scale(const vec<L, T>& factors) {
    vec<L + 1, T> diagonal{static_cast<T>(1)};
    factors.apply([&diagonal](const T& value, dim_t i) { diagonal[i] = value; });
    return mat<L + 1, L + 1, T>::diagonal(diagonal);
}

template<dim_t L, typename T>
inline mat<L + 1, L + 1, T> scale(const mat<L + 1, L + 1, T>& m, const vec<L, T>& factors) {
    return m * scale(factors);
}

template<dim_t L, typename T>
inline mat<L + 1, L + 1, T> scale(T factor) {
    return scale(vec<L, T>{factor});
}

template<dim_t L, typename T>
inline mat<L, L, T> scale(const mat<L, L, T>& m, T factor) {
    return scale(m, vec<L - 1, T>{factor});
}

template<dim_t L, typename T>
inline mat<L + 1, L + 1, T> translate(const vec<L, T>& position) {
    auto m = mat<L + 1, L + 1, T>::identity();
    position.apply([&m](const T& value, dim_t i) { m(L, i) = value; });
    return m;
}

template<dim_t L, typename T>
inline mat<L + 1, L + 1, T> translate(const mat<L + 1, L + 1, T>& m, const vec<L, T>& position) {
    return m * translate(position);
}

}  // namespace projective

template<typename T>
inline mat3<T> change_of_basis(const coord_sys& src, const coord_sys& dst) {
    // Row-vectors: v_dst = v_src * C
    // with C = B_src * B_dst^T  (since B_dst^{-1} = B_dst^T for orthonormal bases)
    return src.basis<T>() * transpose(dst.basis<T>());
}

// Convert a position or translation vector between coordinate systems.
// Both positions and translations are transformed by the change-of-basis matrix.
template<typename T>
inline vec3<T> convert_position(const vec3<T>& pos, const mat3<T>& c) {
    return pos * c;
}

template<typename T>
inline vec3<T> convert_position(const vec3<T>& pos, const coord_sys& src_cs, const coord_sys& dst_cs) {
    return convert_position(pos, change_of_basis<T>(src_cs, dst_cs));
}

// Convert a direction vector (normal, tangent) between coordinate systems.
// Optionally renormalizes the result (default: true).
template<typename T>
inline vec3<T> convert_direction(const vec3<T>& dir, const mat3<T>& c, bool renormalize = true) {
    const vec3<T> d = dir * c;
    return renormalize ? normalize(d) : d;
}

template<typename T>
inline vec3<T> convert_direction(const vec3<T>& dir, const coord_sys& src_cs, const coord_sys& dst_cs, bool renormalize = true) {
    return convert_direction(dir, change_of_basis<T>(src_cs, dst_cs), renormalize);
}

// Convert a scale vector between coordinate systems.
// Scale is per-axis and doesn't have direction, so we take absolute values
// to ignore sign flips from the coordinate system transformation.
template<typename T>
inline vec3<T> convert_scale(const vec3<T>& scale, const mat3<T>& c) {
    const vec3<T> result = scale * c;
    return vec3<T>{std::fabs(result[0]), std::fabs(result[1]), std::fabs(result[2])};
}

template<typename T>
inline vec3<T> convert_scale(const vec3<T>& scale, const coord_sys& src_cs, const coord_sys& dst_cs) {
    return convert_scale(scale, change_of_basis<T>(src_cs, dst_cs));
}

template<typename T>
inline rad3<T> convert_rotation(const rad3<T>& rotation,
                                const mat3<T>& c,
                                rot_seq src_seq,
                                const rot_sign& src_signs,
                                rot_seq dst_seq,
                                const rot_sign& dst_signs) {
    // Build source rotation matrix with src_signs applied per-axis.
    const mat3<T> r_src = impl::euler2mat<T>(rotation, src_seq, src_signs);
    // Basis-change into destination coordinate system.
    const mat3<T> r_dst = transpose(c) * r_src * c;
    // Extract Euler angles with dst_signs applied per-axis.
    return impl::mat2euler<T>(r_dst, dst_seq, dst_signs);
}

template<typename T>
inline rad3<T> convert_rotation(const rad3<T>& rotation,
                                const coord_sys& src_cs,
                                rot_seq src_seq,
                                const rot_sign& src_signs,
                                const coord_sys& dst_cs,
                                rot_seq dst_seq,
                                const rot_sign& dst_signs) {
    return convert_rotation(rotation, change_of_basis<T>(src_cs, dst_cs), src_seq, src_signs, dst_seq, dst_signs);
}

}  // namespace tdm
