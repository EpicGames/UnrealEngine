// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cmath>
#include <cstddef>
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace tdm {

// Scalar types

using dim_t = std::size_t;

// Vector types

template<dim_t L, typename T>
struct vec;

template<typename T>
using vec2 = vec<2, T>;

template<typename T>
using vec3 = vec<3, T>;

template<typename T>
using vec4 = vec<4, T>;

// Common vector types

template<dim_t L>
using ivec = vec<L, int>;

using ivec2 = ivec<2>;
using ivec3 = ivec<3>;
using ivec4 = ivec<4>;

template<dim_t L>
using fvec = vec<L, float>;

using fvec2 = fvec<2>;
using fvec3 = fvec<3>;
using fvec4 = fvec<4>;

// Matrix types

template<dim_t R, dim_t C, typename T>
struct mat;

template<typename T>
using mat2 = mat<2, 2, T>;

template<typename T>
using mat3 = mat<3, 3, T>;

template<typename T>
using mat4 = mat<4, 4, T>;

// Common matrix types

template<dim_t R, dim_t C>
using imat = mat<R, C, int>;

using imat2 = imat<2, 2>;
using imat3 = imat<3, 3>;
using imat4 = imat<4, 4>;

template<dim_t R, dim_t C>
using fmat = mat<R, C, float>;

using fmat2 = fmat<2, 2>;
using fmat3 = fmat<3, 3>;
using fmat4 = fmat<4, 4>;

// Angle types
template<typename T, typename TUnit>
struct ang;

namespace impl {

struct radians;
struct degrees;

}  // namespace impl

template<typename T>
using deg = ang<T, impl::degrees>;

template<typename T>
using rad = ang<T, impl::radians>;

template<typename T>
using deg3 = vec3<deg<T>>;

template<typename T>
using rad3 = vec3<rad<T>>;

using fdeg = deg<float>;
using fdeg3 = deg3<float>;
using frad = rad<float>;
using frad3 = rad3<float>;

// Quaternion types

template<typename T>
struct quat;

using fquat = quat<float>;

enum class chirality {
    left = -1,
    right = 1
};

enum class rot_seq {
    xyz,
    xzy,
    yxz,
    yzx,
    zxy,
    zyx
};

enum class rot_dir {
    negative = -1,
    positive = 1
};

struct rot_sign {
    rot_dir x;
    rot_dir y;
    rot_dir z;
};

enum class axis_dir {
    left,
    right,
    up,
    down,
    front,
    back
};

struct coord_sys;

}  // namespace tdm
