// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "tdm/Types.h"

namespace tdm {

template<dim_t L, typename T>
struct vec {
    using value_type = T;

    static constexpr dim_t dimensions() {
        return L;
    }

    value_type values[L];

    vec() :
        values{} {
    }

    ~vec() = default;

    vec(const vec& rhs) = default;
    vec& operator=(const vec& rhs) = default;

    vec(vec&& rhs) = default;
    vec& operator=(vec&& rhs) = default;

    template<typename U>
    vec(const vec<L, U>& rhs) {
        std::copy(std::begin(rhs.values), std::end(rhs.values), std::begin(values));
    }

    template<typename U>
    vec& operator=(const vec<L, U>& rhs) {
        std::copy(std::begin(rhs.values), std::end(rhs.values), std::begin(values));
        return *this;
    }

    template<typename... Vs, typename std::enable_if<(sizeof...(Vs) == L) || (sizeof...(Vs) == 0)>::type* = nullptr>
    vec(Vs... vs) :
        values{vs...} {
    }

    template<typename U, typename..., typename std::enable_if<std::is_convertible<U, T>::value && (L > 1)>::type* = nullptr>
    explicit vec(U v) {
        std::fill_n(values, dimensions(), v);
    }

    template<typename U, typename..., typename std::enable_if<std::is_convertible<U, T>::value>::type* = nullptr>
    explicit vec(U* pv) {
        std::copy_n(pv, dimensions(), std::begin(values));
    }

    T& operator[](dim_t index) {
        assert(index < dimensions());
        return values[index];
    }

    const T& operator[](dim_t index) const {
        assert(index < dimensions());
        return values[index];
    }

    template<typename F>
    vec& apply(F func) {
        for (dim_t i{}; i < dimensions(); ++i) {
            func(values[i], i);
        }
        return *this;
    }

    template<typename F>
    const vec& apply(F func) const {
        for (dim_t i{}; i < dimensions(); ++i) {
            func(values[i], i);
        }
        return *this;
    }

    vec& operator++() {
        return apply([](T& v, dim_t /*unused*/) { ++v; });
    }

    vec& operator--() {
        return apply([](T& v, dim_t /*unused*/) { --v; });
    }

    template<typename U>
    vec& operator+=(U rhs) {
        return apply([rhs](T& v, dim_t /*unused*/) { v += rhs; });
    }

    template<typename U>
    vec& operator+=(const vec<L, U>& rhs) {
        return apply([&rhs](T& v, dim_t i) { v += rhs[i]; });
    }

    vec& operator+=(const vec& rhs) {
        return operator+= <T>(rhs);
    }

    template<typename U>
    vec& operator-=(U rhs) {
        return apply([rhs](T& v, dim_t /*unused*/) { v -= rhs; });
    }

    template<typename U>
    vec& operator-=(const vec<L, U>& rhs) {
        return apply([&rhs](T& v, dim_t i) { v -= rhs[i]; });
    }

    vec& operator-=(const vec& rhs) {
        return operator-= <T>(rhs);
    }

    template<typename U>
    vec& operator*=(U rhs) {
        return apply([rhs](T& v, dim_t /*unused*/) { v *= rhs; });
    }

    template<typename U>
    vec& operator*=(const vec<L, U>& rhs) {
        return apply([&rhs](T& v, dim_t i) { v *= rhs[i]; });
    }

    vec& operator*=(const vec& rhs) {
        return operator*= <T>(rhs);
    }

    template<typename U>
    vec& operator/=(U rhs) {
        return apply([rhs](T& v, dim_t /*unused*/) { v /= rhs; });
    }

    template<typename U>
    vec& operator/=(const vec<L, U>& rhs) {
        return apply([&rhs](T& v, dim_t i) { v /= rhs[i]; });
    }

    vec& operator/=(const vec& rhs) {
        return operator/= <T>(rhs);
    }

    template<typename..., typename V = T>
    typename std::enable_if<std::is_floating_point<V>::value, V>::type length() const {
        const auto& v = *this;
        return std::sqrt((v * v).sum());
    }

    template<typename..., typename V = T>
    typename std::enable_if<std::is_floating_point<V>::value, vec&>::type normalize() {
        return operator/=(length());
    }

    vec& negate() {
        return apply([](T& v, dim_t /*unused*/) { v = -v; });
    }

    T sum() const {
        T retval{};
        apply([&retval](const T& v, dim_t /*unused*/) { retval += v; });
        return retval;
    }
};

template<dim_t L, typename T>
inline bool operator==(const vec<L, T>& lhs, const vec<L, T>& rhs) {
    bool equal = true;
    lhs.apply([&equal, &rhs](const T& v, dim_t i) { equal = equal && (v == rhs[i]); });
    return equal;
}

template<dim_t L, typename T>
inline bool operator!=(const vec<L, T>& lhs, const vec<L, T>& rhs) {
    return !(lhs == rhs);
}

template<dim_t L, typename T>
inline vec<L, T> operator+(const vec<L, T>& v) {
    return v;
}

template<dim_t L, typename T>
inline vec<L, T> operator-(vec<L, T> v) {
    return v.negate();
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator+(const vec<L, T>& lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) += rhs;
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator+(const vec<L, T>& lhs, U rhs) {
    return vec<L, T>(lhs) += rhs;
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator+(T lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) += rhs;
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator-(const vec<L, T>& lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) -= rhs;
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator-(const vec<L, T>& lhs, U rhs) {
    return vec<L, T>(lhs) -= rhs;
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator-(T lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) -= rhs;
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator*(const vec<L, T>& lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) *= rhs;
}

template<dim_t L, typename T, typename U>
inline typename std::enable_if<std::is_arithmetic<U>::value, vec<L, T>>::type operator*(const vec<L, T>& lhs, U rhs) {
    return vec<L, T>(lhs) *= rhs;
}

template<dim_t L, typename T, typename U>
inline typename std::enable_if<std::is_arithmetic<T>::value, vec<L, T>>::type operator*(T lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) *= rhs;
}

template<dim_t L, typename T, typename U>
inline vec<L, T> operator/(const vec<L, T>& lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) /= rhs;
}

template<dim_t L, typename T, typename U>
inline typename std::enable_if<std::is_arithmetic<U>::value, vec<L, T>>::type operator/(const vec<L, T>& lhs, U rhs) {
    return vec<L, T>(lhs) /= rhs;
}

template<dim_t L, typename T, typename U>
inline typename std::enable_if<std::is_arithmetic<T>::value, vec<L, T>>::type operator/(T lhs, const vec<L, U>& rhs) {
    return vec<L, T>(lhs) /= rhs;
}

template<dim_t L, typename T, typename F>
vec<L, T> applied(const vec<L, T>& lhs, F func) {
    vec<L, T> v{lhs};
    v.apply(func);
    return v;
}

}  // namespace tdm
