// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define TDM_MAJOR_VERSION 6
#define TDM_MINOR_VERSION 0
#define TDM_PATCH_VERSION 0
#define TDM_VERSION_STRING "6.0.0"
