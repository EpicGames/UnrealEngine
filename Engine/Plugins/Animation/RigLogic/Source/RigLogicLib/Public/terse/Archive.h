// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

template<class ArchiveImpl>
class Archive {
public:
    explicit Archive(ArchiveImpl* impl_) :
        impl{impl_},
        userData{nullptr} {
    }

    bool isOk() {
        return impl->isOk();
    }

    void sync() {
        impl->sync();
    }

    void label(const char* value) {
        impl->label(value);
    }

    template<typename... Args>
    void operator()(Args&&... args) {
        dispatch(std::forward<Args>(args)...);
    }

    template<class TSerializable>
    ArchiveImpl& operator<<(TSerializable& source) {
        dispatch(source);
        return *impl;
    }

    template<class TSerializable>
    ArchiveImpl& operator>>(TSerializable& dest) {
        dispatch(dest);
        return *impl;
    }

    void* getUserData() const {
        return userData;
    }

    void setUserData(void* data) {
        userData = data;
    }

protected:
    template<typename Head>
    void dispatch(Head&& head) {
        impl->process(std::forward<Head>(head));
    }

    template<typename Head, typename... Tail>
    void dispatch(Head&& head, Tail&&... tail) {
        dispatch(std::forward<Head>(head));
        dispatch(std::forward<Tail>(tail)...);
    }

private:
    ArchiveImpl* impl;
    void* userData;
};

}  // namespace terse
