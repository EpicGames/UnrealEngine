// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/archives/Traits.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <memory>
#include <tuple>
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace pma {

class MemoryResource;

}  // namespace pma

namespace terse {

namespace impl {

template<typename T>
struct ValueFactory {
    static constexpr bool NeedsAllocator = traits::needs_allocator<T>::value;
    static constexpr bool NeedsMemoryResource = std::is_constructible<T, pma::MemoryResource*>::value;
    static constexpr bool IsPair = traits::is_pair<T>::value;
    static constexpr bool IsTuple = traits::is_tuple<T>::value;
    static constexpr bool IsPrimitive = (!NeedsAllocator && !NeedsMemoryResource & !IsPair & !IsTuple);

    template<class ParentAllocator, bool IsPrimitive = IsPrimitive>
    static typename std::enable_if<IsPrimitive, T>::type create(const ParentAllocator& /*unused*/) {
        return T{};
    }

    template<class ParentAllocator, bool IsPrimitive = IsPrimitive>
    static typename std::enable_if<!IsPrimitive && NeedsMemoryResource, T>::type create(const ParentAllocator& alloc) {
        return T{alloc.getMemoryResource()};
    }

    template<class ParentAllocator, bool IsPrimitive = IsPrimitive>
    static typename std::enable_if<!IsPrimitive && !NeedsMemoryResource && NeedsAllocator, T>::type
    create(const ParentAllocator& alloc) {
        using TAllocator = typename std::allocator_traits<ParentAllocator>::template rebind_alloc<typename T::value_type>;
        return T{TAllocator{alloc}};
    }

    template<class ParentAllocator, bool IsPrimitive = IsPrimitive>
    static typename std::enable_if<!IsPrimitive && !NeedsMemoryResource && !NeedsAllocator && IsPair, T>::type
    create(const ParentAllocator& alloc) {
        using K = typename T::first_type;
        using V = typename T::second_type;
        return T{ValueFactory<K>::create(alloc), ValueFactory<V>::create(alloc)};
    }

    template<class ParentAllocator, bool IsPrimitive = IsPrimitive>
    static typename std::enable_if<!IsPrimitive && !NeedsMemoryResource && !NeedsAllocator && !IsPair && IsTuple, T>::type
    create(const ParentAllocator& alloc) {
        using K = typename std::tuple_element<0, T>::type;
        using V = typename std::tuple_element<0, T>::type;
        return T{ValueFactory<K>::create(alloc), ValueFactory<V>::create(alloc)};
    }
};

}  // namespace impl

}  // namespace terse
