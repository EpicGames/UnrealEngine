// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <type_traits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

static_assert(sizeof(char) == 1ul, "Unsupported platform, char is not 8-bits wide.");

namespace terse {

namespace traits {

template<typename...>
struct sink {
    using type = void;
};

template<typename T, typename = void>
struct needs_allocator : std::false_type {};

template<typename T>
struct needs_allocator<T, typename sink<typename T::allocator_type, decltype(std::declval<T>().get_allocator())>::type>
    : std::true_type {};

template<class>
struct true_sink : std::true_type {};

template<class TContainer>
static auto test_reserve_member(std::int32_t) -> true_sink<decltype(std::declval<TContainer>().reserve(0u))>;

template<class>
static auto test_reserve_member(std::uint32_t) -> std::false_type;

template<class T>
struct has_reserve_member : decltype(test_reserve_member<T>(0)) {};

template<class TContainer>
static auto test_push_back_member(std::int32_t)
    -> true_sink<decltype(std::declval<TContainer>().push_back(std::declval<typename TContainer::value_type>()))>;

template<class>
static auto test_push_back_member(std::uint32_t) -> std::false_type;

template<class T>
struct has_push_back_member : decltype(test_push_back_member<T>(0)) {};

// Serializer member functions

template<class T>
static auto test_serialize_member(std::int32_t) -> true_sink<decltype(std::declval<T>().serialize(std::declval<T&>()))>;

template<class>
static auto test_serialize_member(std::uint32_t) -> std::false_type;

template<class T>
struct has_serialize_member : decltype(test_serialize_member<T>(0)) {};

template<class T>
static auto test_load_member(std::int32_t) -> true_sink<decltype(std::declval<T>().load(std::declval<T&>()))>;

template<class>
static auto test_load_member(std::uint32_t) -> std::false_type;

template<class T>
struct has_load_member : decltype(test_load_member<T>(0)) {};

template<class T>
static auto test_save_member(std::int32_t) -> true_sink<decltype(std::declval<T>().save(std::declval<T&>()))>;

template<class>
static auto test_save_member(std::uint32_t) -> std::false_type;

template<class T>
struct has_save_member : decltype(test_save_member<T>(0)) {};

// Serializer free functions

template<class T>
static auto test_serialize_function(std::int32_t) -> true_sink<decltype(serialize(std::declval<T&>(), std::declval<T&>()))>;

template<class>
static auto test_serialize_function(std::uint32_t) -> std::false_type;

template<class T>
struct has_serialize_function : decltype(test_serialize_function<T>(0)) {};

template<class T>
static auto test_load_function(std::int32_t) -> true_sink<decltype(load(std::declval<T&>(), std::declval<T&>()))>;

template<class>
static auto test_load_function(std::uint32_t) -> std::false_type;

template<class T>
struct has_load_function : decltype(test_load_function<T>(0)) {};

template<class T>
static auto test_save_function(std::int32_t) -> true_sink<decltype(save(std::declval<T&>(), std::declval<T&>()))>;

template<class>
static auto test_save_function(std::uint32_t) -> std::false_type;

template<class T>
struct has_save_function : decltype(test_save_function<T>(0)) {};

// Versioned serializer member functions

template<class T, class V>
static auto test_versioned_serialize_member(std::int32_t)
    -> true_sink<decltype(std::declval<T>().serialize(std::declval<T&>(), V{}))>;

template<class, class>
static auto test_versioned_serialize_member(std::uint32_t) -> std::false_type;

template<class T, class V>
struct has_versioned_serialize_member : decltype(test_versioned_serialize_member<T, V>(0)) {};

template<class T, class V>
static auto test_versioned_load_member(std::int32_t) -> true_sink<decltype(std::declval<T>().load(std::declval<T&>(), V{}))>;

template<class, class>
static auto test_versioned_load_member(std::uint32_t) -> std::false_type;

template<class T, class V>
struct has_versioned_load_member : decltype(test_versioned_load_member<T, V>(0)) {};

template<class T, class V>
static auto test_versioned_save_member(std::int32_t) -> true_sink<decltype(std::declval<T>().save(std::declval<T&>(), V{}))>;

template<class, class>
static auto test_versioned_save_member(std::uint32_t) -> std::false_type;

template<class T, class V>
struct has_versioned_save_member : decltype(test_versioned_save_member<T, V>(0)) {};

// Versioned serializer free functions

template<class T, class V>
static auto test_versioned_serialize_function(std::int32_t)
    -> true_sink<decltype(serialize(std::declval<T&>(), V{}, std::declval<T&>()))>;

template<class, class>
static auto test_versioned_serialize_function(std::uint32_t) -> std::false_type;

template<class T, class V>
struct has_versioned_serialize_function : decltype(test_versioned_serialize_function<T, V>(0)) {};

template<class T, class V>
static auto test_versioned_load_function(std::int32_t) -> true_sink<decltype(load(std::declval<T&>(), V{}, std::declval<T&>()))>;

template<class, class>
static auto test_versioned_load_function(std::uint32_t) -> std::false_type;

template<class T, class V>
struct has_versioned_load_function : decltype(test_versioned_load_function<T, V>(0)) {};

template<class T, class V>
static auto test_versioned_save_function(std::int32_t) -> true_sink<decltype(save(std::declval<T&>(), V{}, std::declval<T&>()))>;

template<class, class>
static auto test_versioned_save_function(std::uint32_t) -> std::false_type;

template<class T, class V>
struct has_versioned_save_function : decltype(test_versioned_save_function<T, V>(0)) {};

// Heuristic traits

template<typename TContainer>
using is_batchable = std::is_scalar<typename TContainer::value_type>;

template<typename TContainer>
struct has_wide_elements {
    static constexpr bool value = (sizeof(typename TContainer::value_type) > 1ul);
};

template<typename T>
struct is_pair : public std::false_type {};

template<typename K, typename V>
struct is_pair<std::pair<K, V>> : public std::true_type {};

template<typename T>
struct is_tuple : public std::false_type {};

template<typename K, typename V>
struct is_tuple<std::tuple<K, V>> : public std::true_type {};

}  // namespace traits

}  // namespace terse
