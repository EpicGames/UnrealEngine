// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/Archive.h"
#include "terse/archives/Common.h"
#include "terse/archives/Traits.h"
#include "terse/types/Anchor.h"
#include "terse/types/ArchiveOffset.h"
#include "terse/types/ArchiveSize.h"
#include "terse/types/Blob.h"
#include "terse/types/DynArray.h"
#include "terse/types/Transparent.h"
#include "terse/types/Versioned.h"
#include "terse/utils/ByteSwap.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <memory>
#include <string>
#include <tuple>
#include <type_traits>
#include <utility>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

template<class TExtender, class TStream, typename TSize, typename TOffset, Endianness EByteOrder>
class ExtendableBinaryInputArchive : public Archive<TExtender> {
public:
    // Given the possibility of both 32 and 64bit platforms, use a fixed width type during serialization
    using SizeType = TSize;
    using OffsetType = TOffset;

    static constexpr Endianness endianness() {
        return EByteOrder;
    }

private:
    using BaseArchive = Archive<TExtender>;

public:
    ExtendableBinaryInputArchive(TExtender* extender, TStream* stream_) :
        BaseArchive{extender},
        stream{stream_} {
    }

    bool isOk() {
        return true;
    }

    void sync() {
    }

    void label(const char* /*unused*/) {
    }

    std::size_t boundSize(std::size_t size) {
        return size;
    }

protected:
    template<typename T>
    void process(Transparent<T>&& dest) {
        process(dest);
    }

    template<typename T>
    void process(Transparent<T>& dest) {
        process(dest.data);
    }

    void process(Anchor<OffsetType>& dest) {
        dest.value = static_cast<OffsetType>(stream->tell());
    }

    void process(ArchiveOffset<OffsetType>& dest) {
        // Load the offset value itself (this points forward within the stream to the position of
        // the data with which the offset is associated)
        process(dest.value);
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy& dest) {
        // Rely on the offset value stored in the associated `ArchiveOffset` and seek to it
        stream->seek(dest.target->value);
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy&& dest) {
        process(dest);
    }

    void process(ArchiveSize<SizeType, OffsetType>& dest) {
        // Load the size value itself (this value can be combined with an offset to get the end of a data region)
        process(dest.value);
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy& dest) {
        // Calculate the offset + size value stored in the associated `ArchiveSize` and `Anchor` and seek to it
        assert(dest.base != nullptr);
        stream->seek(dest.base->value + dest.target->value);
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_member<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_member<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        dest.data.load(*static_cast<TExtender*>(this), V{});
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        dest.data.serialize(*static_cast<TExtender*>(this), V{});
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_function<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_function<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        load(*static_cast<TExtender*>(this), V{}, dest.data);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        serialize(*static_cast<TExtender*>(this), V{}, dest.data);
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_load_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_load_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_load_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_load_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>& dest) {
        // If no versioned serializer is found, but version information was passed along, fall back to unversioned
        // functions if available.
        BaseArchive::dispatch(dest.data);
    }

    template<typename T>
    typename std::enable_if<traits::has_load_member<T>::value, void>::type process(T& dest) {
        dest.load(*static_cast<TExtender*>(this));
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_member<T>::value, void>::type process(T& dest) {
        dest.serialize(*static_cast<TExtender*>(this));
    }

    template<typename T>
    typename std::enable_if<traits::has_load_function<T>::value, void>::type process(T& dest) {
        load(*static_cast<TExtender*>(this), dest);
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_function<T>::value, void>::type process(T& dest) {
        serialize(*static_cast<TExtender*>(this), dest);
    }

    template<typename T>
    typename std::enable_if<!traits::has_load_member<T>::value && !traits::has_serialize_member<T>::value &&
                                !traits::has_load_function<T>::value && !traits::has_serialize_function<T>::value,
                            void>::type
    process(T& dest) {
        // NOLINTNEXTLINE(cppcoreguidelines-pro-type-reinterpret-cast)
        stream->read(reinterpret_cast<char*>(&dest), sizeof(T));
        SwapFrom<EByteOrder>::swap(dest);
    }

    template<typename T, typename... Args>
    void process(Blob<T, Args...>& dest) {
        using ValueType = typename Blob<T, Args...>::value_type;
        // Blob relies on a predetermined (user specified) size of how much data it should consume
        if (dest.size() != 0ul) {
            // NOLINTNEXTLINE(cppcoreguidelines-pro-type-reinterpret-cast)
            stream->read(reinterpret_cast<char*>(dest.data()), dest.size() * sizeof(ValueType));
        }
    }

    template<typename T, std::size_t N>
    void process(std::array<T, N>& dest) {
        for (auto& element : dest) {
            BaseArchive::dispatch(element);
        }
    }

    template<typename T, typename... Args>
    void process(std::vector<T, Args...>& dest) {
        const auto size = processSize();
        processElements(dest, size);
    }

    template<typename T, typename... Args>
    void process(DynArray<T, Args...>& dest) {
        const auto size = processSize();
        processElements(dest, size);
    }

    template<typename T, typename... Args>
    void process(std::basic_string<T, Args...>& dest) {
        const auto size = processSize();
        processElements(dest, size);
    }

    template<typename K, typename V>
    void process(std::pair<K, V>& dest) {
        BaseArchive::dispatch(dest.first);
        BaseArchive::dispatch(dest.second);
    }

    template<typename K, typename V>
    void process(std::tuple<K, V>& dest) {
        BaseArchive::dispatch(std::get<0>(dest));
        BaseArchive::dispatch(std::get<1>(dest));
    }

    std::size_t processSize() {
        SizeType size{};
        process(size);
        return static_cast<TExtender*>(this)->boundSize(static_cast<std::size_t>(size));
    }

    template<class TContainer>
    typename std::enable_if<!traits::is_batchable<TContainer>::value && traits::has_reserve_member<TContainer>::value &&
                            traits::has_push_back_member<TContainer>::value>::type
    processElements(TContainer& dest, std::size_t size) {
        using ValueType = typename TContainer::value_type;
        dest.clear();
        dest.reserve(size);
        for (std::size_t i = 0ul; i < size; ++i) {
            dest.push_back(impl::ValueFactory<ValueType>::create(dest.get_allocator()));
            BaseArchive::dispatch(dest.back());
        }
    }

    template<class TContainer>
    typename std::enable_if<!traits::is_batchable<TContainer>::value && !traits::has_reserve_member<TContainer>::value &&
                            !traits::has_push_back_member<TContainer>::value>::type
    processElements(TContainer& dest, std::size_t size) {
        resize(dest, size);
        for (auto& element : dest) {
            BaseArchive::dispatch(element);
        }
    }

    template<class TContainer>
    typename std::enable_if<traits::is_batchable<TContainer>::value && traits::has_wide_elements<TContainer>::value>::type
    processElements(TContainer& dest, std::size_t size) {
        using ValueType = typename TContainer::value_type;
        if (size != 0ul) {
            resize(dest, size);
            // NOLINTNEXTLINE(cppcoreguidelines-pro-type-reinterpret-cast)
            stream->read(reinterpret_cast<char*>(&dest[0]), size * sizeof(ValueType));
            const std::size_t blockWidth = 16ul / sizeof(ValueType);
            const std::size_t alignedSize = size - (size % blockWidth);
            for (std::size_t i = 0ul; i < alignedSize; i += blockWidth) {
                SwapFrom<EByteOrder>::swap(&dest[i]);
            }

            for (std::size_t i = alignedSize; i < size; ++i) {
                SwapFrom<EByteOrder>::swap(dest[i]);
            }
        }
    }

    template<class TContainer>
    typename std::enable_if<traits::is_batchable<TContainer>::value && !traits::has_wide_elements<TContainer>::value>::type
    processElements(TContainer& dest, std::size_t size) {
        using ValueType = typename TContainer::value_type;
        if (size != 0ul) {
            resize(dest, size);
            // NOLINTNEXTLINE(cppcoreguidelines-pro-type-reinterpret-cast)
            stream->read(reinterpret_cast<char*>(&dest[0]), size * sizeof(ValueType));
        }
    }

private:
    template<class TContainer>
    void resize(TContainer& dest, std::size_t size) {
        dest.resize(size);
    }

    template<typename T, class TAllocator>
    void resize(DynArray<T, TAllocator>& dest, std::size_t size) {
        dest.resize_uninitialized(size);
    }

private:
    TStream* stream;
};

template<class TStream, typename TSize = std::uint32_t, typename TOffset = TSize, Endianness EByteOrder = Endianness::Network>
class BinaryInputArchive : public ExtendableBinaryInputArchive<BinaryInputArchive<TStream, TSize, TOffset, EByteOrder>,
                                                               TStream,
                                                               TSize,
                                                               TOffset,
                                                               EByteOrder> {
private:
    using BaseArchive = ExtendableBinaryInputArchive<BinaryInputArchive, TStream, TSize, TOffset, EByteOrder>;
    friend Archive<BinaryInputArchive>;

public:
    explicit BinaryInputArchive(TStream* stream_) :
        BaseArchive{this, stream_} {
    }

private:
    template<typename T>
    void process(T&& dest) {
        BaseArchive::process(std::forward<T>(dest));
    }
};

}  // namespace terse
