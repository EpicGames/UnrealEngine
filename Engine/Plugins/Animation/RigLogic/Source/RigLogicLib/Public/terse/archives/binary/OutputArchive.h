// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/Archive.h"
#include "terse/archives/Traits.h"
#include "terse/types/Anchor.h"
#include "terse/types/ArchiveOffset.h"
#include "terse/types/ArchiveSize.h"
#include "terse/types/Blob.h"
#include "terse/types/DynArray.h"
#include "terse/types/Transparent.h"
#include "terse/types/Versioned.h"
#include "terse/utils/ByteSwap.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <limits>
#include <memory>
#include <string>
#include <tuple>
#include <type_traits>
#include <utility>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

template<class TExtender, class TStream, typename TSize, typename TOffset, Endianness EByteOrder>
class ExtendableBinaryOutputArchive : public Archive<TExtender> {
private:
    using BaseArchive = Archive<TExtender>;

public:
    // Given the possibility of both 32 and 64bit platforms, use a fixed width type during serialization
    using SizeType = TSize;
    using OffsetType = TOffset;

    static constexpr Endianness endianness() {
        return EByteOrder;
    }

public:
    ExtendableBinaryOutputArchive(TExtender* extender, TStream* stream_) :
        BaseArchive{extender},
        stream{stream_} {
    }

    bool isOk() {
        return true;
    }

    void sync() {
    }

    void label(const char* /*unused*/) {
    }

    std::size_t boundSize(std::size_t size) {
        return size;
    }

protected:
    template<typename T>
    void process(Transparent<T>&& source) {
        process(source);
    }

    template<typename T>
    void process(Transparent<T>& source) {
        process(source.data);
    }

    void process(Anchor<OffsetType>& source) {
        source.value = static_cast<OffsetType>(stream->tell());
    }

    void process(ArchiveOffset<OffsetType>& source) {
// Record the position where the offset is going to be written so it can be seeked to later when
// its proxy is found
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
        source.position = static_cast<decltype(source.position)>(stream->tell());
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        // Since the actual offset value is not yet known at the time when its declaration is encountered,
        // fill its place with zeros as a placeholder, and it will be later populated when its associated
        // proxy is found
        process(typename ArchiveOffset<OffsetType>::ValueType{});
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy& source) {
        // The current position of the stream needs to be written to the position where the associated
        // `ArchiveOffset` is found.
        auto current = stream->tell();
        assert(current <= std::numeric_limits<OffsetType>::max());
        source.target->value = static_cast<OffsetType>(current);
        // Seek to the actual position of the offset marker and write the stream's above captured position there
        stream->seek(source.target->position);
        process(static_cast<OffsetType>(current));
        // Return to the earlier captured stream position so processing can safely resume from there
        stream->seek(current);
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy&& source) {
        process(source);
    }

    void process(ArchiveSize<SizeType, OffsetType>& source) {
// Record the position where the size is going to be written so it can be seeked to later when
// its proxy is found
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
        source.position = static_cast<decltype(source.position)>(stream->tell());
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
        // Since the actual size value is not yet known at the time when it's declaration is encountered,
        // fill it's place with zeros as a placeholder, and it will be later populated when it's associated
        // proxy is found
        process(typename ArchiveSize<SizeType, OffsetType>::ValueType{});
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy& source) {
        // The current position of the stream minus the base offset needs to be written to the position
        // where the associated `ArchiveSize` is found.
        const auto current = stream->tell();
        assert(source.base != nullptr);
        assert(source.base->value <= current);
        auto size = current - source.base->value;
        assert(size <= std::numeric_limits<SizeType>::max());
        source.target->value = static_cast<SizeType>(size);
        // Seek to the actual position of the size marker and write the above calculated size
        stream->seek(source.target->position);
        process(static_cast<SizeType>(size));
        // Return to the earlier captured stream position so processing can safely resume from there
        stream->seek(current);
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_member<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_member<T, V>::value, void>::type process(Versioned<T, V>& source) {
        source.data.save(*static_cast<TExtender*>(this), V{});
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>& source) {
        source.data.serialize(*static_cast<TExtender*>(this), V{});
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_function<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_function<T, V>::value, void>::type process(Versioned<T, V>& source) {
        save(*static_cast<TExtender*>(this), V{}, source.data);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>& source) {
        serialize(*static_cast<TExtender*>(this), V{}, source.data);
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_save_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_save_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_save_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_save_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>& dest) {
        // If no versioned serializer is found, but version information was passed along, fall back to unversioned
        // functions if available.
        BaseArchive::dispatch(dest.data);
    }

    template<typename T>
    typename std::enable_if<traits::has_save_member<T>::value, void>::type process(const T& source) {
        const_cast<T&>(source).save(*static_cast<TExtender*>(this));
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_member<T>::value, void>::type process(const T& source) {
        const_cast<T&>(source).serialize(*static_cast<TExtender*>(this));
    }

    template<typename T>
    typename std::enable_if<traits::has_save_function<T>::value, void>::type process(const T& source) {
        save(*static_cast<TExtender*>(this), const_cast<T&>(source));
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_function<T>::value, void>::type process(const T& source) {
        serialize(*static_cast<TExtender*>(this), const_cast<T&>(source));
    }

    template<typename T>
    typename std::enable_if<!traits::has_save_member<T>::value && !traits::has_serialize_member<T>::value &&
                                !traits::has_save_function<T>::value && !traits::has_serialize_function<T>::value,
                            void>::type
    process(const T& source) {
        T swapped;
        std::memcpy(&swapped, &source, sizeof(T));
        SwapTo<EByteOrder>::swap(swapped);
        // NOLINTNEXTLINE(cppcoreguidelines-pro-type-reinterpret-cast)
        stream->write(reinterpret_cast<char*>(&swapped), sizeof(T));
    }

    template<typename T, typename... Args>
    void process(const Blob<T, Args...>& source) {
        using ValueType = typename Blob<T, Args...>::value_type;
        if (source.size() != 0ul) {
            // NOLINTNEXTLINE(cppcoreguidelines-pro-type-reinterpret-cast)
            stream->write(reinterpret_cast<const char*>(source.data()), source.size() * sizeof(ValueType));
        }
    }

    template<typename T, std::size_t N>
    void process(const std::array<T, N>& source) {
        processElements(source);
    }

    template<typename T, typename... Args>
    void process(const std::vector<T, Args...>& source) {
        processSize(source.size());
        processElements(source);
    }

    template<typename T, typename... Args>
    void process(const DynArray<T, Args...>& source) {
        processSize(source.size());
        processElements(source);
    }

    template<typename T, typename... Args>
    void process(const std::basic_string<T, Args...>& source) {
        processSize(source.size());
        processElements(source);
    }

    template<typename K, typename V>
    void process(const std::pair<K, V>& source) {
        BaseArchive::dispatch(source.first);
        BaseArchive::dispatch(source.second);
    }

    template<typename K, typename V>
    void process(const std::tuple<K, V>& source) {
        BaseArchive::dispatch(std::get<0>(source));
        BaseArchive::dispatch(std::get<1>(source));
    }

    void processSize(std::size_t size) {
        assert(size <= std::numeric_limits<SizeType>::max());
        process(static_cast<SizeType>(size));
    }

    template<class TContainer>
    typename std::enable_if<!traits::is_batchable<TContainer>::value>::type processElements(const TContainer& source) {
        for (const auto& element : source) {
            BaseArchive::dispatch(element);
        }
    }

    template<class TContainer>
    typename std::enable_if<traits::is_batchable<TContainer>::value && traits::has_wide_elements<TContainer>::value>::type
    processElements(const TContainer& source) {
        for (const auto& element : source) {
            BaseArchive::dispatch(element);
        }
    }

    template<class TContainer>
    typename std::enable_if<traits::is_batchable<TContainer>::value && !traits::has_wide_elements<TContainer>::value>::type
    processElements(const TContainer& source) {
        using ValueType = typename TContainer::value_type;
        const auto size = source.size();
#ifdef _MSC_VER
    #pragma warning(suppress : 6326)
#endif
        if (size != 0ul) {
            // NOLINTNEXTLINE(cppcoreguidelines-pro-type-reinterpret-cast)
            stream->write(reinterpret_cast<const char*>(&source[0]), size * sizeof(ValueType));
        }
    }

private:
    TStream* stream;
};

template<class TStream, typename TSize = std::uint32_t, typename TOffset = TSize, Endianness EByteOrder = Endianness::Network>
class BinaryOutputArchive : public ExtendableBinaryOutputArchive<BinaryOutputArchive<TStream, TSize, TOffset, EByteOrder>,
                                                                 TStream,
                                                                 TSize,
                                                                 TOffset,
                                                                 EByteOrder> {
public:
    using BaseArchive = ExtendableBinaryOutputArchive<BinaryOutputArchive, TStream, TSize, TOffset, EByteOrder>;
    friend Archive<BinaryOutputArchive>;

public:
    explicit BinaryOutputArchive(TStream* stream_) :
        BaseArchive{this, stream_} {
    }

private:
    template<typename T>
    void process(T&& dest) {
        BaseArchive::process(std::forward<T>(dest));
    }
};

}  // namespace terse
