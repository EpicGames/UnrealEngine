// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/Archive.h"
#include "terse/archives/Common.h"
#include "terse/archives/Traits.h"
#include "terse/types/Anchor.h"
#include "terse/types/ArchiveOffset.h"
#include "terse/types/ArchiveSize.h"
#include "terse/types/Blob.h"
#include "terse/types/CharInputStreamBuf.h"
#include "terse/types/DynArray.h"
#include "terse/types/Transparent.h"
#include "terse/types/Versioned.h"
#include "terse/utils/Base64.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <memory>
#include <sstream>
#include <string>
#include <tuple>
#include <type_traits>
#include <utility>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

template<class TExtender, class TStream, typename TSize, typename TOffset>
class ExtendableJSONInputArchive : public Archive<TExtender> {
public:
    // Given the possibility of both 32 and 64bit platforms, use a fixed width type during serialization
    using SizeType = TSize;
    using OffsetType = TOffset;

private:
    using BaseArchive = Archive<TExtender>;

public:
    ExtendableJSONInputArchive(TExtender* extender, TStream* stream_) :
        BaseArchive{extender},
        streamBuf{stream_},
        stream{&streamBuf},
        state{false, false, false} {
    }

    bool isOk() const {
        return !state.malformed;
    }

    void sync() {
    }

    std::size_t boundSize(std::size_t size) {
        return size;
    }

    void label(const char* value) {
        if (state.malformed) {
            return;
        }

        skipWhiteSpace();

        if (state.firstMember) {
            state.firstMember = false;
        } else {
            if (!expectChar(',')) {
                return;
            }
            skipWhiteSpace();
        }

        if (!expectChar('"')) {
            return;
        }

        for (std::size_t i = {}; i < std::strlen(value); ++i) {
            if (!expectChar(value[i])) {
                return;
            }
        }

        if (!expectChar('"')) {
            return;
        }

        skipWhiteSpace();

        if (!expectChar(':')) {
            return;
        }

        skipWhiteSpace();
    }

protected:
    template<typename T>
    void process(Transparent<T>&& dest) {
        process(dest);
    }

    template<typename T>
    void process(Transparent<T>& dest) {
        pushTransparency();
        process(dest.data);
    }

    void process(Anchor<OffsetType>& /*unused*/) {
        // Anchor<T> has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(ArchiveOffset<OffsetType>& dest) {
        process(dest.value);
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy& /*unused*/) {
        // ArchiveOffset<T>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy&& /*unused*/) {
        // ArchiveOffset<T>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(ArchiveSize<SizeType, OffsetType>& dest) {
        process(dest.value);
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy& /*unused*/) {
        // ArchiveSize<T, U>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy&& /*unused*/) {
        // ArchiveSize<T, U>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_member<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_member<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        dest.data.load(*static_cast<TExtender*>(this), V{});

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        dest.data.serialize(*static_cast<TExtender*>(this), V{});

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_function<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_load_function<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        load(*static_cast<TExtender*>(this), V{}, dest.data);

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        serialize(*static_cast<TExtender*>(this), V{}, dest.data);

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_load_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_load_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_load_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_load_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>& dest) {
        // If no versioned serializer is found, but version information was passed along, fall back to unversioned
        // functions if available.
        BaseArchive::dispatch(dest.data);
    }

    template<typename T>
    typename std::enable_if<traits::has_load_member<T>::value, void>::type process(T& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        dest.load(*static_cast<TExtender*>(this));

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_member<T>::value, void>::type process(T& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        dest.serialize(*static_cast<TExtender*>(this));

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T>
    typename std::enable_if<traits::has_load_function<T>::value, void>::type process(T& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        load(*static_cast<TExtender*>(this), dest);

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_function<T>::value, void>::type process(T& dest) {
        if (state.malformed) {
            return;
        }

        const bool transparent = popTransparency();
        if (!transparent) {
            preStructInput();
        }

        serialize(*static_cast<TExtender*>(this), dest);

        if (!transparent) {
            postStructInput();
        }
    }

    template<typename T>
    typename std::enable_if<!traits::has_load_member<T>::value && !traits::has_serialize_member<T>::value &&
                                !traits::has_load_function<T>::value && !traits::has_serialize_function<T>::value,
                            void>::type
    process(T& dest) {
        if (state.malformed) {
            return;
        }
        stream >> dest;
    }

    void process(char& dest) {
        if (!expectChar('"')) {
            return;
        }

        if (!readChar(&dest)) {
            return;
        }

        if (!expectChar('"')) {
            return;
        }
    }

    void process(std::uint8_t& dest) {
        std::uint16_t temp = {};
        stream >> temp;
        dest = static_cast<std::uint8_t>(temp);
    }

    void process(std::int8_t& dest) {
        std::int16_t temp = {};
        stream >> temp;
        dest = static_cast<std::int8_t>(temp);
    }

    template<typename T, typename... Args>
    void process(Blob<T, Args...>& dest) {
        std::basic_string<char, std::char_traits<char>, typename Blob<T, Args...>::allocator_type> buffer{dest.get_allocator()};
        // dest.size() is the size of the decoded data already (set by user), from which the length of the
        // encoded data is calculated and used to reserve storage for the temporary buffer where the encoded
        // data is first loaded
        buffer.reserve(base64encode(dest.size()));
        // Read base64-encoded data into temporary buffer
        process(buffer);
        // Decode from temp buffer into dest
        base64decode(dest.data(), buffer.data(), buffer.size());
    }

    template<typename T, std::size_t N>
    void process(std::array<T, N>& dest) {
        if (state.malformed) {
            return;
        }

        skipWhiteSpace();
        if (!expectChar('[')) {
            return;
        }
        skipWhiteSpace();

        if (stream.peek() == ']') {
            expectChar(']');
            return;
        }

        for (auto& element : dest) {
            BaseArchive::dispatch(element);
            if (state.malformed) {
                return;
            }

            skipWhiteSpace();
            char ch = {};
            if (!readChar(&ch)) {
                return;
            }
            if (ch == ',') {
                skipWhiteSpace();
            } else if (ch == ']') {
                break;
            }
        }

        skipWhiteSpace();
    }

    template<typename T, typename... Args>
    void process(std::vector<T, Args...>& dest) {
        if (state.malformed) {
            return;
        }

        skipWhiteSpace();
        if (!expectChar('[')) {
            return;
        }
        skipWhiteSpace();

        if (stream.peek() == ']') {
            expectChar(']');
            return;
        }

        dest.clear();
        while (true) {
            dest.push_back(impl::ValueFactory<T>::create(dest.get_allocator()));
            BaseArchive::dispatch(dest.back());
            if (state.malformed) {
                return;
            }

            skipWhiteSpace();
            char ch = {};
            if (!readChar(&ch)) {
                return;
            }
            if (ch == ',') {
                skipWhiteSpace();
            } else if (ch == ']') {
                break;
            }
        }

        skipWhiteSpace();
    }

    template<typename T, typename... Args>
    void process(DynArray<T, Args...>& dest) {
        if (state.malformed) {
            return;
        }

        skipWhiteSpace();
        if (!expectChar('[')) {
            return;
        }
        skipWhiteSpace();

        if (stream.peek() == ']') {
            expectChar(']');
            return;
        }

        dest.resize_uninitialized(1ul);
        std::size_t validElementCount = {};

        while (true) {
            BaseArchive::dispatch(dest[validElementCount]);
            if (state.malformed) {
                break;
            }

            ++validElementCount;
            if (validElementCount == dest.size()) {
                dest.resize_uninitialized(dest.size() * 2ul);
            }

            skipWhiteSpace();
            char ch = {};
            if (!readChar(&ch)) {
                break;
            }
            if (ch == ',') {
                skipWhiteSpace();
            } else if (ch == ']') {
                break;
            }
        }

        dest.resize(validElementCount);
        skipWhiteSpace();
    }

    template<typename T, typename... Args>
    void process(std::basic_string<T, Args...>& dest) {
        if (state.malformed) {
            return;
        }

        if (!expectChar('"')) {
            return;
        }

        dest.clear();
        char ch = {};
        while (readChar(&ch)) {
            if (ch == '"') {
                return;
            } else {
                dest.push_back(ch);
            }
        }

        // This should not be reached if the string was properly quoted
        state.malformed = true;
    }

    template<typename K, typename V>
    void process(std::pair<K, V>& dest) {
        if (state.malformed) {
            return;
        }

        skipWhiteSpace();
        if (!expectChar('[')) {
            return;
        }
        skipWhiteSpace();

        BaseArchive::dispatch(dest.first);

        skipWhiteSpace();
        if (!expectChar(',')) {
            return;
        }
        skipWhiteSpace();

        BaseArchive::dispatch(dest.second);

        skipWhiteSpace();
        if (!expectChar(']')) {
            return;
        }
        skipWhiteSpace();
    }

    template<typename K, typename V>
    void process(std::tuple<K, V>& dest) {
        if (state.malformed) {
            return;
        }

        skipWhiteSpace();
        if (!expectChar('[')) {
            return;
        }
        skipWhiteSpace();

        BaseArchive::dispatch(std::get<0>(dest));

        skipWhiteSpace();
        if (!expectChar(',')) {
            return;
        }
        skipWhiteSpace();

        BaseArchive::dispatch(std::get<1>(dest));

        skipWhiteSpace();
        if (!expectChar(']')) {
            return;
        }
        skipWhiteSpace();
    }

private:
    bool readChar(char* dest) {
        if (!stream.read(dest, 1)) {
            state.malformed = true;
            return false;
        }
        return true;
    }

    bool expectChar(char expected) {
        char ch = {};
        if (!readChar(&ch)) {
            return false;
        }
        if (ch != expected) {
            state.malformed = true;
            return false;
        }
        return true;
    }

    void skipWhiteSpace() {
        std::ws(stream);
    }

    void preStructInput() {
        state.firstMember = true;

        skipWhiteSpace();
        if (!expectChar('{')) {
            return;
        }
        skipWhiteSpace();
    }

    void postStructInput() {
        skipWhiteSpace();
        if (!expectChar('}')) {
            return;
        }
        skipWhiteSpace();
    }

    void pushTransparency() {
        state.transparent = true;
    }

    bool popTransparency() {
        const bool transparent = state.transparent;
        state.transparent = false;
        return transparent;
    }

private:
    struct SerializationState {
        bool malformed;
        bool firstMember;
        bool transparent;
    };

private:
    CharInputStreamBuf<TStream> streamBuf;
    std::istream stream;
    SerializationState state;
};

template<class TStream, typename TSize = std::uint32_t, typename TOffset = TSize>
class JSONInputArchive : public ExtendableJSONInputArchive<JSONInputArchive<TStream, TSize, TOffset>, TStream, TSize, TOffset> {
private:
    using BaseArchive = ExtendableJSONInputArchive<JSONInputArchive, TStream, TSize, TOffset>;
    friend Archive<JSONInputArchive>;

public:
    explicit JSONInputArchive(TStream* stream_) :
        BaseArchive{this, stream_} {
    }

private:
    template<typename T>
    void process(T&& dest) {
        BaseArchive::process(std::forward<T>(dest));
    }
};

}  // namespace terse
