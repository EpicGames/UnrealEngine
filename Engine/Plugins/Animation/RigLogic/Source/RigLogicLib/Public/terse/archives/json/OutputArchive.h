// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/Archive.h"
#include "terse/archives/Traits.h"
#include "terse/types/Anchor.h"
#include "terse/types/ArchiveOffset.h"
#include "terse/types/ArchiveSize.h"
#include "terse/types/Blob.h"
#include "terse/types/CharOutputStreamBuf.h"
#include "terse/types/DynArray.h"
#include "terse/types/Transparent.h"
#include "terse/types/Versioned.h"
#include "terse/utils/Base64.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <limits>
#include <memory>
#include <sstream>
#include <string>
#include <tuple>
#include <type_traits>
#include <utility>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

template<class TExtender, class TStream, typename TSize, typename TOffset>
class ExtendableJSONOutputArchive : public Archive<TExtender> {
private:
    using BaseArchive = Archive<TExtender>;

public:
    // Given the possibility of both 32 and 64bit platforms, use a fixed width type during serialization
    using SizeType = TSize;
    using OffsetType = TOffset;

public:
    ExtendableJSONOutputArchive(TExtender* extender, TStream* stream_, std::uint32_t indentWidth) :
        BaseArchive{extender},
        streamBuf{stream_},
        stream{&streamBuf},
        state{indentWidth, 0u, false, false} {
    }

    bool isOk() {
        return true;
    }

    void sync() {
        stream << std::flush;
    }

    std::size_t boundSize(std::size_t size) {
        return size;
    }

    void label(const char* value) {
        if (state.firstMember) {
            state.firstMember = false;
        } else {
            stream << ",\n";
        }
        indent();
        stream << "\"" << value << "\"" << ": ";
    }

protected:
    template<typename T>
    void process(Transparent<T>&& source) {
        process(source);
    }

    template<typename T>
    void process(Transparent<T>& source) {
        pushTransparency();
        process(source.data);
    }

    void process(Anchor<OffsetType>& /*unused*/) {
        // Anchor<T> has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(ArchiveOffset<OffsetType>& source) {
        process(source.value);
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy& /*unused*/) {
        // ArchiveOffset<T>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(typename ArchiveOffset<OffsetType>::Proxy&& /*unused*/) {
        // ArchiveOffset<T>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(ArchiveSize<SizeType, OffsetType>& source) {
        process(source.value);
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy& /*unused*/) {
        // ArchiveSize<T, U>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    void process(typename ArchiveSize<SizeType, OffsetType>::Proxy&& /*unused*/) {
        // ArchiveSize<T, U>::Proxy has no meaning in a non-binary format, so it's just silently ignored
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_member<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_member<T, V>::value, void>::type process(Versioned<T, V>& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        const_cast<T&>(source.data).save(*static_cast<TExtender*>(this), V{});
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_member<T, V>::value, void>::type process(Versioned<T, V>& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        const_cast<T&>(source.data).serialize(*static_cast<TExtender*>(this), V{});
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_function<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_save_function<T, V>::value, void>::type process(Versioned<T, V>& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        save(*static_cast<TExtender*>(this), V{}, const_cast<T&>(source.data));
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>&& source) {
        process(source);
    }

    template<typename T, typename V>
    typename std::enable_if<traits::has_versioned_serialize_function<T, V>::value, void>::type process(Versioned<T, V>& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        serialize(*static_cast<TExtender*>(this), V{}, const_cast<T&>(source.data));
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_save_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_save_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>&& dest) {
        process(dest);
    }

    template<typename T, typename V>
    typename std::enable_if<
        !traits::has_versioned_save_member<T, V>::value && !traits::has_versioned_serialize_member<T, V>::value &&
            !traits::has_versioned_save_function<T, V>::value && !traits::has_versioned_serialize_function<T, V>::value,
        void>::type
    process(Versioned<T, V>& dest) {
        // If no versioned serializer is found, but version information was passed along, fall back to unversioned
        // functions if available.
        BaseArchive::dispatch(dest.data);
    }

    template<typename T>
    typename std::enable_if<traits::has_save_member<T>::value, void>::type process(const T& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        const_cast<T&>(source).save(*static_cast<TExtender*>(this));
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_member<T>::value, void>::type process(const T& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        const_cast<T&>(source).serialize(*static_cast<TExtender*>(this));
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T>
    typename std::enable_if<traits::has_save_function<T>::value, void>::type process(const T& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        save(*static_cast<TExtender*>(this), const_cast<T&>(source));
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T>
    typename std::enable_if<traits::has_serialize_function<T>::value, void>::type process(const T& source) {
        const bool transparent = popTransparency();
        if (!transparent) {
            preStructOutput();
        }
        serialize(*static_cast<TExtender*>(this), const_cast<T&>(source));
        if (!transparent) {
            postStructOutput();
        }
    }

    template<typename T>
    typename std::enable_if<!traits::has_save_member<T>::value && !traits::has_serialize_member<T>::value &&
                                !traits::has_save_function<T>::value && !traits::has_serialize_function<T>::value,
                            void>::type
    process(const T& source) {
        stream << source;
    }

    void process(char source) {
        stream << "\"" << source << "\"";
    }

    void process(std::uint8_t source) {
        stream << static_cast<std::uint32_t>(source);
    }

    void process(std::int8_t source) {
        stream << static_cast<std::int32_t>(source);
    }

    template<typename T, typename... Args>
    void process(const Blob<T, Args...>& source) {
        const auto encodedSize = base64encode(source.size());
        std::basic_string<char, std::char_traits<char>, typename Blob<T, Args...>::allocator_type> buffer{encodedSize,
                                                                                                          '\0',
                                                                                                          source.get_allocator()};
        base64encode(&buffer[0], source.data(), source.size());
        // Write base64-encoded data from string temporary buffer
        process(buffer);
    }

    template<typename T, std::size_t N>
    void process(const std::array<T, N>& source) {
        processElements(source);
    }

    template<typename T, typename... Args>
    void process(const std::vector<T, Args...>& source) {
        processElements(source);
    }

    template<typename T, typename... Args>
    void process(const DynArray<T, Args...>& source) {
        processElements(source);
    }

    template<typename T, typename... Args>
    void process(const std::basic_string<T, Args...>& source) {
        stream << "\"" << source << "\"";
    }

    template<typename K, typename V>
    void process(const std::pair<K, V>& source) {
        stream << "[";
        BaseArchive::dispatch(source.first);
        stream << ", ";
        BaseArchive::dispatch(source.second);
        stream << "]";
    }

    template<typename K, typename V>
    void process(const std::tuple<K, V>& source) {
        stream << "[";
        BaseArchive::dispatch(std::get<0>(source));
        stream << ", ";
        BaseArchive::dispatch(std::get<1>(source));
        stream << "]";
    }

    template<class TContainer>
    void processElements(const TContainer& source) {
        stream << "[";
        if (!source.empty()) {
            for (auto it = source.begin();;) {
                BaseArchive::dispatch(*it);
                ++it;
                if (it == source.end()) {
                    break;
                } else {
                    stream << ", ";
                }
            }
        }
        stream << "]";
    }

private:
    void indent() {
        std::fill_n(std::ostream_iterator<char>(stream), state.indentLevel * state.indentWidth, ' ');
    }

    void preStructOutput() {
        state.firstMember = true;

        stream << "{";
        stream << "\n";

        ++state.indentLevel;
    }

    void postStructOutput() {
        --state.indentLevel;

        stream << "\n";
        indent();
        stream << "}";
    }

    void pushTransparency() {
        state.transparent = true;
    }

    bool popTransparency() {
        const bool transparent = state.transparent;
        state.transparent = false;
        return transparent;
    }

private:
    struct SerializationState {
        std::uint32_t indentWidth;
        std::uint32_t indentLevel;
        bool firstMember;
        bool transparent;
    };

private:
    CharOutputStreamBuf<TStream> streamBuf;
    std::ostream stream;
    SerializationState state;
};

template<class TStream, typename TSize = std::uint32_t, typename TOffset = TSize>
class JSONOutputArchive
    : public ExtendableJSONOutputArchive<JSONOutputArchive<TStream, TSize, TOffset>, TStream, TSize, TOffset> {
public:
    using BaseArchive = ExtendableJSONOutputArchive<JSONOutputArchive, TStream, TSize, TOffset>;
    friend Archive<JSONOutputArchive>;

public:
    JSONOutputArchive(TStream* stream_, std::uint32_t indentWidth_) :
        BaseArchive{this, stream_, indentWidth_} {
    }

private:
    template<typename T>
    void process(T&& dest) {
        BaseArchive::process(std::forward<T>(dest));
    }
};

}  // namespace terse
