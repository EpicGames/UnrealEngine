// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

// Anchor is a virtual, utility type only (meaning it does not write or read any data from the stream)
// It only captures the current position in the stream (into an in-memory variable only), at the time
// when the anchor is encountered in the serialization process.
// An Anchor can be associated with an ArchiveSize type, and ArchiveSize will use it as the base offset
// from which to calculate its own size.
template<typename TOffset>
struct Anchor {
    using ValueType = TOffset;
    using OnMoveCallback = void (*)(Anchor*, void*);

    mutable ValueType value;

    OnMoveCallback onMoveCallback;
    void* onMoveCallbackData;

    Anchor() :
        value{},
        onMoveCallback{nullptr},
        onMoveCallbackData{nullptr} {
    }

    ~Anchor() = default;

    Anchor(const Anchor&) = delete;
    Anchor& operator=(const Anchor&) = delete;

    Anchor(Anchor&& rhs) :
        value{},
        onMoveCallback{nullptr},
        onMoveCallbackData{nullptr} {
        std::swap(value, rhs.value);
        std::swap(onMoveCallback, rhs.onMoveCallback);
        std::swap(onMoveCallbackData, rhs.onMoveCallbackData);
        if (onMoveCallback) {
            onMoveCallback(this, onMoveCallbackData);
        }
    }

    Anchor& operator=(Anchor&& rhs) {
        std::swap(value, rhs.value);
        std::swap(onMoveCallback, rhs.onMoveCallback);
        std::swap(onMoveCallbackData, rhs.onMoveCallbackData);
        if (onMoveCallback) {
            onMoveCallback(this, onMoveCallbackData);
        }
        return *this;
    }

    void onMove(OnMoveCallback callback, void* data) {
        onMoveCallback = callback;
        onMoveCallbackData = data;
    }
};

}  // namespace terse
