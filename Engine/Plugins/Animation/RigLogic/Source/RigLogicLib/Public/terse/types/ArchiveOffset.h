// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

// ArchiveOffset is a type that stores an absolute stream offset (both in-memory and writes that value into the stream as well).
// It has an accompanying Proxy type, which is a virtual, utility type, and they work together to achieve the function
// of ArchiveOffset.
// When ArchiveOffset is encountered during the serialization process, at first an empty offset (zero) value will be written
// into the stream and the position of that offset value will be captured in an in-memory variable. Later, when its associated
// ArchiveOffset<T>::Proxy is encountered, the current stream position will be captured, and it will update the in-memory value
// of its ArchiveOffset with it. Then, it will seek the stream to the earlier captured position of the offset value (where zeros
// were written initially), and it will write the offset value captured by the proxy into the stream. Lastly, it will seek back
// to the position of the stream before the proxy was encountered, and resume serialization.
template<typename TOffset>
struct ArchiveOffset {
    using ValueType = TOffset;

    struct Proxy {
        ArchiveOffset* target;

        explicit Proxy(ArchiveOffset& ptr) :
            target{std::addressof(ptr)} {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wdangling-pointer"
#endif
#if !defined(__clang_analyzer__)
            target->proxy = this;
#endif
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
        }

        ~Proxy() {
            if (target != nullptr) {
                target->proxy = nullptr;
            }
        }

        Proxy(const Proxy&) = delete;
        Proxy& operator=(const Proxy&) = delete;

        Proxy(Proxy&& rhs) :
            target{nullptr} {
            std::swap(target, rhs.target);
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wdangling-pointer"
#endif
            target->proxy = this;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
        }

        Proxy& operator=(Proxy&& rhs) {
            std::swap(target, rhs.target);
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wdangling-pointer"
#endif
            target->proxy = this;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
            return *this;
        }
    };

    // The position of the marker itself in the stream (this is a runtime-only value
    // which is not written to the file, needed only for the serializer to know where
    // to seek within the stream when the marker's actual value needs to be written)
    mutable std::size_t position;
    // The position in the stream where the marker wants to point (this is the actual
    // value that is written to the file)
    mutable ValueType value;
    // When offset is moved, it's associated proxy must be updated about the new address
    Proxy* proxy;

    ArchiveOffset() :
        position{},
        value{},
        proxy{nullptr} {
    }

    ~ArchiveOffset() = default;

    ArchiveOffset(const ArchiveOffset&) = delete;
    ArchiveOffset& operator=(const ArchiveOffset&) = delete;

    ArchiveOffset(ArchiveOffset&& rhs) :
        position{},
        value{},
        proxy{nullptr} {
        std::swap(position, rhs.position);
        std::swap(value, rhs.value);
        std::swap(proxy, rhs.proxy);
        // Update proxy with new address
        if (proxy != nullptr) {
            proxy->target = this;
        }
    }

    ArchiveOffset& operator=(ArchiveOffset&& rhs) {
        std::swap(position, rhs.position);
        std::swap(value, rhs.value);
        std::swap(proxy, rhs.proxy);
        // Update proxy with new address
        if (proxy != nullptr) {
            proxy->target = this;
        }
        return *this;
    }
};

template<typename TOffset>
typename ArchiveOffset<TOffset>::Proxy proxy(ArchiveOffset<TOffset>& offset) {
    return typename ArchiveOffset<TOffset>::Proxy{offset};
}

}  // namespace terse
