// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/types/Anchor.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <cstdint>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

// ArchiveSize is a type that stores an arbitrary size value (both in-memory and writes that value into the stream as well).
// It has two accompanying types: Proxy and Anchor, which are both virtual, utility types, and they work together to achieve
// the function of ArchiveSize.
// When ArchiveSize is encountered during the serialization process, at first an empty size (zero) value will be written
// into the stream and the position of that size value will be captured in an in-memory variable.
// Later, when its associated Anchor is encountered, it will capture the current stream position as the base (start) offset,
// from which the size is going to be computed (size is relative to some base offset).
// Lastly, when its ArchiveSize<T>::Proxy is encountered, the current stream position will be captured, and the earlier captured
// anchor base offset will be subtracted from it (resulting in the size value - the size between Anchor offset and Proxy offset).
// With the size value computed, it will update the in-memory value stored in ArchiveSize with it. Then, it will seek the
// stream to the earlier captured position of the size value (where zeros were written initially), and it will write the size
// value into the stream at that position. With all that done, it will seek back to the position of the stream before the
// proxy was encountered, and resume serialization.
template<typename TSize, typename TOffset>
struct ArchiveSize {
    using ValueType = TSize;

    struct Proxy {
        ArchiveSize* target;
        Anchor<TOffset>* base;

        Proxy(ArchiveSize& size, Anchor<TOffset>& offset) :
            target{std::addressof(size)},
            base{std::addressof(offset)} {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wdangling-pointer"
#endif
#if !defined(__clang_analyzer__)
            target->proxy = this;
            base->onMove(onBaseMoved, this);
#endif
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
        }

        ~Proxy() {
            if (target != nullptr) {
                target->proxy = nullptr;
            }
            if (base != nullptr) {
                base->onMove(onBaseMoved, nullptr);
            }
        }

        Proxy(const Proxy&) = delete;
        Proxy& operator=(const Proxy&) = delete;

        Proxy(Proxy&& rhs) :
            target{nullptr},
            base{nullptr} {
            std::swap(target, rhs.target);
            std::swap(base, rhs.base);
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wdangling-pointer"
#endif
            target->proxy = this;
            base->onMove(onBaseMoved, this);
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
        }

        Proxy& operator=(Proxy&& rhs) {
            std::swap(target, rhs.target);
            std::swap(base, rhs.base);
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wdangling-pointer"
#endif
            target->proxy = this;
            base->onMove(onBaseMoved, this);
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
            return *this;
        }

    private:
        static void onBaseMoved(Anchor<TOffset>* anchor, void* data) {
            if (data != nullptr) {
                auto proxy = static_cast<Proxy*>(data);
                proxy->base = anchor;
            }
        }
    };

    // The position of the marker itself in the stream (this is a runtime-only value
    // which is not written to the file, needed only for the serializer to know where
    // to seek within the stream when the marker's actual value needs to be written)
    mutable std::size_t position;
    // The size value itself (what is written to the file)
    mutable ValueType value;
    // When size is moved, it's associated proxy must be updated about the new address
    Proxy* proxy;

    ArchiveSize() :
        position{},
        value{},
        proxy{nullptr} {
    }

    ~ArchiveSize() = default;

    ArchiveSize(const ArchiveSize&) = delete;
    ArchiveSize& operator=(const ArchiveSize&) = delete;

    ArchiveSize(ArchiveSize&& rhs) :
        position{},
        value{},
        proxy{nullptr} {
        std::swap(position, rhs.position);
        std::swap(value, rhs.value);
        std::swap(proxy, rhs.proxy);
        // Update proxy with new address
        if (proxy != nullptr) {
            proxy->target = this;
        }
    }

    ArchiveSize& operator=(ArchiveSize&& rhs) {
        std::swap(position, rhs.position);
        std::swap(value, rhs.value);
        std::swap(proxy, rhs.proxy);
        // Update proxy with new address
        if (proxy != nullptr) {
            proxy->target = this;
        }
        return *this;
    }
};

template<typename TSize, typename TOffset>
typename ArchiveSize<TSize, TOffset>::Proxy proxy(ArchiveSize<TSize, TOffset>& size, Anchor<TOffset>& base) {
    return typename ArchiveSize<TSize, TOffset>::Proxy{size, base};
}

}  // namespace terse
