// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/types/DynArray.h"

namespace terse {

// Blob is a wrapper type around a byte-array essentially, and is used to serialize binary blobs,
// whose size is predefined by another entity (e.g. an ArchiveSize object).
// During serialization, the amount of bytes either written to or read from the stream will be
// controlled by the user (by setting the size on the blob type through the setSize member function).
// For text-based serializers, binary blobs are Base64 encoded/decoded.
template<typename T, typename TAllocator>
class Blob {
public:
    static_assert(sizeof(T) == sizeof(char), "Blob supports only native byte-sized types.");

    using value_type = T;
    using allocator_type = TAllocator;

public:
    Blob() = default;

    explicit Blob(const allocator_type& alloc) :
        bytes{alloc} {
    }

    Blob(std::size_t size, const allocator_type& alloc) :
        bytes{size, alloc} {
    }

    allocator_type get_allocator() const noexcept {
        return bytes.get_allocator();
    }

    value_type* data() {
        return bytes.data();
    }

    const value_type* data() const {
        return bytes.data();
    }

    std::size_t size() const {
        return bytes.size();
    }

    void setSize(std::size_t newSize) {
        bytes.resize_uninitialized(newSize);
    }

private:
    DynArray<value_type, allocator_type> bytes;
};

}  // namespace terse
