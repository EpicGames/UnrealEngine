// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <streambuf>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

template<class TStream, std::size_t BufferSize = 4096ul>
class CharInputStreamBuf : public std::streambuf {
public:
    using PosType = std::streambuf::pos_type;
    using OffType = std::streambuf::off_type;
    using CharType = std::streambuf::char_type;
    using IntType = std::streambuf::int_type;
    using TraitsType = std::streambuf::traits_type;

public:
    explicit CharInputStreamBuf(TStream* stream_) :
        stream{stream_},
        buffer{} {
        char* bufferEnd = buffer.data() + buffer.size();
        setg(bufferEnd, bufferEnd, bufferEnd);
    }

    std::streamsize xsgetn(CharType* destination, std::streamsize size) override {
        if (size <= 0) {
            return 0;
        }

        std::streamsize bytesNeeded = size;

        // Exhaust buffer contents first
        if (gptr() < egptr()) {
            const std::size_t bytesInBuffer = static_cast<std::size_t>(egptr() - gptr());
            const std::size_t bytesToCopy = std::min(static_cast<std::size_t>(bytesNeeded), bytesInBuffer);
            std::memcpy(destination, gptr(), bytesToCopy);
            destination += bytesToCopy;
            bytesNeeded -= static_cast<std::streamsize>(bytesToCopy);
            gbump(static_cast<IntType>(bytesToCopy));
        }

        // If there are still bytes needed (more data than buffer contained), read the rest directly from the stream
        if (bytesNeeded > 0) {
            const auto bytesRead = static_cast<std::streamsize>(stream->read(destination, static_cast<std::size_t>(bytesNeeded)));
            bytesNeeded -= bytesRead;
        }

        // Refill buffer for subsequent accesses (if possible)
        underflow();

        return size - bytesNeeded;
    }

    IntType underflow() override {
        if (gptr() < egptr()) {
            return TraitsType::to_int_type(*gptr());
        }

        char* base = buffer.data();
        char* start = base;
        // Initially eback() == buffer.end() , but if that's not the case, buffer has already been
        // filled before, in which case first copy the data from the put-back area (located at the end of
        // the buffer) into the beginning of the buffer, making them the next characters to be read from it.
        // Put-back area is just one byte in this implementation currently.
        if (eback() == base) {
            std::memcpy(base, egptr() - 1ul, 1ul);
            ++start;
        }

        // Now fill the rest of the buffer after the put-back area is moved to the beginning of the buffer
        const std::size_t bytesRead = stream->read(start, buffer.size() - static_cast<std::size_t>(start - base));
        if (bytesRead == 0ul) {
            return TraitsType::eof();
        }

        setg(base, start, start + bytesRead);
        return TraitsType::to_int_type(*gptr());
    }

    std::streamsize showmanyc() override {
        return static_cast<std::streamsize>(stream->size() - stream->tell());
    }

private:
    TStream* stream;
    std::array<char, BufferSize> buffer;
};

}  // namespace terse
