// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <streambuf>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

template<class TStream, std::size_t BufferSize = 4096ul>
class CharOutputStreamBuf : public std::streambuf {
public:
    using PosType = std::streambuf::pos_type;
    using OffType = std::streambuf::off_type;
    using CharType = std::streambuf::char_type;
    using IntType = std::streambuf::int_type;
    using TraitsType = std::streambuf::traits_type;

public:
    explicit CharOutputStreamBuf(TStream* stream_) :
        stream{stream_},
        buffer{} {
        setp(buffer.data(), buffer.data() + buffer.size() - 1ul);
    }

    ~CharOutputStreamBuf() {
        CharOutputStreamBuf::sync();
    }

    std::streamsize xsputn(const CharType* source, std::streamsize size) override {
        // Write a sequence of characters
        if (size <= 0) {
            return 0;
        }

        // If there's space, write data into buffer
        const std::ptrdiff_t spaceLeft = epptr() - pptr();
        if (size < spaceLeft) {
            std::memcpy(pptr(), source, static_cast<std::size_t>(size));
            pbump(static_cast<IntType>(size));
            return size;
        }

        // No space in buffer, flush it first, and write directly into stream
        if (sync()) {
            return 0;
        }
        return static_cast<std::streamsize>(stream->write(source, static_cast<std::size_t>(size)));
    }

    IntType overflow(IntType value) override {
        // Write a single character
        if (value == TraitsType::eof()) {
            return value;
        }

        const CharType data = static_cast<CharType>(value);
        *pptr() = data;
        pbump(1);
        return (sync() ? TraitsType::eof() : value);
    }

    IntType sync() override {
        const std::ptrdiff_t diff = pptr() - pbase();
        if (diff > 0) {
            const std::size_t bytesToWrite = static_cast<std::size_t>(diff);
            const std::size_t bytesWritten = stream->write(buffer.data(), bytesToWrite);
            pbump(static_cast<IntType>(-diff));
            return (bytesToWrite == bytesWritten ? 0 : -1);
        }
        return 0;
    }

private:
    TStream* stream;
    std::array<char, BufferSize> buffer;
};

}  // namespace terse
