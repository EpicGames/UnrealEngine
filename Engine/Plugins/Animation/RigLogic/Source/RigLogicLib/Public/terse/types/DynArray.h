// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstddef>
#include <cstring>
#include <functional>
#include <iterator>
#include <memory>
#if defined(_MSC_VER) && !defined(__clang__) && (_MSC_VER < 1938) && (_MSC_VER >= 1900) && (__cplusplus >= 202002L)
    #include <span>
#endif
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

/**
 * @brief Resizable array-like abstraction for trivial-types only.
 */
template<typename T, class TAllocator>
class DynArray {
public:
    static_assert(std::is_trivial<T>::value, "Uninitialized vector doesn't support non-trivial types.");

    using value_type = T;
    using allocator_type = TAllocator;

private:
    using pointer_type = std::unique_ptr<value_type, std::function<void(value_type*)>>;

public:
    explicit DynArray(const allocator_type& allocator) :
        alloc{allocator},
        sz{},
        ptr{nullptr} {
    }

    DynArray() :
        DynArray{allocator_type{}} {
    }

    DynArray(std::size_t size, const allocator_type& allocator = allocator_type{}) :
        alloc{allocator},
        sz{size},
        ptr{create(sz, alloc)} {
    }

    DynArray(std::size_t size, const value_type& value, const allocator_type& allocator = allocator_type{}) :
        DynArray{size, allocator} {

        std::fill_n(data(), size, value);
    }

    DynArray(const value_type* source, std::size_t size, const allocator_type& allocator = allocator_type{}) :
        DynArray{size, allocator} {

        if ((data() != nullptr) && (source != nullptr)) {
            std::memcpy(data(), source, size * sizeof(value_type));
        }
    }

    template<typename TIterator>
    DynArray(TIterator start, TIterator end, const allocator_type& allocator = allocator_type{}) :
        DynArray{static_cast<std::size_t>(std::distance(start, end)), allocator} {

#if defined(_MSC_VER) && !defined(__clang__) && (_MSC_VER < 1938)
        if (size() != 0ul) {
    #if (_MSC_VER >= 1900) && (__cplusplus >= 202002L)
            std::copy(start, end, std::span{data(), size()}.begin());
    #else
            std::copy(start, end, stdext::checked_array_iterator<value_type*>(data(), size()));
    #endif
        }
#else
        std::copy(start, end, data());
#endif
    }

    ~DynArray() = default;

    DynArray(const DynArray& rhs) :
        DynArray{rhs.size(), rhs.get_allocator()} {
        if ((data() != nullptr) && (rhs.data() != nullptr)) {
            std::memcpy(data(), rhs.data(), rhs.size() * sizeof(value_type));
        }
    }

    DynArray& operator=(const DynArray& rhs) {
        DynArray tmp{rhs};
        std::swap(alloc, tmp.alloc);
        std::swap(ptr, tmp.ptr);
        std::swap(sz, tmp.sz);
        return *this;
    }

    DynArray(DynArray&& rhs) noexcept :
        alloc{},
        sz{},
        ptr{} {

        std::swap(alloc, rhs.alloc);
        std::swap(ptr, rhs.ptr);
        std::swap(sz, rhs.sz);
    }

    DynArray& operator=(DynArray&& rhs) noexcept {
        std::swap(alloc, rhs.alloc);
        std::swap(ptr, rhs.ptr);
        std::swap(sz, rhs.sz);
        return *this;
    }

    allocator_type get_allocator() const noexcept {
        return alloc;
    }

    void clear() {
        ptr.reset();
        sz = 0ul;
    }

    value_type* data() {
        return ptr.get();
    }

    const value_type* data() const {
        return ptr.get();
    }

    std::size_t size() const {
        return sz;
    }

    bool empty() const {
        return (sz == 0ul);
    }

    value_type& operator[](std::size_t index) {
        assert(index < size());
        return data()[index];
    }

    const value_type& operator[](std::size_t index) const {
        assert(index < size());
        return data()[index];
    }

    value_type* begin() {
        return data();
    }

    value_type* end() {
        return data() + sz;
    }

    const value_type* cbegin() const {
        return data();
    }

    const value_type* cend() const {
        return data() + sz;
    }

    const value_type* begin() const {
        return cbegin();
    }

    const value_type* end() const {
        return cend();
    }

    void resize(std::size_t size, const value_type& value) {
        if (size > sz) {
            pointer_type old{ptr.release(), [this](value_type* p) { alloc.deallocate(p, sz); }};
            ptr = create(size, alloc);
            assert(ptr != nullptr);
            if (old != nullptr) {
                std::memcpy(ptr.get(), old.get(), sz * sizeof(value_type));
            }
            std::fill_n(begin() + sz, size - sz, value);
        }
        sz = size;
    }

    void resize(std::size_t size) {
        resize(size, value_type{});
    }

    void resize_uninitialized(std::size_t size) {
        if (size > sz) {
            pointer_type old{ptr.release(), [this](value_type* p) { alloc.deallocate(p, sz); }};
            ptr = create(size, alloc);
            assert(ptr != nullptr);
            if (old != nullptr) {
                std::memcpy(ptr.get(), old.get(), sz * sizeof(value_type));
            }
        }
        sz = size;
    }

    template<typename TIterator>
    void assign(TIterator start, TIterator end) {
        resize_uninitialized(static_cast<std::size_t>(std::distance(start, end)));
#if defined(_MSC_VER) && !defined(__clang__) && (_MSC_VER < 1938)
        if (size() != 0ul) {
    #if (_MSC_VER >= 1900) && (__cplusplus >= 202002L)
            std::copy(start, end, std::span{data(), size()}.begin());
    #else
            std::copy(start, end, stdext::checked_array_iterator<value_type*>(data(), size()));
    #endif
        }
#else
        std::copy(start, end, data());
#endif
    }

private:
    static pointer_type create(std::size_t size, allocator_type alloc) {
        return pointer_type{alloc.allocate(size), [alloc, size](value_type* p) mutable { alloc.deallocate(p, size); }};
    }

private:
    allocator_type alloc;
    std::size_t sz;
    pointer_type ptr;
};

}  // namespace terse
