// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace terse {

// Textual serialization (currently JSON only) generates a pair of braces (`{}`) around every type
// that has a load, save or serialize member or free function, capable of serializing the type in question.
// In some cases, with lots of nested structures, for a simpler textual representation, some of these
// layers of abstractions might be preferable to be ignored. The Transparent<T> wrapper type is used for
// exactly that purpose. By wrapping any member from within a serializer function, that member will be
// treated as transparent, thus avoiding the generation of a pair of braces for it.
template<typename T>
struct Transparent {
    using WrappedType = T;
    WrappedType& data;
};

template<typename T>
Transparent<T> transparent(T& data) {
    return Transparent<T>{data};
}

}  // namespace terse
