// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

namespace terse {

// Convenience type for users to define version tags (Version<1>, Version<2>, ...), but
// it's usage is not mandatory, the user can define custom version tags on it's own, e.g.:
// struct v1 {};
// struct v2 {};
// both being also valid version tags, that may be utilized with Terse.
template<std::uint64_t V>
struct Version {
    static constexpr std::uint64_t value() {
        return V;
    }
};

// This is a wrapper type that is able to alter the serialization behavior, as in, it bundles
// a serializable object together with a version tag, and dispatches to different serialization
// functions, based on the bundled version tag.
template<typename T, typename V>
struct Versioned {
    using WrappedType = T;
    using TypeVersion = V;
    WrappedType& data;
};

template<typename T, typename V>
Versioned<T, V> versioned(T& dest, V /*unused*/) {
    return Versioned<T, V>{dest};
}

}  // namespace terse
