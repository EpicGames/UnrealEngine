// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cstddef>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

static const char* alphabet = "ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789+/";

constexpr std::size_t base64encode(std::size_t size) {
    return ((4ul * size / 3ul) + 3ul) & ~3ul;
}

// destination should be of length ((4 * size / 3) + 3) & ~3
inline std::size_t base64encode(char* destination, const char* source, std::size_t size) {
    char* out = destination;

    int val = 0;
    int valb = -6;

    for (std::size_t pos = {}; pos < size; ++pos) {
        const auto c = static_cast<unsigned char>(source[pos]);
        val = (val << 8) + c;
        valb += 8;
        while (valb >= 0) {
            *out++ = (alphabet[(val >> valb) & 0x3F]);
            valb -= 6;
        }
    }

    if (valb > -6) {
        *out++ = alphabet[((val << 8) >> (valb + 8)) & 0x3F];
    }

    while (static_cast<std::size_t>(out - destination) % 4) {
        *out++ = '=';
    }

    // Length of base64encoded data
    return static_cast<std::size_t>(out - destination);
}

constexpr std::size_t base64decode(std::size_t size) {
    return (size * 3ul) / 4ul;
}

// destination should be of length (size / 4) * 3
inline std::size_t base64decode(char* destination, const char* source, std::size_t size) {
    char* out = destination;

    int buffer[256];
    std::fill_n(buffer, 256, -1);
    for (int i = 0; i < 64; i++) {
        buffer[static_cast<std::size_t>(alphabet[i])] = i;
    }

    int val = 0;
    int valb = -8;

    for (std::size_t pos = {}; pos < size; ++pos) {
        const auto c = static_cast<unsigned char>(source[pos]);
        if (buffer[c] == -1) {
            break;
        }
        val = (val << 6) + buffer[c];
        valb += 6;
        if (valb >= 0) {
            *out++ = static_cast<char>((val >> valb) & 0xFF);
            valb -= 8;
        }
    }

    // Length of base64decoded data
    return static_cast<std::size_t>(out - destination);
}

}  // namespace terse
