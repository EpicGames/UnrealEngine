// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef TERSE_ENABLE_SSE
    #define ENABLE_SSE_BSWAP
#endif  // TERSE_ENABLE_SSE

#include "terse/utils/Endianness.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <type_traits>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace terse {

namespace traits {

template<std::size_t size>
struct uint_of_size {
    using type = typename std::conditional<
        (size == 1ul),
        std::uint8_t,
        typename std::conditional<(size == 2ul),
                                  std::uint16_t,
                                  typename std::conditional<(size <= 4ul), std::uint32_t, std::uint64_t>::type>::type>::type;
};

}  // namespace traits

namespace impl {

struct block128 {

    static constexpr std::size_t alignment() {
#if defined(__arm__) || defined(__aarch64__) || defined(_M_ARM) || defined(_M_ARM64)
        return std::alignment_of<std::max_align_t>::value;
#else
        return 16ul;
#endif
    }
};

}  // namespace impl

enum class Endianness {
    Little,
    Big,
    Network = Big
};

template<Endianness EByteOrder>
struct ByteSwapper;

template<>
struct ByteSwapper<Endianness::Little> {

    template<typename T>
    static void swap(T& value) {
#ifdef TARGET_LITTLE_ENDIAN
        static_cast<void>(value);
#else
        using UIntType = typename traits::uint_of_size<sizeof(T)>::type;
        static_assert(sizeof(T) == sizeof(UIntType), "No matching unsigned integral type found for the given type.");
        // Using memcpy is the only well-defined way of reconstructing arbitrary types from raw bytes.
        // The seemingly unnecessary copies and memcpy calls are all optimized away,
        // compiler knows what's up.
        UIntType swapped;
        std::memcpy(&swapped, &value, sizeof(T));
        swapped = bswap(swapped);
        std::memcpy(&value, &swapped, sizeof(T));
#endif  // TARGET_LITTLE_ENDIAN
    }

    template<typename T>
    static void swap(T* values) {
#ifdef TARGET_LITTLE_ENDIAN
        static_cast<void>(values);
#else
        using UIntType = typename traits::uint_of_size<sizeof(T)>::type;
        static_assert(sizeof(T) == sizeof(UIntType), "No matching unsigned integral type found for the given type.");
        // Using memcpy is the only well-defined way of reconstructing arbitrary types from raw bytes.
        // The seemingly unnecessary copies and memcpy calls are all optimized away,
        // compiler knows what's up.
        alignas(impl::block128::alignment()) UIntType swapped[16ul / sizeof(T)];
        std::memcpy(static_cast<UIntType*>(swapped), values, 16ul);
        bswap(static_cast<UIntType*>(swapped));
        std::memcpy(values, static_cast<UIntType*>(swapped), 16ul);
#endif  // TARGET_LITTLE_ENDIAN
    }
};

template<>
struct ByteSwapper<Endianness::Big> {

    template<typename T>
    static void swap(T& value) {
#ifdef TARGET_LITTLE_ENDIAN
        using UIntType = typename traits::uint_of_size<sizeof(T)>::type;
        static_assert(sizeof(T) == sizeof(UIntType), "No matching unsigned integral type found for the given type.");
        // Using memcpy is the only well-defined way of reconstructing arbitrary types from raw bytes.
        // The seemingly unnecessary copies and memcpy calls are all optimized away,
        // compiler knows what's up.
        UIntType swapped;
        std::memcpy(&swapped, &value, sizeof(T));
        swapped = bswap(swapped);
        std::memcpy(&value, &swapped, sizeof(T));
#else
        static_cast<void>(value);
#endif  // TARGET_LITTLE_ENDIAN
    }

    template<typename T>
    static void swap(T* values) {
#ifdef TARGET_LITTLE_ENDIAN
        using UIntType = typename traits::uint_of_size<sizeof(T)>::type;
        static_assert(sizeof(T) == sizeof(UIntType), "No matching unsigned integral type found for the given type.");
        // Using memcpy is the only well-defined way of reconstructing arbitrary types from raw bytes.
        // The seemingly unnecessary copies and memcpy calls are all optimized away,
        // compiler knows what's up.
        alignas(impl::block128::alignment()) UIntType swapped[16ul / sizeof(T)];
        std::memcpy(static_cast<UIntType*>(swapped), values, 16ul);
        bswap(static_cast<UIntType*>(swapped));
        std::memcpy(values, static_cast<UIntType*>(swapped), 16ul);
#else
        static_cast<void>(values);
#endif  // TARGET_LITTLE_ENDIAN
    }
};

template<Endianness EByteOrder>
using SwapTo = ByteSwapper<EByteOrder>;

template<Endianness EByteOrder>
using SwapFrom = ByteSwapper<EByteOrder>;

template<typename T>
inline void networkToHost(T& value) {
    SwapFrom<Endianness::Network>::swap(value);
}

template<typename T>
inline void networkToHost128(T* values) {
    SwapFrom<Endianness::Network>::swap(values);
}

template<typename T>
inline void hostToNetwork(T& value) {
    SwapTo<Endianness::Network>::swap(value);
}

template<typename T>
inline void hostToNetwork128(T* values) {
    SwapTo<Endianness::Network>::swap(values);
}

}  // namespace terse
