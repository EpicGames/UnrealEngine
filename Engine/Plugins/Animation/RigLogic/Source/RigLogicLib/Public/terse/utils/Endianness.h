// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

#if defined(ENABLE_SSE_BSWAP)
    #include <immintrin.h>
#endif

#if !defined(NO_ENDIAN_H)
    #if defined(USE_ENDIAN_H) || defined(USE_MACHINE_ENDIAN_H) || defined(USE_SYS_ENDIAN_H) || defined(USE_SYS_ISA_DEFS_H) ||    \
        defined(USE_SYS_PARAM_H)
        #define OVERRIDDEN_ENDIAN_H
    #endif

    #if !defined(OVERRIDDEN_ENDIAN_H)
        #if defined(__linux__) || defined(__GLIBC__) || defined(__CYGWIN__) || defined(__ANDROID__)
            #define USE_ENDIAN_H
        #elif defined(__APPLE__)
            #define USE_MACHINE_ENDIAN_H
        #elif defined(__FreeBSD__) || defined(__NetBSD__) || defined(__OpenBSD__)
            #define USE_SYS_ENDIAN_H
        #elif defined(__sun)
            #define USE_SYS_ISA_DEFS_H
        #elif defined(__MINGW32__) || defined(__MINGW64__) || !(defined(_WIN64) || defined(_WIN32))
            #define USE_SYS_PARAM_H
        #endif
    #endif

    #if defined(USE_ENDIAN_H)
        #include <endian.h>
    #elif defined(USE_MACHINE_ENDIAN_H)
        #include <machine/endian.h>
    #elif defined(USE_SYS_ENDIAN_H)
        #include <sys/endian.h>
    #elif defined(USE_SYS_ISA_DEFS_H)
        #include <sys/isa_defs.h>
    #elif defined(USE_SYS_PARAM_H)
        #include <sys/param.h>
    #endif
#endif

#if !defined(TARGET_LITTLE_ENDIAN) && !defined(TARGET_BIG_ENDIAN)
    #if (defined(__BYTE_ORDER__) && __BYTE_ORDER__ == __ORDER_LITTLE_ENDIAN__) /*GCC*/ ||                                        \
        (defined(__BYTE_ORDER) && __BYTE_ORDER == __LITTLE_ENDIAN) /*Linux*/ ||                                                  \
        (defined(_BYTE_ORDER) && _BYTE_ORDER == _LITTLE_ENDIAN) /*xBSD,Sun*/ ||                                                  \
        (defined(BYTE_ORDER) && BYTE_ORDER == LITTLE_ENDIAN) /*Apple,MingW*/ || defined(__LITTLE_ENDIAN__) /*GCC Mac*/ ||        \
        defined(__ARMEL__) /*GCC,Clang*/ || defined(__THUMBEL__) /*GCC,Clang*/ || defined(__AARCH64EL__) /*GCC,Clang*/ ||        \
        defined(_MIPSEL) /*GCC,Clang*/ || defined(__MIPSEL) /*GCC,Clang*/ || defined(__MIPSEL__) /*GCC,Clang*/ ||                \
        defined(_M_IX86) /*MSVC*/ || defined(_M_X64) /*MSVC*/ || defined(_M_IA64) /*MSVC*/ || defined(_M_AMD64) /*MSVC*/ ||      \
        defined(_M_ARM) /*MSVC*/ || defined(_M_ARM64) /*MSVC*/
        #define TARGET_LITTLE_ENDIAN
    #elif (defined(__BYTE_ORDER__) && __BYTE_ORDER__ == __ORDER_BIG_ENDIAN__) /*GCC*/ ||                                         \
        (defined(__BYTE_ORDER) && __BYTE_ORDER == __BIG_ENDIAN) /*Linux*/ ||                                                     \
        (defined(_BYTE_ORDER) && _BYTE_ORDER == _BIG_ENDIAN) /*xBSD,Sun*/ ||                                                     \
        (defined(BYTE_ORDER) && BYTE_ORDER == BIG_ENDIAN) /*Apple,MingW*/ || defined(_M_PPC) /*MSVC for XBox-360*/ ||            \
        defined(__BIG_ENDIAN__) /*GCC Mac*/ || defined(__ARMEB__) /*GCC,Clang*/ || defined(__THUMBEB__) /*GCC,Clang*/ ||         \
        defined(__AARCH64EB__) /*GCC,Clang*/ || defined(_MIPSEB) /*GCC,Clang*/ || defined(__MIPSEB) /*GCC,Clang*/ ||             \
        defined(__MIPSEB__) /*GCC,Clang*/
        #define TARGET_BIG_ENDIAN
    #elif defined(_WIN32)
        #define TARGET_LITTLE_ENDIAN
    #endif  // End of byte order checks
#endif      // End of guard for explicitly defined endianness

/*
 * Swap intrinsics
 */
#if defined(__clang__) || (defined(__GNUC__) && ((__GNUC__ == 4 && __GNUC_MINOR__ >= 8) || __GNUC__ >= 5))
    #define bswap16(x) __builtin_bswap16((x))
    #define bswap32(x) __builtin_bswap32((x))
    #define bswap64(x) __builtin_bswap64((x))
#elif defined(__linux__) || defined(__GLIBC__)
    #include <byteswap.h>
    #define bswap16(x) bswap_16((x))
    #define bswap32(x) bswap_32((x))
    #define bswap64(x) bswap_64((x))
#elif defined(_MSC_VER)
    #include <stdlib.h>
    #define bswap16(x) _byteswap_ushort((x))
    #define bswap32(x) _byteswap_ulong((x))
    #define bswap64(x) _byteswap_uint64((x))
#elif defined(__APPLE__)
    #include <libkern/OSByteOrder.h>
    #define bswap16(x) OSSwapInt16((x))
    #define bswap32(x) OSSwapInt32((x))
    #define bswap64(x) OSSwapInt64((x))
#elif defined(__FreeBSD__) || defined(__NetBSD__)
    #include <sys/endian.h>  // This defines the intrinsics as per the chosen naming convention
#elif defined(__OpenBSD__)
    #include <sys/endian.h>
    #define bswap16(x) swap16((x))
    #define bswap32(x) swap32((x))
    #define bswap64(x) swap64((x))
#elif defined(__sun) || defined(sun)
    #include <sys/byteorder.h>
    #define bswap16(x) BSWAP_16((x))
    #define bswap32(x) BSWAP_32((x))
    #define bswap64(x) BSWAP_64((x))
#else
static inline std::uint16_t bswap16(std::uint16_t x) {
    return (((x & std::uint16_t{0x00FF}) << 8) | ((x & std::uint16_t{0xFF00}) >> 8));
}

static inline std::uint32_t bswap32(std::uint32_t x) {
    return (((x & std::uint32_t{0x000000FF}) << 24) | ((x & std::uint32_t{0x0000FF00}) << 8) |
            ((x & std::uint32_t{0x00FF0000}) >> 8) | ((x & std::uint32_t{0xFF000000}) >> 24));
}

static inline std::uint64_t bswap64(std::uint64_t x) {
    return (((x & std::uint64_t{0x00000000000000FF}) << 56) | ((x & std::uint64_t{0x000000000000FF00}) << 40) |
            ((x & std::uint64_t{0x0000000000FF0000}) << 24) | ((x & std::uint64_t{0x00000000FF000000}) << 8) |
            ((x & std::uint64_t{0x000000FF00000000}) >> 8) | ((x & std::uint64_t{0x0000FF0000000000}) >> 24) |
            ((x & std::uint64_t{0x00FF000000000000}) >> 40) | ((x & std::uint64_t{0xFF00000000000000}) >> 56));
}

#endif

#if defined(ENABLE_SSE_BSWAP)

static inline void bswap16x8(std::uint16_t* source) {
    const __m128i v = _mm_load_si128(reinterpret_cast<__m128i*>(source));
    const __m128i swapped = _mm_or_si128(_mm_slli_epi16(v, 8), _mm_srli_epi16(v, 8));
    _mm_store_si128(reinterpret_cast<__m128i*>(source), swapped);
}

static inline void bswap32x4(std::uint32_t* source) {
    const __m128i v = _mm_load_si128(reinterpret_cast<__m128i*>(source));
    const __m128i swapped = _mm_shuffle_epi8(v, _mm_set_epi8(12, 13, 14, 15, 8, 9, 10, 11, 4, 5, 6, 7, 0, 1, 2, 3));
    _mm_store_si128(reinterpret_cast<__m128i*>(source), swapped);
}

static inline void bswap64x2(std::uint64_t* source) {
    const __m128i v = _mm_load_si128(reinterpret_cast<__m128i*>(source));
    const __m128i swapped = _mm_shuffle_epi8(v, _mm_set_epi8(8, 9, 10, 11, 12, 13, 14, 15, 0, 1, 2, 3, 4, 5, 6, 7));
    _mm_store_si128(reinterpret_cast<__m128i*>(source), swapped);
}

#else

static inline void bswap16x8(std::uint16_t* source) {
    source[0] = bswap16(source[0]);
    source[1] = bswap16(source[1]);
    source[2] = bswap16(source[2]);
    source[3] = bswap16(source[3]);
    source[4] = bswap16(source[4]);
    source[5] = bswap16(source[5]);
    source[6] = bswap16(source[6]);
    source[7] = bswap16(source[7]);
}

static inline void bswap32x4(std::uint32_t* source) {
    source[0] = bswap32(source[0]);
    source[1] = bswap32(source[1]);
    source[2] = bswap32(source[2]);
    source[3] = bswap32(source[3]);
}

static inline void bswap64x2(std::uint64_t* source) {
    source[0] = bswap64(source[0]);
    source[1] = bswap64(source[1]);
}

#endif  // ENABLE_SSE_BSWAP

// Target architecture specific ntoh and hton for all relevant sizes
// In case of big endian architectures this is a noop
#if defined(TARGET_LITTLE_ENDIAN)
    #define ntoh8(x) (x)
    #define hton8(x) (x)
    #define ntoh16(x) bswap16((x))
    #define hton16(x) bswap16((x))
    #define ntoh32(x) bswap32((x))
    #define hton32(x) bswap32((x))
    #define ntoh64(x) bswap64((x))
    #define hton64(x) bswap64((x))
    // Vectorized swap
    #define ntoh8x16(x) (x)
    #define hton8x16(x) (x)
    #define ntoh16x8(x) bswap16x8((x))
    #define hton16x8(x) bswap16x8((x))
    #define ntoh32x4(x) bswap32x4((x))
    #define hton32x4(x) bswap32x4((x))
    #define ntoh64x2(x) bswap64x2((x))
    #define hton64x2(x) bswap64x2((x))
#elif defined(TARGET_BIG_ENDIAN)
    #define ntoh8(x) (x)
    #define hton8(x) (x)
    #define ntoh16(x) (x)
    #define hton16(x) (x)
    #define ntoh32(x) (x)
    #define hton32(x) (x)
    #define ntoh64(x) (x)
    #define hton64(x) (x)
    // Vectorized swap
    #define ntoh8x16(x) (x)
    #define hton8x16(x) (x)
    #define ntoh16x8(x) (x)
    #define hton16x8(x) (x)
    #define ntoh32x4(x) (x)
    #define hton32x4(x) (x)
    #define ntoh64x2(x) (x)
    #define hton64x2(x) (x)
#else
    #error "Platform not supported, no byte swap functions defined."
#endif

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wunused-value"
#endif

// Process single values

inline std::uint8_t ntoh(std::uint8_t x) {
    return ntoh8(x);
}

inline std::uint16_t ntoh(std::uint16_t x) {
    return ntoh16(x);
}

inline std::uint32_t ntoh(std::uint32_t x) {
    return ntoh32(x);
}

inline std::uint64_t ntoh(std::uint64_t x) {
    return ntoh64(x);
}

inline std::uint8_t hton(std::uint8_t x) {
    return hton8(x);
}

inline std::uint16_t hton(std::uint16_t x) {
    return hton16(x);
}

inline std::uint32_t hton(std::uint32_t x) {
    return hton32(x);
}

inline std::uint64_t hton(std::uint64_t x) {
    return hton64(x);
}

// Process multiple blocks simultaneously

inline void ntoh(std::uint8_t* x) {
    ntoh8x16(x);
}

inline void ntoh(std::uint16_t* x) {
    ntoh16x8(x);
}

inline void ntoh(std::uint32_t* x) {
    ntoh32x4(x);
}

inline void ntoh(std::uint64_t* x) {
    ntoh64x2(x);
}

inline void hton(std::uint8_t* x) {
    hton8x16(x);
}

inline void hton(std::uint16_t* x) {
    hton16x8(x);
}

inline void hton(std::uint32_t* x) {
    hton32x4(x);
}

inline void hton(std::uint64_t* x) {
    hton64x2(x);
}

// Byte swap overloads

inline std::uint8_t bswap(std::uint8_t x) {
    // No operation
    return x;
}

inline std::uint16_t bswap(std::uint16_t x) {
    return bswap16(x);
}

inline std::uint32_t bswap(std::uint32_t x) {
    return bswap32(x);
}

inline std::uint64_t bswap(std::uint64_t x) {
    return bswap64(x);
}

inline void bswap(std::uint8_t* x) {
    // No operation
    static_cast<void>(x);
}

inline void bswap(std::uint16_t* x) {
    bswap16x8(x);
}

inline void bswap(std::uint32_t* x) {
    bswap32x4(x);
}

inline void bswap(std::uint64_t* x) {
    bswap64x2(x);
}

#if defined(__clang__) || defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
