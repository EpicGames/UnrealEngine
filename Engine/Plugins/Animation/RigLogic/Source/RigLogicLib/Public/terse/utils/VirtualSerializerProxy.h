// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace terse {

template<class Serializable>
class VirtualSerializerProxy {
public:
    explicit VirtualSerializerProxy(Serializable* instance_) :
        instance{instance_} {
    }

    template<class TArchive>
    void load(TArchive& archive) {
        instance->load(archive);
    }

    template<class TArchive>
    void save(TArchive& archive) {
        instance->save(archive);
    }

private:
    Serializable* instance;
};

}  // namespace terse
