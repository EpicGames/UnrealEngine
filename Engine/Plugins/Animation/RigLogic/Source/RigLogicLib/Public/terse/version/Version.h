// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define TERSE_MAJOR_VERSION 3
#define TERSE_MINOR_VERSION 1
#define TERSE_PATCH_VERSION 15
#define TERSE_VERSION_STRING "3.1.15"
