// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef TRIMD_ENABLE_AVX
    #include "trimd/Polyfill.h"

    #include <immintrin.h>

namespace trimd {

namespace avx {

struct F256 {
    using value_type = float;

    __m256 data;

    F256() :
        data{_mm256_setzero_ps()} {
    }

    explicit F256(__m256 value) :
        data{value} {
    }

    explicit F256(float value) :
        F256{_mm256_set1_ps(value)} {
    }

    F256(float v1, float v2, float v3, float v4, float v5, float v6, float v7, float v8) :
        data{_mm256_set_ps(v8, v7, v6, v5, v4, v3, v2, v1)} {
    }

    static F256 fromAlignedSource(const float* source) {
        return F256{_mm256_load_ps(source)};
    }

    static F256 fromUnalignedSource(const float* source) {
        return F256{_mm256_loadu_ps(source)};
    }

    static F256 loadSingleValue(const float* source) {
        const __m256i mask = _mm256_set_epi32(0, 0, 0, 0, 0, 0, 0, -1);
        return F256{_mm256_maskload_ps(source, mask)};
    }

    #ifdef TRIMD_ENABLE_F16C
    static F256 fromAlignedSource(const std::uint16_t* source);
    static F256 fromUnalignedSource(const std::uint16_t* source);
    static F256 loadSingleValue(const std::uint16_t* source);
    #endif  // TRIMD_ENABLE_F16C

    template<typename T>
    static void prefetchT0(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_T0);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    template<typename T>
    static void prefetchT1(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_T1);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    template<typename T>
    static void prefetchT2(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_T2);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    template<typename T>
    static void prefetchNTA(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_NTA);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    void alignedLoad(const float* source) {
        data = _mm256_load_ps(source);
    }

    void unalignedLoad(const float* source) {
        data = _mm256_loadu_ps(source);
    }

    void alignedStore(float* dest) const {
        _mm256_store_ps(dest, data);
    }

    void unalignedStore(float* dest) const {
        _mm256_storeu_ps(dest, data);
    }

    #ifdef TRIMD_ENABLE_F16C
    void alignedLoad(const std::uint16_t* source);
    void unalignedLoad(const std::uint16_t* source);
    void alignedStore(std::uint16_t* dest) const;
    void unalignedStore(std::uint16_t* dest) const;
    #endif  // TRIMD_ENABLE_F16C

    float sum() const {
        // (data[3] + data[7], data[2] + data[6], data[1] + data[5], data[0] + data[4])
        const __m128 x128 = _mm_add_ps(_mm256_extractf128_ps(data, 1), _mm256_castps256_ps128(data));
        // (-, -, data[1] + data[3] + data[5] + data[7], data[0] + data[2] + data[4] + data[6])
        const __m128 x64 = _mm_add_ps(x128, _mm_movehl_ps(x128, x128));
        // (-, -, -, data[0] + data[1] + data[2] + data[3] + data[4] + data[5] + data[6] + data[7])
        const __m128 x32 = _mm_add_ss(x64, _mm_shuffle_ps(x64, x64, 0x55));
        return _mm_cvtss_f32(x32);
    }

    F256& operator+=(const F256& rhs) {
        data = _mm256_add_ps(data, rhs.data);
        return *this;
    }

    F256& operator-=(const F256& rhs) {
        data = _mm256_sub_ps(data, rhs.data);
        return *this;
    }

    F256& operator*=(const F256& rhs) {
        data = _mm256_mul_ps(data, rhs.data);
        return *this;
    }

    F256& operator/=(const F256& rhs) {
        data = _mm256_div_ps(data, rhs.data);
        return *this;
    }

    F256& operator&=(const F256& rhs) {
        data = _mm256_and_ps(data, rhs.data);
        return *this;
    }

    F256& operator|=(const F256& rhs) {
        data = _mm256_or_ps(data, rhs.data);
        return *this;
    }

    F256& operator^=(const F256& rhs) {
        data = _mm256_xor_ps(data, rhs.data);
        return *this;
    }

    static constexpr std::size_t size() {
        return sizeof(decltype(data)) / sizeof(float);
    }

    static constexpr std::size_t alignment() {
        return alignof(decltype(data));
    }
};

    #ifdef TRIMD_ENABLE_F16C

TRIMD_TARGET_F16C inline F256 F256::fromAlignedSource(const std::uint16_t* source) {
    return F256{_mm256_cvtph_ps(_mm_load_si128(reinterpret_cast<const __m128i*>(source)))};
}

TRIMD_TARGET_F16C inline F256 F256::fromUnalignedSource(const std::uint16_t* source) {
    return F256{_mm256_cvtph_ps(_mm_loadu_si128(reinterpret_cast<const __m128i*>(source)))};
}

TRIMD_TARGET_F16C inline F256 F256::loadSingleValue(const std::uint16_t* source) {
    return F256{_mm256_cvtph_ps(_mm_loadu_si16(source))};
}

TRIMD_TARGET_F16C inline void F256::alignedLoad(const std::uint16_t* source) {
    data = _mm256_cvtph_ps(_mm_load_si128(reinterpret_cast<const __m128i*>(source)));
}

TRIMD_TARGET_F16C inline void F256::unalignedLoad(const std::uint16_t* source) {
    data = _mm256_cvtph_ps(_mm_loadu_si128(reinterpret_cast<const __m128i*>(source)));
}

TRIMD_TARGET_F16C inline void F256::alignedStore(std::uint16_t* dest) const {
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic push
            #pragma GCC diagnostic ignored "-Wold-style-cast"
            #if !defined(__clang__)
                #pragma GCC diagnostic ignored "-Wuseless-cast"
            #endif
        #endif
    _mm_store_si128(reinterpret_cast<__m128i*>(dest), _mm256_cvtps_ph(data, _MM_FROUND_CUR_DIRECTION));
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic pop
        #endif
}

TRIMD_TARGET_F16C inline void F256::unalignedStore(std::uint16_t* dest) const {
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic push
            #pragma GCC diagnostic ignored "-Wold-style-cast"
            #if !defined(__clang__)
                #pragma GCC diagnostic ignored "-Wuseless-cast"
            #endif
        #endif
    _mm_storeu_si128(reinterpret_cast<__m128i*>(dest), _mm256_cvtps_ph(data, _MM_FROUND_CUR_DIRECTION));
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic pop
        #endif
}

    #endif  // TRIMD_ENABLE_F16C

inline F256 operator==(const F256& lhs, const F256& rhs) {
    return F256{_mm256_cmp_ps(lhs.data, rhs.data, _CMP_EQ_OQ)};
}

inline F256 operator!=(const F256& lhs, const F256& rhs) {
    return F256{_mm256_cmp_ps(lhs.data, rhs.data, _CMP_NEQ_OQ)};
}

inline F256 operator<(const F256& lhs, const F256& rhs) {
    return F256{_mm256_cmp_ps(lhs.data, rhs.data, _CMP_LT_OQ)};
}

inline F256 operator<=(const F256& lhs, const F256& rhs) {
    return F256{_mm256_cmp_ps(lhs.data, rhs.data, _CMP_LE_OQ)};
}

inline F256 operator>(const F256& lhs, const F256& rhs) {
    return F256{_mm256_cmp_ps(lhs.data, rhs.data, _CMP_GT_OQ)};
}

inline F256 operator>=(const F256& lhs, const F256& rhs) {
    return F256{_mm256_cmp_ps(lhs.data, rhs.data, _CMP_GE_OQ)};
}

inline F256 operator+(const F256& lhs, const F256& rhs) {
    return F256(lhs) += rhs;
}

inline F256 operator-(const F256& lhs, const F256& rhs) {
    return F256(lhs) -= rhs;
}

inline F256 operator*(const F256& lhs, const F256& rhs) {
    return F256(lhs) *= rhs;
}

inline F256 operator/(const F256& lhs, const F256& rhs) {
    return F256(lhs) /= rhs;
}

inline F256 operator&(const F256& lhs, const F256& rhs) {
    return F256(lhs) &= rhs;
}

inline F256 operator|(const F256& lhs, const F256& rhs) {
    return F256(lhs) |= rhs;
}

inline F256 operator^(const F256& lhs, const F256& rhs) {
    return F256(lhs) ^= rhs;
}

inline F256 operator~(const F256& rhs) {
    F256 result{};
    result = (result == result);  // Generate register with all 1s
    return (result ^= rhs);
}

inline void transpose(F256& row0, F256& row1, F256& row2, F256& row3, F256& row4, F256& row5, F256& row6, F256& row7) {
    __m256 t0 = _mm256_unpacklo_ps(row0.data, row1.data);
    __m256 t1 = _mm256_unpackhi_ps(row0.data, row1.data);
    __m256 t2 = _mm256_unpacklo_ps(row2.data, row3.data);
    __m256 t3 = _mm256_unpackhi_ps(row2.data, row3.data);
    __m256 t4 = _mm256_unpacklo_ps(row4.data, row5.data);
    __m256 t5 = _mm256_unpackhi_ps(row4.data, row5.data);
    __m256 t6 = _mm256_unpacklo_ps(row6.data, row7.data);
    __m256 t7 = _mm256_unpackhi_ps(row6.data, row7.data);
    __m256 tt0 = _mm256_shuffle_ps(t0, t2, _MM_SHUFFLE(1, 0, 1, 0));
    __m256 tt1 = _mm256_shuffle_ps(t0, t2, _MM_SHUFFLE(3, 2, 3, 2));
    __m256 tt2 = _mm256_shuffle_ps(t1, t3, _MM_SHUFFLE(1, 0, 1, 0));
    __m256 tt3 = _mm256_shuffle_ps(t1, t3, _MM_SHUFFLE(3, 2, 3, 2));
    __m256 tt4 = _mm256_shuffle_ps(t4, t6, _MM_SHUFFLE(1, 0, 1, 0));
    __m256 tt5 = _mm256_shuffle_ps(t4, t6, _MM_SHUFFLE(3, 2, 3, 2));
    __m256 tt6 = _mm256_shuffle_ps(t5, t7, _MM_SHUFFLE(1, 0, 1, 0));
    __m256 tt7 = _mm256_shuffle_ps(t5, t7, _MM_SHUFFLE(3, 2, 3, 2));
    row0.data = _mm256_permute2f128_ps(tt0, tt4, 0x20);
    row1.data = _mm256_permute2f128_ps(tt1, tt5, 0x20);
    row2.data = _mm256_permute2f128_ps(tt2, tt6, 0x20);
    row3.data = _mm256_permute2f128_ps(tt3, tt7, 0x20);
    row4.data = _mm256_permute2f128_ps(tt0, tt4, 0x31);
    row5.data = _mm256_permute2f128_ps(tt1, tt5, 0x31);
    row6.data = _mm256_permute2f128_ps(tt2, tt6, 0x31);
    row7.data = _mm256_permute2f128_ps(tt3, tt7, 0x31);
}

inline F256 abs(const F256& rhs) {
    const __m256 ABSMASK = _mm256_castsi256_ps(_mm256_set1_epi32(~(1 << 31)));
    return F256{_mm256_and_ps(ABSMASK, rhs.data)};
}

inline F256 andnot(const F256& lhs, const F256& rhs) {
    return F256{_mm256_andnot_ps(lhs.data, rhs.data)};
}

inline F256 rsqrt(const F256& rhs) {
    #ifndef TRIMD_ENABLE_FAST_INVERSE_SQRT
    return F256{_mm256_rsqrt_ps(rhs.data)};
    #else
    // Pure AVX2-only implementation:
    //
    // const __m256i shifted = _mm256_srli_epi32(_mm256_castps_si256(rhs.data), 1);
    // const __m256i subtracted = _mm256_sub_epi32(_mm256_set1_epi32(0x5f1ffff9), shifted);
    // F256 result{_mm256_castsi256_ps(subtracted)};
    // result *= F256{0.703952253f} * (F256{2.38924456f} - rhs * result * result);
    // return result;
    //
    // Combination of SSE and AVX:
    const __m128 lower = _mm256_castps256_ps128(rhs.data);
    const __m128 upper = _mm256_extractf128_ps(rhs.data, 1);
    const __m128i shiftedLower = _mm_srli_epi32(_mm_castps_si128(lower), 1);
    const __m128i shiftedUpper = _mm_srli_epi32(_mm_castps_si128(upper), 1);
    const __m128i subtractedLower = _mm_sub_epi32(_mm_set1_epi32(0x5f1ffff9), shiftedLower);
    const __m128i subtractedUpper = _mm_sub_epi32(_mm_set1_epi32(0x5f1ffff9), shiftedUpper);
    F256 result{
        _mm256_insertf128_ps(_mm256_castps128_ps256(_mm_castsi128_ps(subtractedLower)), _mm_castsi128_ps(subtractedUpper), 1)};
    result *= F256{0.703952253f} * (F256{2.38924456f} - rhs * result * result);
    return result;
    #endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
}

}  // namespace avx

}  // namespace trimd

#endif  // TRIMD_ENABLE_AVX
