// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trimd {

namespace fallback {

template<typename T128>
struct T256 {
    using value_type = typename T128::value_type;

    T128 data1;
    T128 data2;

    T256(const T128& d1, const T128& d2) :
        data1{d1},
        data2{d2} {
    }

    T256() :
        data1{},
        data2{} {
    }

    T256(value_type v1, value_type v2, value_type v3, value_type v4, value_type v5, value_type v6, value_type v7, value_type v8) :
        data1{v1, v2, v3, v4},
        data2{v5, v6, v7, v8} {
    }

    explicit T256(value_type value) :
        T256{value, value, value, value, value, value, value, value} {
    }

    template<typename U>
    static T256 fromAlignedSource(const U* source) {
        return T256{T128::fromAlignedSource(source), T128::fromAlignedSource(source + T128::size())};
    }

    template<typename U>
    static T256 fromUnalignedSource(const U* source) {
        return T256{T128::fromUnalignedSource(source), T128::fromUnalignedSource(source + T128::size())};
    }

    template<typename U>
    static T256 loadSingleValue(const U* source) {
        return T256{T128::loadSingleValue(source), T128{}};
    }

    template<typename U>
    static void prefetchT0(const U* source) {
        T128::prefetchT0(source);
    }

    template<typename U>
    static void prefetchT1(const U* source) {
        T128::prefetchT1(source);
    }

    template<typename U>
    static void prefetchT2(const U* source) {
        T128::prefetchT2(source);
    }

    template<typename U>
    static void prefetchNTA(const U* source) {
        T128::prefetchNTA(source);
    }

    template<typename U>
    void alignedLoad(const U* source) {
        data1.alignedLoad(source);
        data2.alignedLoad(source + T128::size());
    }

    template<typename U>
    void unalignedLoad(const U* source) {
        data1.unalignedLoad(source);
        data2.unalignedLoad(source + T128::size());
    }

    template<typename U>
    void alignedStore(U* dest) const {
        data1.alignedStore(dest);
        data2.alignedStore(dest + T128::size());
    }

    template<typename U>
    void unalignedStore(U* dest) const {
        data1.unalignedStore(dest);
        data2.unalignedStore(dest + T128::size());
    }

    value_type sum() const {
        return data1.sum() + data2.sum();
    }

    T256& operator+=(const T256& rhs) {
        data1 += rhs.data1;
        data2 += rhs.data2;
        return *this;
    }

    T256& operator-=(const T256& rhs) {
        data1 -= rhs.data1;
        data2 -= rhs.data2;
        return *this;
    }

    T256& operator*=(const T256& rhs) {
        data1 *= rhs.data1;
        data2 *= rhs.data2;
        return *this;
    }

    T256& operator/=(const T256& rhs) {
        data1 /= rhs.data1;
        data2 /= rhs.data2;
        return *this;
    }

    T256& operator&=(const T256& rhs) {
        data1 &= rhs.data1;
        data2 &= rhs.data2;
        return *this;
    }

    T256& operator|=(const T256& rhs) {
        data1 |= rhs.data1;
        data2 |= rhs.data2;
        return *this;
    }

    T256& operator^=(const T256& rhs) {
        data1 ^= rhs.data1;
        data2 ^= rhs.data2;
        return *this;
    }

    static constexpr std::size_t size() {
        return T128::size() * 2ul;
    }

    static constexpr std::size_t alignment() {
        // T128 alignment is the minimal requirement, but it might be beneficial to force here an alignment
        // of a theoretical T256, so the autovectorizer might generate better code on platforms not directly
        // supported by TRiMD.
        return T128::alignment();
    }
};

template<typename T128>
inline T256<T128> operator==(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>{lhs.data1 == rhs.data1, lhs.data2 == rhs.data2};
}

template<typename T128>
inline T256<T128> operator!=(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>{lhs.data1 != rhs.data1, lhs.data2 != rhs.data2};
}

template<typename T128>
inline T256<T128> operator<(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>{lhs.data1 < rhs.data1, lhs.data2 < rhs.data2};
}

template<typename T128>
inline T256<T128> operator<=(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>{lhs.data1 <= rhs.data1, lhs.data2 <= rhs.data2};
}

template<typename T128>
inline T256<T128> operator>(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>{lhs.data1 > rhs.data1, lhs.data2 > rhs.data2};
}

template<typename T128>
inline T256<T128> operator>=(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>{lhs.data1 >= rhs.data1, lhs.data2 >= rhs.data2};
}

template<typename T128>
inline T256<T128> operator+(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>(lhs) += rhs;
}

template<typename T128>
inline T256<T128> operator-(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>(lhs) -= rhs;
}

template<typename T128>
inline T256<T128> operator*(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>(lhs) *= rhs;
}

template<typename T128>
inline T256<T128> operator/(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>(lhs) /= rhs;
}

template<typename T128>
inline T256<T128> operator&(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>(lhs) &= rhs;
}

template<typename T128>
inline T256<T128> operator|(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>(lhs) |= rhs;
}

template<typename T128>
inline T256<T128> operator^(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>(lhs) ^= rhs;
}

template<typename T128>
inline T256<T128> operator~(const T256<T128>& rhs) {
    return T256<T128>{~rhs.data1, ~rhs.data2};
}

template<typename T128>
inline void transpose(T256<T128>& row0,
                      T256<T128>& row1,
                      T256<T128>& row2,
                      T256<T128>& row3,
                      T256<T128>& row4,
                      T256<T128>& row5,
                      T256<T128>& row6,
                      T256<T128>& row7) {
    transpose(row0.data1, row1.data1, row2.data1, row3.data1);
    transpose(row0.data2, row1.data2, row2.data2, row3.data2);
    transpose(row4.data1, row5.data1, row6.data1, row7.data1);
    transpose(row4.data2, row5.data2, row6.data2, row7.data2);
    std::swap(row0.data2, row4.data1);
    std::swap(row1.data2, row5.data1);
    std::swap(row2.data2, row6.data1);
    std::swap(row3.data2, row7.data1);
}

template<typename T128>
inline T256<T128> abs(const T256<T128>& rhs) {
    return T256<T128>{abs(rhs.data1), abs(rhs.data2)};
}

template<typename T128>
inline T256<T128> andnot(const T256<T128>& lhs, const T256<T128>& rhs) {
    return T256<T128>{andnot(lhs.data1, rhs.data1), andnot(lhs.data2, rhs.data2)};
}

template<typename T128>
inline T256<T128> rsqrt(const T256<T128>& rhs) {
    return T256<T128>{rsqrt(rhs.data1), rsqrt(rhs.data2)};
}

}  // namespace fallback

}  // namespace trimd
