// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(FORCE_INLINE)
    #if defined(_MSC_VER)
        #define FORCE_INLINE __forceinline
    #else
        #define FORCE_INLINE inline __attribute__((always_inline))
    #endif
#endif
