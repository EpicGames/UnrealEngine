// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef TRIMD_ENABLE_NEON
    #include "trimd/Fallback.h"
    #include "trimd/Polyfill.h"

    #include <arm_neon.h>

    #include <cstring>

namespace trimd {

namespace neon {

struct F128 {
    using value_type = float;

    float32x4_t data;

    F128() :
        data{vdupq_n_f32(0)} {
    }

    explicit F128(float32x4_t value) :
        data{value} {
    }

    explicit F128(float value) :
        F128{vdupq_n_f32(value)} {
    }

    F128(float v1, float v2, float v3, float v4) {
        const float source[] = {v1, v2, v3, v4};
        data = vld1q_f32(source);
    }

    static F128 fromAlignedSource(const float* source) {
        return F128{vld1q_f32(source)};
    }

    static F128 fromUnalignedSource(const float* source) {
        return F128{vld1q_f32(source)};
    }

    static F128 loadSingleValue(const float* source) {
        return F128{vsetq_lane_f32(*source, vdupq_n_f32(0), 0)};
    }

    #ifdef TRIMD_ENABLE_NEON_FP16
    static F128 fromAlignedSource(const std::uint16_t* source);
    static F128 fromUnalignedSource(const std::uint16_t* source);
    static F128 loadSingleValue(const std::uint16_t* source);
    #endif  // TRIMD_ENABLE_NEON_FP16

    template<typename T>
    static void prefetchT0(const T* source) {
        prefetch_t0(source);
    }

    template<typename T>
    static void prefetchT1(const T* source) {
        prefetch_t1(source);
    }

    template<typename T>
    static void prefetchT2(const T* source) {
        prefetch_t2(source);
    }

    template<typename T>
    static void prefetchNTA(const T* source) {
        prefetch_nta(source);
    }

    void alignedLoad(const float* source) {
        data = vld1q_f32(source);
    }

    void unalignedLoad(const float* source) {
        data = vld1q_f32(source);
    }

    void alignedStore(float* dest) const {
        vst1q_f32(dest, data);
    }

    void unalignedStore(float* dest) const {
        vst1q_f32(dest, data);
    }

    #ifdef TRIMD_ENABLE_NEON_FP16
    void alignedLoad(const std::uint16_t* source);
    void unalignedLoad(const std::uint16_t* source);
    void alignedStore(std::uint16_t* dest) const;
    void unalignedStore(std::uint16_t* dest) const;
    #endif  // TRIMD_ENABLE_NEON_FP16

    float sum() const {
        const float32x2_t tmp = vadd_f32(vget_high_f32(data), vget_low_f32(data));
        return vget_lane_f32(vpadd_f32(tmp, tmp), 0);
    }

    F128& operator+=(const F128& rhs) {
        data = vaddq_f32(data, rhs.data);
        return *this;
    }

    F128& operator-=(const F128& rhs) {
        data = vsubq_f32(data, rhs.data);
        return *this;
    }

    F128& operator*=(const F128& rhs) {
        data = vmulq_f32(data, rhs.data);
        return *this;
    }

    F128& operator/=(const F128& rhs) {
        // reciprocal0 = 1 / rhs (initial estimate)
        const float32x4_t reciprocal0 = vrecpeq_f32(rhs.data);
        // Newton-Raphson step to refine the initial reciprocal estimate.
        // If accuracy is not enough, additional refinement steps may be added (as many as needed)
        // until desired accuracy is reached (just duplicate the below line).
        // reciprocal1 = reciprocal0 * (2.0 - (reciprocal0 * rhs))
        const float32x4_t reciprocal1 = vmulq_f32(reciprocal0, vrecpsq_f32(reciprocal0, rhs.data));
        // data = data * reciprocal1
        data = vmulq_f32(data, reciprocal1);
        return *this;
    }

    F128& operator&=(const F128& rhs) {
        data = vreinterpretq_f32_u32(vandq_u32(vreinterpretq_u32_f32(data), vreinterpretq_u32_f32(rhs.data)));
        return *this;
    }

    F128& operator|=(const F128& rhs) {
        data = vreinterpretq_f32_u32(vorrq_u32(vreinterpretq_u32_f32(data), vreinterpretq_u32_f32(rhs.data)));
        return *this;
    }

    F128& operator^=(const F128& rhs) {
        data = vreinterpretq_f32_u32(veorq_u32(vreinterpretq_u32_f32(data), vreinterpretq_u32_f32(rhs.data)));
        return *this;
    }

    static constexpr std::size_t size() {
        return sizeof(decltype(data)) / sizeof(float);
    }

    static constexpr std::size_t alignment() {
        return alignof(decltype(data));
    }
};

    #ifdef TRIMD_ENABLE_NEON_FP16

TRIMD_TARGET_NEON_FP16 inline F128 F128::fromAlignedSource(const std::uint16_t* source) {
    return F128{vcvt_f32_f16(vld1_f16(reinterpret_cast<const float16_t*>(source)))};
}

TRIMD_TARGET_NEON_FP16 inline F128 F128::fromUnalignedSource(const std::uint16_t* source) {
    return F128{vcvt_f32_f16(vld1_f16(reinterpret_cast<const float16_t*>(source)))};
}

TRIMD_TARGET_NEON_FP16 inline F128 F128::loadSingleValue(const std::uint16_t* source) {
    float16_t value;
    std::memcpy(&value, source, sizeof(float16_t));
    return F128{vcvt_f32_f16(vset_lane_f16(value, vdup_n_f16(float16_t{}), 0))};
}

TRIMD_TARGET_NEON_FP16 inline void F128::alignedLoad(const std::uint16_t* source) {
    data = vcvt_f32_f16(vld1_f16(reinterpret_cast<const float16_t*>(source)));
}

TRIMD_TARGET_NEON_FP16 inline void F128::unalignedLoad(const std::uint16_t* source) {
    data = vcvt_f32_f16(vld1_f16(reinterpret_cast<const float16_t*>(source)));
}

TRIMD_TARGET_NEON_FP16 inline void F128::alignedStore(std::uint16_t* dest) const {
    vst1_f16(reinterpret_cast<float16_t*>(dest), vcvt_f16_f32(data));
}

TRIMD_TARGET_NEON_FP16 inline void F128::unalignedStore(std::uint16_t* dest) const {
    vst1_f16(reinterpret_cast<float16_t*>(dest), vcvt_f16_f32(data));
}

    #endif  // TRIMD_ENABLE_NEON_FP16

inline F128 operator==(const F128& lhs, const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vceqq_f32(lhs.data, rhs.data))};
}

inline F128 operator!=(const F128& lhs, const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vmvnq_u32(vceqq_f32(lhs.data, rhs.data)))};
}

inline F128 operator<(const F128& lhs, const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vcltq_f32(lhs.data, rhs.data))};
}

inline F128 operator<=(const F128& lhs, const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vcleq_f32(lhs.data, rhs.data))};
}

inline F128 operator>(const F128& lhs, const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vcgtq_f32(lhs.data, rhs.data))};
}

inline F128 operator>=(const F128& lhs, const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vcgeq_f32(lhs.data, rhs.data))};
}

inline F128 operator+(const F128& lhs, const F128& rhs) {
    return F128(lhs) += rhs;
}

inline F128 operator-(const F128& lhs, const F128& rhs) {
    return F128(lhs) -= rhs;
}

inline F128 operator*(const F128& lhs, const F128& rhs) {
    return F128(lhs) *= rhs;
}

inline F128 operator/(const F128& lhs, const F128& rhs) {
    return F128(lhs) /= rhs;
}

inline F128 operator&(const F128& lhs, const F128& rhs) {
    return F128(lhs) &= rhs;
}

inline F128 operator|(const F128& lhs, const F128& rhs) {
    return F128(lhs) |= rhs;
}

inline F128 operator^(const F128& lhs, const F128& rhs) {
    return F128(lhs) ^= rhs;
}

inline F128 operator~(const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vmvnq_u32(vreinterpretq_u32_f32(rhs.data)))};
}

inline void transpose(F128& row0, F128& row1, F128& row2, F128& row3) {
    // row01 = [row0.x, row1.x, row0.z, row1.z], [row0.y, row1.y, row0.w, row1.w]
    float32x4x2_t row01 = vtrnq_f32(row0.data, row1.data);
    // row23 = [row2.x, row3.x, row2.z, row3.z], [row2.y, row3.y, row2.w, row3.w]
    float32x4x2_t row23 = vtrnq_f32(row2.data, row3.data);

    // row0 = row0.x, row1.x, row2.x, row3.x
    row0 = F128{vcombine_f32(vget_low_f32(row01.val[0]), vget_low_f32(row23.val[0]))};
    // row1 = row0.y, row1.y, row2.y, row3.y
    row1 = F128{vcombine_f32(vget_low_f32(row01.val[1]), vget_low_f32(row23.val[1]))};
    // row2 = row0.z, row1.z, row2.z, row3.z
    row2 = F128{vcombine_f32(vget_high_f32(row01.val[0]), vget_high_f32(row23.val[0]))};
    // row3 = row0.w, row1.w, row2.w, row3.w
    row3 = F128{vcombine_f32(vget_high_f32(row01.val[1]), vget_high_f32(row23.val[1]))};
}

inline F128 abs(const F128& rhs) {
    return F128{vabsq_f32(rhs.data)};
}

inline F128 andnot(const F128& lhs, const F128& rhs) {
    return F128{vreinterpretq_f32_u32(vbicq_u32(vreinterpretq_u32_f32(rhs.data), vreinterpretq_u32_f32(lhs.data)))};
}

inline F128 rsqrt(const F128& rhs) {
    #ifndef TRIMD_ENABLE_FAST_INVERSE_SQRT
    const float32x4_t reciprocal0 = vrsqrteq_f32(rhs.data);
    const float32x4_t reciprocal1 = vmulq_f32(vrsqrtsq_f32(vmulq_f32(reciprocal0, reciprocal0), rhs.data), reciprocal0);
    return F128{reciprocal1};
    #else
    const uint32x4_t shifted = vshrq_n_u32(vreinterpretq_u32_f32(rhs.data), 1);
    const uint32x4_t subtracted = vsubq_u32(vdupq_n_u32(0x5f1ffff9), shifted);
    F128 result{vreinterpretq_f32_u32(subtracted)};
    result *= F128{0.703952253f} * (F128{2.38924456f} - rhs * result * result);
    return result;
    #endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
}

using F256 = fallback::T256<F128>;
using fallback::abs;
using fallback::andnot;
using fallback::rsqrt;
using fallback::transpose;

}  // namespace neon

}  // namespace trimd

#endif  // TRIMD_ENABLE_NEON
