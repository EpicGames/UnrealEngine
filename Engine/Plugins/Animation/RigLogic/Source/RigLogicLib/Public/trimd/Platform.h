// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(__arm__) || defined(__aarch64__) || defined(_M_ARM64) || defined(_M_ARM64EC)
    #define TRIMD_PLATFORM_ARM 1
#elif defined(i386) || defined(__i386) || defined(__i386__) || defined(__IA32__) || defined(_M_IX86) || defined(__X86__) ||      \
    defined(_X86_) || defined(__THW_INTEL__) || defined(__I86__) || defined(__amd64__) || defined(__amd64) ||                    \
    defined(__x86_64__) || defined(__x86_64) || defined(_M_X64) || defined(_M_AMD64)
    #define TRIMD_PLATFORM_X86 1
#endif

#if !defined(TRIMD_TARGET_F16C)
    #if defined(__clang__) || defined(__GNUC__)
        #define TRIMD_TARGET_F16C __attribute__((target("f16c")))
    #else
        #define TRIMD_TARGET_F16C
    #endif
#endif

#if !defined(TRIMD_TARGET_NEON_FP16)
    #if defined(__clang__) || defined(__GNUC__)
        #define TRIMD_TARGET_NEON_FP16 __attribute__((target("+fp16")))
    #else
        #define TRIMD_TARGET_NEON_FP16
    #endif
#endif

namespace trimd {

struct CPUFeatures {
    // ARM
    bool NEON;
    bool FP16;
    // X86
    bool SSE;
    bool SSE2;
    bool SSE3;
    bool SSSE3;
    bool SSE41;
    bool SSE42;
    bool AVX;
    bool F16C;
};

}  // namespace trimd

#ifdef TRIMD_ENABLE_RUNTIME_FEATURE_DETECTION

    #ifdef TRIMD_PLATFORM_X86
        #ifdef _MSC_VER
            #include <intrin.h>
        #endif  // _MSC_VER
    #endif      // TRIMD_PLATFORM_X86

    #ifdef TRIMD_PLATFORM_ARM
        #if defined(__APPLE__)
            #include <mach/machine.h>
            #include <sys/sysctl.h>
            #include <sys/types.h>
        #elif defined(WIN32)
            #include "trimd/PlatformWindows.h"
        #else
            #include <sys/auxv.h>
        #endif
    #endif  // TRIMD_PLATFORM_ARM

namespace trimd {

    #ifdef TRIMD_PLATFORM_X86
        #ifdef _MSC_VER
static void cpuidex(int info[4], int functionid, int subfunctionid) {
    __cpuidex(info, functionid, subfunctionid);
}
        #else
// Based on https://github.com/ispc/ispc/blob/main/builtins/dispatch.c
static void cpuidex(int info[4], int functionid, int subfunctionid) {
    __asm__ __volatile__("cpuid"
                         : "=a"(info[0]), "=b"(info[1]), "=c"(info[2]), "=d"(info[3])
                         : "0"(functionid), "2"(subfunctionid));
}
        #endif

inline CPUFeatures getCPUFeatures() {
    CPUFeatures flags = {};
    int info[4];
    cpuidex(info, 1, 0);
    flags.SSE = (info[3] & (1 << 25)) != 0;
    flags.SSE2 = (info[3] & (1 << 26)) != 0;
    flags.SSE3 = (info[2] & (1 << 0)) != 0;
    flags.SSSE3 = (info[2] & (1 << 9)) != 0;
    flags.SSE41 = (info[2] & (1 << 19)) != 0;
    flags.SSE42 = (info[2] & (1 << 20)) != 0;
    flags.AVX = (info[2] & (1 << 28)) != 0;
    flags.F16C = (info[2] & (1 << 29)) != 0;
    return flags;
}
    #endif  // TRIMD_PLATFORM_X86

    #ifdef TRIMD_PLATFORM_ARM
        #if defined(__APPLE__)
static bool getsysctlflag(const char* name) {
    unsigned int enabled = {};
    size_t enabledSize = sizeof(enabled);
    return sysctlbyname(name, &enabled, &enabledSize, nullptr, 0) == 0 && enabled;
}

inline CPUFeatures getCPUFeatures() {
    CPUFeatures flags = {};
    flags.NEON = getsysctlflag("hw.optional.AdvSIMD") || getsysctlflag("hw.optional.neon");
    flags.FP16 = (getsysctlflag("hw.optional.arm.FEAT_FP16") || getsysctlflag("hw.optional.neon_fp16")) &&
                 (getsysctlflag("hw.optional.AdvSIMD_HPFPCvt") || getsysctlflag("hw.optional.neon_hpfp"));
    return flags;
}
        #elif defined(WIN32)
            #ifndef PF_ARM_NEON_INSTRUCTIONS_AVAILABLE
                #define PF_ARM_NEON_INSTRUCTIONS_AVAILABLE 19
            #endif  // PF_ARM_NEON_INSTRUCTIONS_AVAILABLE

inline CPUFeatures getCPUFeatures() {
    CPUFeatures flags = {};
    flags.NEON = IsProcessorFeaturePresent(PF_ARM_NEON_INSTRUCTIONS_AVAILABLE) != 0;
    // Heuristic, guessing that FP16 must be available if the below conditions hold true
    flags.FP16 = (IsProcessorFeaturePresent(PF_FLOATING_POINT_EMULATED) == 0) &&
                 (IsProcessorFeaturePresent(PF_ARM_FMAC_INSTRUCTIONS_AVAILABLE) != 0);
    return flags;
}
        #else
            #ifdef __arm__
                // ARM-32bit
                #ifndef HWCAP_NEON
                    #define HWCAP_NEON (1 << 12)
                #endif  // HWCAP_NEON

                #ifndef HWCAP_FPHP
                    #define HWCAP_FPHP (1 << 22)
                #endif  // HWCAP_FPHP

                #ifndef HWCAP_ASIMDHP
                    #define HWCAP_ASIMDHP (1 << 23)
                #endif  // HWCAP_ASIMDHP
            #else
                // ARM-64bit
                #ifndef HWCAP_ASIMD
                    #define HWCAP_ASIMD (1 << 1)
                #endif  // HWCAP_ASIMD

                #ifndef HWCAP_FPHP
                    #define HWCAP_FPHP (1 << 9)
                #endif  // HWCAP_FPHP

                #ifndef HWCAP_ASIMDHP
                    #define HWCAP_ASIMDHP (1 << 10)
                #endif  // HWCAP_ASIMDHP
            #endif

inline CPUFeatures getCPUFeatures() {
    CPUFeatures flags = {};
    const unsigned long hwcaps = getauxval(AT_HWCAP);
            #ifdef __arm__
    // ARM-32bit
    flags.NEON = (hwcaps & HWCAP_NEON) != 0;
            #else
    // ARM-64bit
    flags.NEON = (hwcaps & HWCAP_ASIMD) != 0;
            #endif

    flags.FP16 = ((hwcaps & HWCAP_ASIMDHP) != 0) && ((hwcaps & HWCAP_FPHP) != 0);
    return flags;
}
        #endif  // __linux__
    #endif      // TRIMD_PLATFORM_ARM

}  // namespace trimd

#else

namespace trimd {

inline CPUFeatures getCPUFeatures() {
    return CPUFeatures{};
}

}  // namespace trimd

#endif  // TRIMD_ENABLE_RUNTIME_FEATURE_DETECTION
