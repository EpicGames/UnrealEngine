// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(TRIMD_ENABLE_AVX) || defined(TRIMD_ENABLE_SSE)
    #include <immintrin.h>

    #include <cstdint>
    #include <cstring>

    #if (!defined(__clang__) && defined(_MSC_VER) && _MSC_VER <= 1900) ||                                                        \
        (!defined(__clang__) && defined(__GNUC__) && __GNUC__ < 9)
        #define _mm_loadu_si64 _mm_loadl_epi64
    #endif

    #if (!defined(__clang__) && defined(_MSC_VER) && _MSC_VER <= 1900) || (defined(__clang__) && __clang_major__ < 8) ||         \
        (!defined(__clang__) && defined(__GNUC__) && __GNUC__ < 11)
inline __m128i _mm_loadu_si16(const void* source) {
    return _mm_insert_epi16(_mm_setzero_si128(), *reinterpret_cast<const std::int16_t*>(source), 0);
}
    #endif

    #if !defined(__clang__) && defined(_MSC_VER) && (_MSC_VER <= 1900) && defined(_WIN32) && !defined(_WIN64)
inline std::int64_t _mm_cvtsi128_si64(__m128i source) {
    const std::uint32_t low32 = static_cast<std::uint32_t>(_mm_cvtsi128_si32(source));
    const std::uint32_t high32 = static_cast<std::uint32_t>(_mm_cvtsi128_si32(_mm_srli_epi64(source, 32)));
    return static_cast<std::int64_t>((static_cast<std::uint64_t>(high32) << 32) | low32);
}
    #endif

    #if (!defined(__clang__) && defined(_MSC_VER) && _MSC_VER <= 1900) ||                                                        \
        (!defined(__clang__) && defined(__GNUC__) && __GNUC__ < 9) || (defined(__clang__) && __clang_major__ < 8)
inline void _mm_storeu_si64(void* dest, __m128i source) {
    const int64_t value = _mm_cvtsi128_si64(source);
    std::memcpy(dest, &value, sizeof(value));
}
    #endif

#endif  // defined(TRIMD_ENABLE_AVX) || defined(TRIMD_ENABLE_SSE)

#ifdef TRIMD_ENABLE_NEON
    #include <arm_neon.h>

    #if !defined(__clang__) && !defined(__GNUC__)

        #ifdef TRIMD_ENABLE_NEON_FP16
// MSVC's arm64_neon.h defines vector types (float16x4_t etc.) but omits the scalar float16_t.
// __n16 is the MSVC intrinsic type for a 16-bit NEON lane.
typedef __n16 float16_t;
// MSVC arm64_neon.h omits vdup_n_f16 and vset_lane_f16. float16x4_t and uint16x4_t both
// alias __n64 on MSVC, so u16 ops on a float16x4_t register are safe.
TRIMD_TARGET_NEON_FP16 inline float16x4_t vdup_n_f16(float16_t value) {
    return vdup_n_u16(value.n16_u16[0]);
}

            // vset_lane_f16 must be a macro since lane index must be a compile-time constant
            #define vset_lane_f16(value, vec, lane) vset_lane_u16((value).n16_u16[0], (vec), (lane))
        #endif  // TRIMD_ENABLE_NEON_FP16

    #endif  // !defined(__clang__) && !defined(__GNUC__)

namespace trimd {

namespace neon {

    #if defined(__clang__) || defined(__GNUC__)
template<typename T>
inline void prefetch_t0(const T* p) {
    __builtin_prefetch(p, 0, 3);
}
template<typename T>
inline void prefetch_t1(const T* p) {
    __builtin_prefetch(p, 0, 2);
}
template<typename T>
inline void prefetch_t2(const T* p) {
    __builtin_prefetch(p, 0, 1);
}
template<typename T>
inline void prefetch_nta(const T* p) {
    __builtin_prefetch(p, 0, 0);
}
    #else
template<typename T>
inline void prefetch_t0(const T* p) {
    __prefetch2(p, 0);
}  // PLDL1KEEP
template<typename T>
inline void prefetch_t1(const T* p) {
    __prefetch2(p, 2);
}  // PLDL2KEEP
template<typename T>
inline void prefetch_t2(const T* p) {
    __prefetch2(p, 4);
}  // PLDL3KEEP
template<typename T>
inline void prefetch_nta(const T* p) {
    __prefetch2(p, 1);
}  // PLDL1STRM
    #endif

}  // namespace neon

}  // namespace trimd

#endif  // TRIMD_ENABLE_NEON
