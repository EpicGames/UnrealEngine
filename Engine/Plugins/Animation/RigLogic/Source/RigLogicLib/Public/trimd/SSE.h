// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef TRIMD_ENABLE_SSE
    #include "trimd/Fallback.h"
    #include "trimd/Polyfill.h"

    #include <immintrin.h>

namespace trimd {

namespace sse {

struct F128 {
    using value_type = float;

    __m128 data;

    F128() :
        data{_mm_setzero_ps()} {
    }

    explicit F128(__m128 value) :
        data{value} {
    }

    explicit F128(float value) :
        F128{_mm_set1_ps(value)} {
    }

    F128(float v1, float v2, float v3, float v4) :
        data{_mm_set_ps(v4, v3, v2, v1)} {
    }

    static F128 fromAlignedSource(const float* source) {
        return F128{_mm_load_ps(source)};
    }

    static F128 fromUnalignedSource(const float* source) {
        return F128{_mm_loadu_ps(source)};
    }

    static F128 loadSingleValue(const float* source) {
        return F128{_mm_load_ss(source)};
    }

    #ifdef TRIMD_ENABLE_F16C
    static F128 fromAlignedSource(const std::uint16_t* source);
    static F128 fromUnalignedSource(const std::uint16_t* source);
    static F128 loadSingleValue(const std::uint16_t* source);
    #endif  // TRIMD_ENABLE_F16C

    template<typename T>
    static void prefetchT0(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_T0);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    template<typename T>
    static void prefetchT1(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_T1);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    template<typename T>
    static void prefetchT2(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_T2);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    template<typename T>
    static void prefetchNTA(const T* source) {
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wold-style-cast"
    #endif
        _mm_prefetch(reinterpret_cast<const char*>(source), _MM_HINT_NTA);
    #if defined(__clang__) || defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    }

    void alignedLoad(const float* source) {
        data = _mm_load_ps(source);
    }

    void unalignedLoad(const float* source) {
        data = _mm_loadu_ps(source);
    }

    void alignedStore(float* dest) const {
        _mm_store_ps(dest, data);
    }

    void unalignedStore(float* dest) const {
        _mm_storeu_ps(dest, data);
    }

    #ifdef TRIMD_ENABLE_F16C
    void alignedLoad(const std::uint16_t* source);
    void unalignedLoad(const std::uint16_t* source);
    void alignedStore(std::uint16_t* dest) const;
    void unalignedStore(std::uint16_t* dest) const;
    #endif  // TRIMD_ENABLE_F16C

    float sum() const {
        __m128 temp = _mm_movehdup_ps(data);
        __m128 result = _mm_add_ps(data, temp);
        temp = _mm_movehl_ps(temp, result);
        result = _mm_add_ss(result, temp);
        return _mm_cvtss_f32(result);
    }

    F128& operator+=(const F128& rhs) {
        data = _mm_add_ps(data, rhs.data);
        return *this;
    }

    F128& operator-=(const F128& rhs) {
        data = _mm_sub_ps(data, rhs.data);
        return *this;
    }

    F128& operator*=(const F128& rhs) {
        data = _mm_mul_ps(data, rhs.data);
        return *this;
    }

    F128& operator/=(const F128& rhs) {
        data = _mm_div_ps(data, rhs.data);
        return *this;
    }

    F128& operator&=(const F128& rhs) {
        data = _mm_and_ps(data, rhs.data);
        return *this;
    }

    F128& operator|=(const F128& rhs) {
        data = _mm_or_ps(data, rhs.data);
        return *this;
    }

    F128& operator^=(const F128& rhs) {
        data = _mm_xor_ps(data, rhs.data);
        return *this;
    }

    static constexpr std::size_t size() {
        return sizeof(decltype(data)) / sizeof(float);
    }

    static constexpr std::size_t alignment() {
        return alignof(decltype(data));
    }
};

    #ifdef TRIMD_ENABLE_F16C

TRIMD_TARGET_F16C inline F128 F128::fromAlignedSource(const std::uint16_t* source) {
    return F128{_mm_cvtph_ps(_mm_loadl_epi64(reinterpret_cast<const __m128i*>(source)))};
}

TRIMD_TARGET_F16C inline F128 F128::fromUnalignedSource(const std::uint16_t* source) {
    return F128{_mm_cvtph_ps(_mm_loadu_si64(reinterpret_cast<const __m128i*>(source)))};
}

TRIMD_TARGET_F16C inline F128 F128::loadSingleValue(const std::uint16_t* source) {
    return F128{_mm_cvtph_ps(_mm_loadu_si16(source))};
}

TRIMD_TARGET_F16C inline void F128::alignedLoad(const std::uint16_t* source) {
    data = _mm_cvtph_ps(_mm_loadl_epi64(reinterpret_cast<const __m128i*>(source)));
}

TRIMD_TARGET_F16C inline void F128::unalignedLoad(const std::uint16_t* source) {
    data = _mm_cvtph_ps(_mm_loadu_si64(reinterpret_cast<const __m128i*>(source)));
}

TRIMD_TARGET_F16C inline void F128::alignedStore(std::uint16_t* dest) const {
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic push
            #pragma GCC diagnostic ignored "-Wold-style-cast"
            #if !defined(__clang__)
                #pragma GCC diagnostic ignored "-Wuseless-cast"
            #endif
        #endif
    _mm_storel_epi64(reinterpret_cast<__m128i*>(dest), _mm_cvtps_ph(data, _MM_FROUND_CUR_DIRECTION));
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic pop
        #endif
}

TRIMD_TARGET_F16C inline void F128::unalignedStore(std::uint16_t* dest) const {
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic push
            #pragma GCC diagnostic ignored "-Wold-style-cast"
            #if !defined(__clang__)
                #pragma GCC diagnostic ignored "-Wuseless-cast"
            #endif
        #endif
    _mm_storeu_si64(reinterpret_cast<__m128i*>(dest), _mm_cvtps_ph(data, _MM_FROUND_CUR_DIRECTION));
        #if defined(__clang__) || defined(__GNUC__)
            #pragma GCC diagnostic pop
        #endif
}

    #endif  // TRIMD_ENABLE_F16C

inline F128 operator==(const F128& lhs, const F128& rhs) {
    return F128{_mm_cmpeq_ps(lhs.data, rhs.data)};
}

inline F128 operator!=(const F128& lhs, const F128& rhs) {
    return F128{_mm_cmpneq_ps(lhs.data, rhs.data)};
}

inline F128 operator<(const F128& lhs, const F128& rhs) {
    return F128{_mm_cmplt_ps(lhs.data, rhs.data)};
}

inline F128 operator<=(const F128& lhs, const F128& rhs) {
    return F128{_mm_cmple_ps(lhs.data, rhs.data)};
}

inline F128 operator>(const F128& lhs, const F128& rhs) {
    return F128{_mm_cmpgt_ps(lhs.data, rhs.data)};
}

inline F128 operator>=(const F128& lhs, const F128& rhs) {
    return F128{_mm_cmpge_ps(lhs.data, rhs.data)};
}

inline F128 operator+(const F128& lhs, const F128& rhs) {
    return F128(lhs) += rhs;
}

inline F128 operator-(const F128& lhs, const F128& rhs) {
    return F128(lhs) -= rhs;
}

inline F128 operator*(const F128& lhs, const F128& rhs) {
    return F128(lhs) *= rhs;
}

inline F128 operator/(const F128& lhs, const F128& rhs) {
    return F128(lhs) /= rhs;
}

inline F128 operator&(const F128& lhs, const F128& rhs) {
    return F128(lhs) &= rhs;
}

inline F128 operator|(const F128& lhs, const F128& rhs) {
    return F128(lhs) |= rhs;
}

inline F128 operator^(const F128& lhs, const F128& rhs) {
    return F128(lhs) ^= rhs;
}

inline F128 operator~(const F128& rhs) {
    static const auto ones = static_cast<int>(0xFFFFFFFF);
    F128 result{_mm_castsi128_ps(_mm_set1_epi32(ones))};  // Generate register with all 1s
    return (result ^= rhs);
}

inline void transpose(F128& row0, F128& row1, F128& row2, F128& row3) {
    _MM_TRANSPOSE4_PS(row0.data, row1.data, row2.data, row3.data);
}

inline F128 abs(const F128& rhs) {
    const __m128 ABSMASK = _mm_castsi128_ps(_mm_set1_epi32(~(1 << 31)));
    return F128{_mm_and_ps(ABSMASK, rhs.data)};
}

inline F128 andnot(const F128& lhs, const F128& rhs) {
    return F128{_mm_andnot_ps(lhs.data, rhs.data)};
}

inline F128 rsqrt(const F128& rhs) {
    #ifndef TRIMD_ENABLE_FAST_INVERSE_SQRT
    return F128{_mm_rsqrt_ps(rhs.data)};
    #else
    const __m128i shifted = _mm_srli_epi32(_mm_castps_si128(rhs.data), 1);
    const __m128i subtracted = _mm_sub_epi32(_mm_set1_epi32(0x5f1ffff9), shifted);
    F128 result{_mm_castsi128_ps(subtracted)};
    result *= F128{0.703952253f} * (F128{2.38924456f} - rhs * result * result);
    return result;
    #endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
}

using F256 = fallback::T256<F128>;
using fallback::abs;
using fallback::andnot;
using fallback::rsqrt;
using fallback::transpose;

}  // namespace sse

}  // namespace trimd

#endif  // TRIMD_ENABLE_SSE
