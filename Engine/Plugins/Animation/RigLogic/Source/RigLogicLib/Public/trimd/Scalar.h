// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trimd/Fallback.h"
#include "trimd/Utils.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cmath>
#include <cstddef>
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trimd {

namespace scalar {

template<typename T>
struct T128 {
    using value_type = typename std::remove_cv<T>::type;

    static_assert(sizeof(value_type) == 4, "Only 32-bit types are supported");

    std::array<value_type, 4> data;

    T128() :
        data{} {
    }

    T128(value_type v1, value_type v2, value_type v3, value_type v4) :
        data({v1, v2, v3, v4}) {
    }

    explicit T128(value_type value) :
        T128(value, value, value, value) {
    }

    static T128 fromAlignedSource(const value_type* source) {
        return T128{source[0], source[1], source[2], source[3]};
    }

    static T128 fromUnalignedSource(const value_type* source) {
        return T128::fromAlignedSource(source);
    }

    static T128 loadSingleValue(const value_type* source) {
        return T128{source[0], value_type{}, value_type{}, value_type{}};
    }

    template<typename U>
    static void prefetchT0(const U* /*unused*/) {
        // Intentionally noop
    }

    template<typename U>
    static void prefetchT1(const U* /*unused*/) {
        // Intentionally noop
    }

    template<typename U>
    static void prefetchT2(const U* /*unused*/) {
        // Intentionally noop
    }

    template<typename U>
    static void prefetchNTA(const U* /*unused*/) {
        // Intentionally noop
    }

    void alignedLoad(const value_type* source) {
        data[0] = source[0];
        data[1] = source[1];
        data[2] = source[2];
        data[3] = source[3];
    }

    void unalignedLoad(const value_type* source) {
        alignedLoad(source);
    }

    void alignedStore(value_type* dest) const {
        dest[0] = data[0];
        dest[1] = data[1];
        dest[2] = data[2];
        dest[3] = data[3];
    }

    void unalignedStore(value_type* dest) const {
        alignedStore(dest);
    }

    value_type sum() const {
        return data[0] + data[1] + data[2] + data[3];
    }

    T128& operator+=(const T128& rhs) {
        data[0] += rhs.data[0];
        data[1] += rhs.data[1];
        data[2] += rhs.data[2];
        data[3] += rhs.data[3];
        return *this;
    }

    T128& operator-=(const T128& rhs) {
        data[0] -= rhs.data[0];
        data[1] -= rhs.data[1];
        data[2] -= rhs.data[2];
        data[3] -= rhs.data[3];
        return *this;
    }

    T128& operator*=(const T128& rhs) {
        data[0] *= rhs.data[0];
        data[1] *= rhs.data[1];
        data[2] *= rhs.data[2];
        data[3] *= rhs.data[3];
        return *this;
    }

    T128& operator/=(const T128& rhs) {
        data[0] /= rhs.data[0];
        data[1] /= rhs.data[1];
        data[2] /= rhs.data[2];
        data[3] /= rhs.data[3];
        return *this;
    }

    T128& operator&=(const T128& rhs) {
        data[0] = bitcast<value_type>(bitcast<std::uint32_t>(data[0]) & bitcast<std::uint32_t>(rhs.data[0]));
        data[1] = bitcast<value_type>(bitcast<std::uint32_t>(data[1]) & bitcast<std::uint32_t>(rhs.data[1]));
        data[2] = bitcast<value_type>(bitcast<std::uint32_t>(data[2]) & bitcast<std::uint32_t>(rhs.data[2]));
        data[3] = bitcast<value_type>(bitcast<std::uint32_t>(data[3]) & bitcast<std::uint32_t>(rhs.data[3]));
        return *this;
    }

    T128& operator|=(const T128& rhs) {
        data[0] = bitcast<value_type>(bitcast<std::uint32_t>(data[0]) | bitcast<std::uint32_t>(rhs.data[0]));
        data[1] = bitcast<value_type>(bitcast<std::uint32_t>(data[1]) | bitcast<std::uint32_t>(rhs.data[1]));
        data[2] = bitcast<value_type>(bitcast<std::uint32_t>(data[2]) | bitcast<std::uint32_t>(rhs.data[2]));
        data[3] = bitcast<value_type>(bitcast<std::uint32_t>(data[3]) | bitcast<std::uint32_t>(rhs.data[3]));
        return *this;
    }

    T128& operator^=(const T128& rhs) {
        data[0] = bitcast<value_type>(bitcast<std::uint32_t>(data[0]) ^ bitcast<std::uint32_t>(rhs.data[0]));
        data[1] = bitcast<value_type>(bitcast<std::uint32_t>(data[1]) ^ bitcast<std::uint32_t>(rhs.data[1]));
        data[2] = bitcast<value_type>(bitcast<std::uint32_t>(data[2]) ^ bitcast<std::uint32_t>(rhs.data[2]));
        data[3] = bitcast<value_type>(bitcast<std::uint32_t>(data[3]) ^ bitcast<std::uint32_t>(rhs.data[3]));
        return *this;
    }

    static constexpr std::size_t size() {
        return sizeof(decltype(data)) / sizeof(value_type);
    }

    static constexpr std::size_t alignment() {
#if defined(__arm__) || defined(__aarch64__) || defined(_M_ARM) || defined(_M_ARM64)
        return std::alignment_of<std::max_align_t>::value;
#else
        return sizeof(decltype(data));
#endif
    }
};

template<typename T>
inline T128<T> operator==(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>{bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[0] == rhs.data[0]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[1] == rhs.data[1]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[2] == rhs.data[2]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[3] == rhs.data[3])))};
}

template<typename T>
inline T128<T> operator!=(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>{bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[0] != rhs.data[0]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[1] != rhs.data[1]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[2] != rhs.data[2]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[3] != rhs.data[3])))};
}

template<typename T>
inline T128<T> operator<(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>{bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[0] < rhs.data[0]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[1] < rhs.data[1]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[2] < rhs.data[2]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[3] < rhs.data[3])))};
}

template<typename T>
inline T128<T> operator<=(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>{bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[0] <= rhs.data[0]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[1] <= rhs.data[1]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[2] <= rhs.data[2]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[3] <= rhs.data[3])))};
}

template<typename T>
inline T128<T> operator>(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>{bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[0] > rhs.data[0]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[1] > rhs.data[1]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[2] > rhs.data[2]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[3] > rhs.data[3])))};
}

template<typename T>
inline T128<T> operator>=(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>{bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[0] >= rhs.data[0]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[1] >= rhs.data[1]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[2] >= rhs.data[2]))),
                   bitcast<T>(static_cast<std::uint32_t>(-(lhs.data[3] >= rhs.data[3])))};
}

template<typename T>
inline T128<T> operator+(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>(lhs) += rhs;
}

template<typename T>
inline T128<T> operator-(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>(lhs) -= rhs;
}

template<typename T>
inline T128<T> operator*(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>(lhs) *= rhs;
}

template<typename T>
inline T128<T> operator/(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>(lhs) /= rhs;
}

template<typename T>
inline T128<T> operator&(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>(lhs) &= rhs;
}

template<typename T>
inline T128<T> operator|(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>(lhs) |= rhs;
}

template<typename T>
inline T128<T> operator^(const T128<T>& lhs, const T128<T>& rhs) {
    return T128<T>(lhs) ^= rhs;
}

template<typename T>
inline T128<T> operator~(const T128<T>& rhs) {
    return T128<T>(bitcast<T>(~bitcast<std::uint32_t>(rhs.data[0])),
                   bitcast<T>(~bitcast<std::uint32_t>(rhs.data[1])),
                   bitcast<T>(~bitcast<std::uint32_t>(rhs.data[2])),
                   bitcast<T>(~bitcast<std::uint32_t>(rhs.data[3])));
}

template<typename T>
inline void transpose(T128<T>& row0, T128<T>& row1, T128<T>& row2, T128<T>& row3) {
    T128<T> transposed0{row0.data[0], row1.data[0], row2.data[0], row3.data[0]};
    T128<T> transposed1{row0.data[1], row1.data[1], row2.data[1], row3.data[1]};
    T128<T> transposed2{row0.data[2], row1.data[2], row2.data[2], row3.data[2]};
    T128<T> transposed3{row0.data[3], row1.data[3], row2.data[3], row3.data[3]};
    row0 = transposed0;
    row1 = transposed1;
    row2 = transposed2;
    row3 = transposed3;
}

template<typename T>
inline T128<T> abs(const T128<T>& rhs) {
    return {std::abs(rhs.data[0]), std::abs(rhs.data[1]), std::abs(rhs.data[2]), std::abs(rhs.data[3])};
}

template<typename T>
inline T128<T> andnot(const T128<T>& lhs, const T128<T>& rhs) {
    return ~lhs & rhs;
}

template<typename T>
inline T128<T> rsqrt(const T128<T>& rhs) {
#ifndef TRIMD_ENABLE_FAST_INVERSE_SQRT
    return T128<T>{1.0f / std::sqrt(rhs.data[0]),
                   1.0f / std::sqrt(rhs.data[1]),
                   1.0f / std::sqrt(rhs.data[2]),
                   1.0f / std::sqrt(rhs.data[3])};
#else
    std::uint32_t asInts[4];
    std::memcpy(asInts, rhs.data.data(), sizeof(asInts));
    asInts[0] = 0x5f1ffff9 - (asInts[0] >> 1);
    asInts[1] = 0x5f1ffff9 - (asInts[1] >> 1);
    asInts[2] = 0x5f1ffff9 - (asInts[2] >> 1);
    asInts[3] = 0x5f1ffff9 - (asInts[3] >> 1);
    T128<T> result;
    std::memcpy(result.data.data(), asInts, sizeof(asInts));
    result.data[0] *= 0.703952253f * (2.38924456f - rhs.data[0] * result.data[0] * result.data[0]);
    result.data[1] *= 0.703952253f * (2.38924456f - rhs.data[1] * result.data[1] * result.data[1]);
    result.data[2] *= 0.703952253f * (2.38924456f - rhs.data[2] * result.data[2] * result.data[2]);
    result.data[3] *= 0.703952253f * (2.38924456f - rhs.data[3] * result.data[3] * result.data[3]);
    return result;
#endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
}

using F128 = T128<float>;
using F256 = fallback::T256<F128>;
using fallback::abs;
using fallback::andnot;
using fallback::rsqrt;
using fallback::transpose;

}  // namespace scalar

}  // namespace trimd
