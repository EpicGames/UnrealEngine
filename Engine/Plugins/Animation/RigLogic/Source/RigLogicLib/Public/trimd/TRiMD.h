// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trimd/Platform.h"
// Includes that go after platform detection macros
#include "trimd/AVX.h"
#include "trimd/NEON.h"
#include "trimd/SSE.h"
#include "trimd/Scalar.h"

namespace trimd {

#if defined(TRIMD_ENABLE_AVX)
using F256 = avx::F256;
using avx::abs;
using avx::andnot;
using avx::rsqrt;
using avx::transpose;
#elif defined(TRIMD_ENABLE_SSE)
using F256 = sse::F256;
#elif defined(TRIMD_ENABLE_NEON)
using F256 = neon::F256;
#else
using F256 = scalar::F256;
#endif  // TRIMD_ENABLE_AVX

#if defined(TRIMD_ENABLE_SSE)
using F128 = sse::F128;
using sse::abs;
using sse::andnot;
using sse::rsqrt;
using sse::transpose;
#elif defined(TRIMD_ENABLE_NEON)
using F128 = neon::F128;
using neon::abs;
using neon::andnot;
using neon::rsqrt;
using neon::transpose;
#else
using F128 = scalar::F128;
#endif  // TRIMD_ENABLE_SSE

using scalar::abs;
using scalar::andnot;
using scalar::rsqrt;
using scalar::transpose;

}  // namespace trimd
