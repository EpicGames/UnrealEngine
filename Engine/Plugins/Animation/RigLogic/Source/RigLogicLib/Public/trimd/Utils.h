// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trimd/Macros.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstring>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace trimd {

template<typename TTarget, typename TSource>
FORCE_INLINE TTarget bitcast(TSource source) {
    static_assert(sizeof(TTarget) == sizeof(TSource), "Target and source must be of equal size.");
    TTarget target;
    std::memcpy(&target, &source, sizeof(TSource));
    return target;
}

}  // namespace trimd
