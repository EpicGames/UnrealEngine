// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define TRIMD_MAJOR_VERSION 2
#define TRIMD_MINOR_VERSION 4
#define TRIMD_PATCH_VERSION 8
#define TRIMD_VERSION_STRING "2.4.8"
