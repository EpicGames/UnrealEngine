// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/Defs.h"

#include <cstddef>
#include <cstdint>

namespace trio {

class Writable;

class TRIOAPI Readable {
public:
    /**
        @brief Read bytes from stream into the given buffer.
        @param destination
            Destination buffer into which the data is going to be read from the stream.
        @param size
            Number of bytes to read from stream.
        @return
            Number of bytes read.
    */
    virtual std::size_t read(char* destination, std::size_t size) = 0;
    /**
        @brief Read bytes from this stream into the given stream.
        @param destination
            Destination stream into which the data is going to be read from this stream.
        @param size
            Number of bytes to read from stream.
        @return
            Number of bytes read.
    */
    virtual std::size_t read(Writable* destination, std::size_t size) = 0;

protected:
    virtual ~Readable();
};

class TRIOAPI Writable {
public:
    /**
        @brief Writes bytes from the given buffer to the stream.
        @param source
            Source buffer from which the data is going to be written to the stream.
        @param size
            Number of bytes to write to the stream.
        @return
            Number of bytes written.
    */
    virtual std::size_t write(const char* source, std::size_t size) = 0;
    /**
        @brief Writes bytes from the given stream to this stream.
        @param source
            Source stream from which the data is going to be written into this stream.
        @param size
            Number of bytes to write to the stream.
        @return
            Number of bytes written.
    */
    virtual std::size_t write(Readable* source, std::size_t size) = 0;

protected:
    virtual ~Writable();
};

class TRIOAPI Seekable {
public:
    /**
        @brief Get the current position in the stream.
        @return
            Position in the stream relative to it's start, with 0 denoting the start position.
    */
    virtual std::uint64_t tell() = 0;
    /**
        @brief Set the current position in the stream.
        @param position
            Position in the stream relative to it's start, with 0 denoting the start position.
    */
    virtual void seek(std::uint64_t position) = 0;

protected:
    virtual ~Seekable();
};

class TRIOAPI Openable {
public:
    /**
        @brief Open access to the stream.
    */
    virtual void open() = 0;

protected:
    virtual ~Openable();
};

class TRIOAPI Closeable {
public:
    /**
        @brief Close access to the stream.
    */
    virtual void close() = 0;

protected:
    virtual ~Closeable();
};

class TRIOAPI Controllable : public Openable, public Closeable {
protected:
    virtual ~Controllable();
};

class TRIOAPI Bounded {
public:
    /**
        @brief Obtain size of stream in bytes.
        @return
            Size in bytes.
    */
    virtual std::uint64_t size() = 0;

protected:
    virtual ~Bounded();
};

class TRIOAPI Buffered {
public:
    /**
        @brief Flush the changes to filesystem.
    */
    virtual void flush() = 0;

protected:
    virtual ~Buffered();
};

class TRIOAPI Resizable {
public:
    /**
        @brief Resize file to the requested size.
    */
    virtual void resize(std::uint64_t size) = 0;

protected:
    virtual ~Resizable();
};

}  // namespace trio
