// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/Concepts.h"
#include "trio/Defs.h"
#include "trio/types/Aliases.h"
#include "trio/types/Parameters.h"

#include <cstdint>

namespace trio {

class TRIOAPI BoundedIOStream : public Controllable, public Readable, public Writable, public Seekable, public Bounded {
public:
    using AccessMode = trio::AccessMode;
    using OpenMode = trio::OpenMode;

    static const sc::StatusCode OpenError;
    static const sc::StatusCode ReadError;
    static const sc::StatusCode WriteError;
    static const sc::StatusCode AlreadyOpenError;
    static const sc::StatusCode SeekError;

public:
    virtual ~BoundedIOStream();
};

}  // namespace trio
