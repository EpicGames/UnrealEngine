// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/Defs.h"
#include "trio/Stream.h"

namespace trio {

/**
    @brief Standard file stream.
*/
class TRIOAPI FileStream : public BoundedIOStream {
public:
    /**
        @brief Factory method for creation of a FileStream instance.
        @param path
            UTF-8 encoded path to file to be opened.
        @param accessMode
            Control whether the file is opened for reading or writing.
        @param openMode
            Control whether the file is opened in binary or textual mode.
        @param memRes
            The memory resource to be used for the allocation of the FileStream instance.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static FileStream* create(const char* path, AccessMode accessMode, OpenMode openMode, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a FileStream instance.
        @param instance
            Instance of FileStream to be freed.
        @see create
    */
    static void destroy(FileStream* instance);

    FileStream() = default;
    ~FileStream() override;

    FileStream(const FileStream&) = delete;
    FileStream& operator=(const FileStream&) = delete;

    FileStream(FileStream&&) = default;
    FileStream& operator=(FileStream&&) = default;
};

}  // namespace trio

namespace pma {

template<>
struct DefaultInstanceCreator<trio::FileStream> {
    using type = FactoryCreate<trio::FileStream>;
};

template<>
struct DefaultInstanceDestroyer<trio::FileStream> {
    using type = FactoryDestroy<trio::FileStream>;
};

}  // namespace pma
