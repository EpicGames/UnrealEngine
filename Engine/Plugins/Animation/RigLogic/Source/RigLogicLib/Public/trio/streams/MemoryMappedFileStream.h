// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/Defs.h"
#include "trio/Stream.h"

#include <cstdint>

namespace trio {

/**
    @brief Memory mapped file stream.
*/
class TRIOAPI MemoryMappedFileStream : public BoundedIOStream, public Buffered, public Resizable {
public:
    /**
        @brief Factory method for creation of a MemoryMappedFileStream instance.
        @param path
            UTF-8 encoded path to file to be opened.
        @param accessMode
            Control whether the file is opened for reading or writing.
        @param memRes
            The memory resource to be used for the allocation of the MemoryMappedFileStream instance.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static MemoryMappedFileStream* create(const char* path, AccessMode accessMode, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a MemoryMappedFileStream instance.
        @param instance
            Instance of MemoryMappedFileStream to be freed.
        @see create
    */
    static void destroy(MemoryMappedFileStream* instance);

    MemoryMappedFileStream() = default;
    ~MemoryMappedFileStream() override;

    MemoryMappedFileStream(const MemoryMappedFileStream&) = delete;
    MemoryMappedFileStream& operator=(const MemoryMappedFileStream&) = delete;

    MemoryMappedFileStream(MemoryMappedFileStream&&) = default;
    MemoryMappedFileStream& operator=(MemoryMappedFileStream&&) = default;
};

}  // namespace trio

namespace pma {

template<>
struct DefaultInstanceCreator<trio::MemoryMappedFileStream> {
    using type = FactoryCreate<trio::MemoryMappedFileStream>;
};

template<>
struct DefaultInstanceDestroyer<trio::MemoryMappedFileStream> {
    using type = FactoryDestroy<trio::MemoryMappedFileStream>;
};

}  // namespace pma
