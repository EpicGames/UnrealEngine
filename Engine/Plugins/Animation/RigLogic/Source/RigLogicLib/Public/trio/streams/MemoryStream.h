// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/Defs.h"
#include "trio/Stream.h"

#include <cstdint>

namespace trio {

/**
    @brief In-memory stream.
*/
class TRIOAPI MemoryStream : public BoundedIOStream {
public:
    /**
        @brief Factory method for creation of a MemoryStream instance.
        @param memRes
            The memory resource to be used for the allocation of the MemoryStream instance.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static MemoryStream* create(MemoryResource* memRes = nullptr);
    /**
        @brief Factory method for creation of a MemoryStream instance.
        @param initialSize
            Initial size of the memory stream.
        @param memRes
            The memory resource to be used for the allocation of the MemoryStream instance.
        @note
            If a custom memory resource is not given, a default allocation mechanism will be used.
        @warning
            User is responsible for releasing the returned pointer by calling destroy.
        @see destroy
    */
    static MemoryStream* create(std::size_t initialSize, MemoryResource* memRes = nullptr);
    /**
        @brief Method for freeing a MemoryStream instance.
        @param instance
            Instance of MemoryStream to be freed.
        @see create
    */
    static void destroy(MemoryStream* instance);

    MemoryStream() = default;
    ~MemoryStream() override;

    MemoryStream(const MemoryStream&) = delete;
    MemoryStream& operator=(const MemoryStream&) = delete;

    MemoryStream(MemoryStream&&) = default;
    MemoryStream& operator=(MemoryStream&&) = default;
};

}  // namespace trio

namespace pma {

template<>
struct DefaultInstanceCreator<trio::MemoryStream> {
    using type = FactoryCreate<trio::MemoryStream>;
};

template<>
struct DefaultInstanceDestroyer<trio::MemoryStream> {
    using type = FactoryDestroy<trio::MemoryStream>;
};

}  // namespace pma
