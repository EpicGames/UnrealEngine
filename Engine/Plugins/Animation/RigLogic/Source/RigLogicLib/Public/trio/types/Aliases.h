// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <pma/MemoryResource.h>
#include <pma/ScopedPtr.h>
#include <status/Status.h>
#include <status/StatusCode.h>

namespace trio {

using sc::Status;

using namespace pma;

}  // namespace trio
