// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace trio {

enum class AccessMode {
    Read = 1,
    Write = 2,
    ReadWrite = 3
};

enum class OpenMode {
    Binary = 4,
    Text = 8
};

}  // namespace trio
