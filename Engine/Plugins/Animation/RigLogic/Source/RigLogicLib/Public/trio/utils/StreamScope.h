// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "trio/Stream.h"

#include <utility>

namespace trio {

class StreamScope {
public:
    explicit StreamScope(Controllable* stream_) :
        stream{stream_} {
        stream->open();
    }

    ~StreamScope() {
        if (stream != nullptr) {
            stream->close();
        }
    }

    StreamScope(const StreamScope&) = delete;
    StreamScope& operator=(const StreamScope&) = delete;

    StreamScope(StreamScope&& rhs) noexcept :
        stream{nullptr} {
        std::swap(stream, rhs.stream);
    }

    StreamScope& operator=(StreamScope&& rhs) noexcept {
        std::swap(stream, rhs.stream);
        return *this;
    }

private:
    Controllable* stream;
};

}  // namespace trio
