// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define TRIO_MAJOR_VERSION 4
#define TRIO_MINOR_VERSION 0
#define TRIO_PATCH_VERSION 21
#define TRIO_VERSION_STRING "4.0.21"
