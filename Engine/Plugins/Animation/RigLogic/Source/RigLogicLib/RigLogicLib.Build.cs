// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RigLogicLib : ModuleRules
	{
		public RigLogicLib(ReadOnlyTargetRules Target) : base(Target)
		{
			IWYUSupport = IWYUSupport.None;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
						"Core",
						"CoreUObject",
						"Engine",
						"ControlRig"
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PublicDependencyModuleNames.Add("UnrealEd");
				PublicDependencyModuleNames.Add("EditorFramework");
			}

			Type = ModuleType.CPlusPlus;

			PublicDefinitions.Add("RL_BUILD_WITH_XYZ_ROTATION_ORDER");

			if (Target.LinkType != TargetLinkType.Monolithic)
			{
				PrivateDefinitions.Add("RL_BUILD_SHARED");
				PublicDefinitions.Add("RL_SHARED");
			}

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateDefinitions.Add("RL_BUILD_WITH_HALF_FLOATS");
				PrivateDefinitions.Add("TRIMD_CUSTOM_WINDOWS_H=\"WindowsPlatformUE.h\"");
				PrivateDefinitions.Add("TRIO_CUSTOM_WINDOWS_H=\"WindowsPlatformUE.h\"");
				PrivateDefinitions.Add("TRIO_WINDOWS_FILE_MAPPING_AVAILABLE");
				if (Target.Architectures.Contains(UnrealArch.Arm64) || Target.Architectures.Contains(UnrealArch.Arm64ec))
				{
					PrivateDefinitions.Add("RL_BUILD_WITH_NEON");
				}
			}

			if (Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.LinuxArm64)
			{
				PrivateDefinitions.Add("TRIO_MREMAP_AVAILABLE");
			}

			if (Target.Platform == UnrealTargetPlatform.Linux ||
					Target.Platform == UnrealTargetPlatform.LinuxArm64 ||
					Target.Platform == UnrealTargetPlatform.Mac)
			{
				PrivateDefinitions.Add("RL_BUILD_WITH_HALF_FLOATS");
				PrivateDefinitions.Add("TRIO_MMAP_AVAILABLE");
			}

			PrivateDefinitions.Add("RL_AUTODETECT_SSE");
			PrivateDefinitions.Add("RL_AUTODETECT_HALF_FLOATS");
		}
	}
}
