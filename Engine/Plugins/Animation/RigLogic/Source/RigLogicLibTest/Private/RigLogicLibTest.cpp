// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicLibTest.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FRigLogicLibTest"

DEFINE_LOG_CATEGORY(LogRigLogicLibTest);

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRigLogicLibTest, RigLogicLibTest)
