// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "RigLogicLibTest.h"
#include "gtest/gtest.h"
#include "Misc/AutomationTest.h"

class FRigLogicLibTestPrinter
: public ::testing::EmptyTestEventListener
{
	virtual void OnTestStart(const ::testing::TestInfo& InTestInfo) override
	{
		UE_LOGF(LogRigLogicLibTest, Verbose, "Test %ls.%ls Starting", *FString(InTestInfo.test_suite_name()), *FString(InTestInfo.name()));
	}

	virtual void OnTestPartResult(const ::testing::TestPartResult& InTestPartResult) override
	{
		if (InTestPartResult.failed())
		{
			UE_LOGF(LogRigLogicLibTest, Error, "FAILED in %ls:%d\n%ls", *FString(InTestPartResult.file_name()), InTestPartResult.line_number(), *FString(InTestPartResult.summary()));
		}
		else
		{
			UE_LOGF(LogRigLogicLibTest, Verbose, "Succeeded in %ls:%d\n%ls", *FString(InTestPartResult.file_name()), InTestPartResult.line_number(), *FString(InTestPartResult.summary()));
		}
	}

	virtual void OnTestEnd(const ::testing::TestInfo& InTestInfo) override
	{
		UE_LOGF(LogRigLogicLibTest, Verbose, "Test %ls.%ls Ending", *FString(InTestInfo.test_suite_name()), *FString(InTestInfo.name()));
	}
};

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FRigLogicLibTestSuite, "RigLogic.RigLogicLib", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

void FRigLogicLibTestSuite::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	::testing::InitGoogleTest();

	const ::testing::UnitTest* Instance = ::testing::UnitTest::GetInstance();
	for (int32 TestSuiteIndex = 0; TestSuiteIndex < Instance->total_test_suite_count(); ++TestSuiteIndex)
	{
		const ::testing::TestSuite* TestSuite = Instance->GetTestSuite(TestSuiteIndex);
		const FString TestCaseName = FString::Format(TEXT("{0}"), { TestSuite->name() });
		OutBeautifiedNames.Add(TestCaseName);
	}

	OutTestCommands = OutBeautifiedNames;	
}

bool FRigLogicLibTestSuite::RunTest(const FString& Parameters)
{
	::testing::TestEventListeners& Listeners = ::testing::UnitTest::GetInstance()->listeners();

	FRigLogicLibTestPrinter TestPrinter;
	Listeners.Append(&TestPrinter);

	const FString Filter = FString::Format(TEXT("{0}*"), { Parameters });
	const TArray<ANSICHAR> TestFilter{ TCHAR_TO_ANSI(*Filter), Filter.Len() + 1 };
	::testing::GTEST_FLAG(filter) = TestFilter.GetData();

	TestTrue("RigLogicLib Tests", RUN_ALL_TESTS() == 0);

	Listeners.Release(&TestPrinter);

	return true;
}

#endif  // WITH_DEV_AUTOMATION_TESTS