// Copyright Epic Games, Inc. All Rights Reserved.

#include "arrayviewtests/TestArrayView.h"

#include "arrayviewtests/Utils.h"

namespace av {

TYPED_TEST_SUITE(TestArrayView, ValueTypes, );

TYPED_TEST(TestArrayView, DefaultCtor) {
    ArrayView<TypeParam> arr;
    EXPECT_EQ(arr.data(), nullptr);
    EXPECT_EQ(arr.size(), 0ul);
}

TYPED_TEST(TestArrayView, NullCtor) {
    ArrayView<TypeParam> arr{nullptr, 0ul};
    EXPECT_EQ(arr.data(), nullptr);
    EXPECT_EQ(arr.size(), 0ul);
}

TYPED_TEST(TestArrayView, CopyCtor) {
    auto src1 = this->template initArray<length>();
    auto src2 = this->template initArray<length>();

    ArrayView<TypeParam> arr1(src1);
    ArrayView<TypeParam> arr2(src2);

    ArrayView<TypeParam> arr3(arr1);

    EXPECT_EQ(arr1, arr3);
    arr2[0] = arr2[length - 1];
    EXPECT_NE(arr1, arr2);
}

TYPED_TEST(TestArrayView, CopyAssignment) {
    auto src1 = this->template initArray<length>();
    auto src2 = this->template initArray<length>();

    ArrayView<TypeParam> arr1(src1);
    ArrayView<TypeParam> arr2(src2);

    ArrayView<TypeParam> arr3 = arr1;

    EXPECT_EQ(arr1, arr3);
    arr2[0] = arr2[length - 1];
    EXPECT_NE(arr1, arr2);
}

TYPED_TEST(TestArrayView, CreateFromArray) {
    auto src = this->template initArray<length>();

    ArrayView<TypeParam> arr(src);

    EXPECT_EQ(arr, src);
}

TYPED_TEST(TestArrayView, CreateFromVector) {
    auto src = this->template initVector<length>();

    ArrayView<TypeParam> arr(src);

    EXPECT_EQ(arr, src);
}

TYPED_TEST(TestArrayView, CreateConstFromArray) {
    auto src = this->template initArray<length>();

    ArrayView<const TypeParam> arr(src);

    EXPECT_EQ(arr, src);
}

TYPED_TEST(TestArrayView, CreateConstFromVector) {
    auto src = this->template initVector<length>();

    ArrayView<const TypeParam> arr(src);

    EXPECT_EQ(arr, src);
}

TYPED_TEST(TestArrayView, CreateFromVectorDifferent) {
    auto src1 = this->template initVector<length + 1>();
    auto src2 = this->template initVector<length>();
    ArrayView<TypeParam> arr(src1);
    EXPECT_EQ(arr, src1);
    src1.push_back(src1[0]);
    EXPECT_NE(arr, src1);
    EXPECT_NE(arr, src2);
}

TYPED_TEST(TestArrayView, CreateFromPtr) {
    TypeParam src[length];
    this->template initArray<length>(src);

    ArrayView<TypeParam> arr(src, length);

    EXPECT_TRUE(this->equal(arr, src, length));
}

TYPED_TEST(TestArrayView, CreateFromPtrDifferent) {
    TypeParam src1[length];
    TypeParam src2[length];
    this->template initArray<length>(src1);
    this->template initArray<length>(src2);

    ArrayView<TypeParam> arr(src1, length);

    EXPECT_FALSE(this->equal(arr, src1, length - 1));
    src2[0] = src2[length - 1];
    EXPECT_FALSE(this->equal(arr, src2, length));
}

TYPED_TEST(TestArrayView, CreateConstFromPtr) {
    TypeParam src[length];
    this->template initArray<length>(src);

    ArrayView<const TypeParam> arr(src, length);

    EXPECT_TRUE(this->equal(arr, src, length));
}

TYPED_TEST(TestArrayView, CreateFromLValue) {
    auto src = this->template initVector<length>();
    ArrayView<TypeParam> arr{src};
    ArrayView<const TypeParam> arrImmutable{arr};
    EXPECT_EQ(arr, arrImmutable);
    arrImmutable = arr;
    EXPECT_EQ(arr, arrImmutable);
}

TYPED_TEST(TestArrayView, CreateConstFromLValueConst) {
    auto src = this->template initVector<length>();
    const ArrayView<TypeParam> arr{src};
    ArrayView<const TypeParam> arrImmutable{arr};
    EXPECT_EQ(arr, arrImmutable);
    arrImmutable = arr;
    EXPECT_EQ(arr, arrImmutable);
}

TYPED_TEST(TestArrayView, CreateFromRValue) {
    std::vector<TypeParam> src1 = this->template initVector<length>();
    ArrayView<TypeParam> v1(ArrayView<TypeParam>{src1.data(), src1.size()});
    EXPECT_EQ(v1, src1);
    v1 = ArrayView<TypeParam>{src1.data(), src1.size()};
    EXPECT_EQ(v1, src1);
}

TYPED_TEST(TestArrayView, CreateConstFromLValue) {
    std::vector<TypeParam> src1 = this->template initVector<length>();
    ArrayView<const TypeParam> v1(ArrayView<TypeParam>{src1.data(), src1.size()});
    EXPECT_EQ(v1, src1);
    v1 = ArrayView<TypeParam>{src1.data(), src1.size()};
    EXPECT_EQ(v1, src1);
}

TYPED_TEST(TestArrayView, RValueTContainerAssignment) {
    EXPECT_FALSE(can_instantiate<ArrayView<TypeParam>>(std::vector<TypeParam>{}));
}

TYPED_TEST(TestArrayView, MixDifferentTypesTContainerAssignment) {
    std::vector<void*> container;
    EXPECT_FALSE(can_instantiate<ConstArrayView<TypeParam>>(container));
}

TYPED_TEST(TestArrayView, NonConstFromConstAssignment) {
    EXPECT_FALSE(can_instantiate<ArrayView<TypeParam>>(ConstArrayView<TypeParam>()));
}

TYPED_TEST(TestArrayView, ConstFromNonConstAssignment) {
    EXPECT_TRUE(can_instantiate<ConstArrayView<TypeParam>>(ArrayView<TypeParam>()));
}

TYPED_TEST(TestArrayView, ConstFromConstDifferentTypeAssignment) {
    EXPECT_FALSE(can_instantiate<ConstArrayView<TypeParam>>(ConstArrayView<void*>()));
}

TYPED_TEST(TestArrayView, ConstFromNonConstDifferentTypeAssignment) {
    EXPECT_FALSE(can_instantiate<ConstArrayView<TypeParam>>(ArrayView<void*>()));
}

TYPED_TEST(TestArrayView, NonConstFromConstDifferentTypeAssignment) {
    EXPECT_FALSE(can_instantiate<ArrayView<TypeParam>>(ConstArrayView<void*>()));
}

TYPED_TEST(TestArrayView, NonConstFromNonConstDifferentTypeAssignment) {
    EXPECT_FALSE(can_instantiate<ArrayView<TypeParam>>(ArrayView<void*>()));
}

template<typename T>
struct ExpectsArrayView {
    ExpectsArrayView(ArrayView<T> /*unused*/) {
    }
};

TYPED_TEST(TestArrayView, ImplicitCastToVector) {
    std::vector<TypeParam> vec(length);
    EXPECT_TRUE(can_instantiate<ExpectsArrayView<TypeParam>>(vec));
}

TYPED_TEST(TestArrayView, ChangeElement) {
    auto src = this->template initArray<length>();
    ArrayView<TypeParam> arr(src);

    arr[0] = arr[length - 1];
    arr.at(1) = arr.at(length - 2);
    EXPECT_EQ(arr[0], arr[length - 1]);
    EXPECT_EQ(arr[1], arr[length - 2]);
}

TYPED_TEST(TestArrayView, DifferentSize) {
    auto src1 = this->template initArray<length>();
    auto src2 = this->template initArray<length + 1>();

    ArrayView<TypeParam> arr1(src1);
    ArrayView<TypeParam> arr2(src2);

    EXPECT_NE(arr1, arr2);
}

TYPED_TEST(TestArrayView, ConstAt) {
    auto src = this->template initArray<length>();
    const ArrayView<TypeParam> arr(src);
    TypeParam x = arr.at(length - 1);
    EXPECT_EQ(x, src[length - 1]);
}

TYPED_TEST(TestArrayView, ConstIndexOperator) {
    auto src = this->template initArray<length>();
    const ArrayView<TypeParam> arr(src);
    TypeParam x = arr[length - 1];
    EXPECT_EQ(x, src[length - 1]);
}

TYPED_TEST(TestArrayView, NonConstBeginEnd) {
    auto src = this->template initArray<length>();
    auto src2 = src;
    ArrayView<TypeParam> arr(src);
    for (auto& el : arr) {
        el = arr[0];
    }
    EXPECT_NE(src2, arr);
}

TYPED_TEST(TestArrayView, Subview) {
    auto src = this->template initArray<length>();
    ArrayView<TypeParam> arr(src);
    auto sub1 = arr.subview(0, arr.size());
    EXPECT_EQ(sub1, arr);
    auto sub2 = arr.subview(0, length);
    EXPECT_EQ(sub2, sub1);
    auto sub3 = arr.subview(1, length / 2);
    for (std::size_t i = 0; i < sub3.size(); ++i) {
        EXPECT_EQ(sub3[i], src[i + 1]);
    }
}

TYPED_TEST(TestArrayView, SubviewSize) {
    auto src = this->template initArray<length>();
    ArrayView<TypeParam> arr(src);

    EXPECT_EQ((arr.subview(0, 0).size()), 0ul);
    EXPECT_EQ((arr.subview(4, 1).size()), 1ul);
    EXPECT_EQ((arr.subview(4, 2).size()), 2ul);
    EXPECT_EQ((arr.subview(4, 3).size()), 3ul);

    EXPECT_EQ((arr.subview(length, 0).size()), 0ul);
    EXPECT_EQ((arr.subview(0, length).size()), length);

    EXPECT_EQ((arr.first(0).size()), 0ul);
    EXPECT_EQ((arr.first(3).size()), 3ul);
    EXPECT_EQ((arr.first(length).size()), length);

    EXPECT_EQ((arr.last(0).size()), 0ul);
    EXPECT_EQ((arr.last(3).size()), 3ul);
    EXPECT_EQ((arr.last(length).size()), length);
}

TYPED_TEST(TestArrayView, SubviewFirst) {
    auto src = this->template initArray<length>();
    ArrayView<TypeParam> arr(src);
    auto sub1 = arr.first(arr.size());
    EXPECT_EQ(sub1, arr);
    auto sub2 = arr.first(length);
    EXPECT_EQ(sub2, sub1);
    auto sub3 = arr.first(length / 2);
    for (std::size_t i = 0; i < sub3.size(); ++i) {
        EXPECT_EQ(sub3[i], src[i]);
    }
}

TYPED_TEST(TestArrayView, SubviewLast) {
    auto src = this->template initArray<length>();
    ArrayView<TypeParam> arr(src);
    auto sub1 = arr.last(arr.size());
    EXPECT_EQ(sub1, arr);
    auto sub2 = arr.last(length);
    EXPECT_EQ(sub2, sub1);
    auto sub3 = arr.last(length / 2);
    for (std::size_t i = 0; i < sub3.size(); ++i) {
        EXPECT_EQ(sub3[i], src[i + length / 2]);
    }
}

TYPED_TEST(TestArrayView, CompareArrayViews) {
    auto src1 = this->template initVector<length>();
    ArrayView<TypeParam> arrView1{src1};
    EXPECT_EQ(arrView1, src1);

    ArrayView<TypeParam> arrView2{src1};
    EXPECT_EQ(arrView1, arrView2);

    auto src2 = std::vector<TypeParam>{};
    arrView2 = ArrayView<TypeParam>{src2};
    EXPECT_NE(arrView1, arrView2);
    EXPECT_NE(arrView2, src1);
}

TYPED_TEST(TestArrayView, CompareWithVector) {
    using val_type = TypeParam;
    std::vector<val_type> src = this->template initVector<length>();
    std::vector<val_type> vec = src;
    ArrayView<const val_type> view(src);
    EXPECT_EQ(view, vec);
    vec[0] = vec[vec.size() - 1ul];
    EXPECT_NE(vec, view);
    vec[0] = view[0];
    EXPECT_EQ(view, vec);
    vec.push_back({});
    EXPECT_NE(view, vec);
}

}  // namespace av
