// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "arrayviewtests/Defs.h"

#include <arrayview/ArrayView.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace av {

template<typename T>
class TestArrayView : public ::testing::Test {
public:
    template<std::size_t size>
    std::array<T, size> initArray() {
        std::array<T, size> array{{}};
        for (std::size_t i = 0; i < array.size(); ++i) {
            array.at(i) = static_cast<T>(i);
        }
        return array;
    }

    template<std::size_t size>
    void initArray(T* ptr) {
        for (std::size_t i = 0; i < size; ++i) {
            *(ptr + i) = static_cast<T>(i);
        }
    }

    template<std::size_t size>
    std::vector<T> initVector() {
        std::array<T, size> array = initArray<size>();
        return std::vector<T>(array.begin(), array.end());
    }

    template<typename U>
    bool equal(const ArrayView<U>& arr, T* ptr, const std::size_t size) {
        assert(nullptr != ptr);
        return (arr == std::vector<T>(ptr, ptr + size));
    }
};

template<>
template<std::size_t size>
std::array<std::string, size> TestArrayView<std::string>::initArray() {
    std::array<std::string, size> array{{}};
    for (std::size_t i = 0; i < array.size(); ++i) {
        array.at(i) = std::to_string(i);
    }
    return array;
}

template<>
template<std::size_t size>
void TestArrayView<std::string>::initArray(std::string* ptr) {
    for (std::size_t i = 0; i < size; ++i) {
        *(ptr + i) = std::to_string(i);
    }
}

using ValueTypes = ::testing::Types<int, int32_t, int64_t, float, double, std::string>;
constexpr std::size_t length{10};

}  // namespace av
