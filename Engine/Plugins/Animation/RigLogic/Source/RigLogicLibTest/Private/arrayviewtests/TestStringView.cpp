// Copyright Epic Games, Inc. All Rights Reserved.

#include "arrayviewtests/Defs.h"

#include <arrayview/StringView.h>

namespace av {

TEST(TestStringView, CompareWithStringView) {
    const char* lhs = "lhs";
    const char* rhs = "rhs";

    StringView sv1{lhs, 3};
    StringView sv2{lhs, 3};
    StringView sv3{rhs, 3};

    EXPECT_EQ(sv1, sv2);
    EXPECT_NE(sv1, sv3);
}

TEST(TestStringView, CompareWithString) {
    const char* lhs = "lhs";
    const std::string rhs1 = "lhs";
    const std::string rhs2 = "rhs";

    StringView sv{lhs, 3};
    EXPECT_EQ(sv, rhs1);
    EXPECT_EQ(rhs1, sv);

    EXPECT_NE(sv, rhs2);
    EXPECT_NE(rhs2, sv);
}

}  // namespace av
