// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <type_traits>
#include <utility>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

template<typename>
struct true_sink : std::true_type {};

template<typename T, typename... Args>
static auto try_instantiate(std::int32_t /*unused*/, Args&&... args) -> true_sink<decltype(T{std::forward<Args>(args)...})>;

template<typename T, typename... Args>
static auto try_instantiate(std::uint32_t /*unused*/, Args&&... /*unused*/) -> std::false_type;

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4100)
#endif
template<typename T, typename... Args>
constexpr bool can_instantiate(Args&&... args) {
    using Result = decltype(try_instantiate<T>(0, std::forward<Args>(args)...));
    return Result::value;
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
