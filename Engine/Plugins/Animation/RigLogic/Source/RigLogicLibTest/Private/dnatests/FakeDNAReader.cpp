// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/FakeDNAReader.h"

namespace dna {

FakeDNAReader::~FakeDNAReader() = default;

}  // namespace dna
