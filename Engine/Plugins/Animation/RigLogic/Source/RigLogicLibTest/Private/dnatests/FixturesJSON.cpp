// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #include "dnatests/FixturesJSON.h"

namespace dna {

const char* jsonDNA = "{\n\
    \"signature\": {\n\
        \"data\": {\n\
            \"value\": [\"D\", \"N\", \"A\"]\n\
        }\n\
    },\n\
    \"version\": {\n\
        \"generation\": 2,\n\
        \"version\": 8\n\
    },\n\
    \"index\": {\n\
        \"entries\": [{\n\
            \"id\": 1684370275,\n\
            \"version\": 65537,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1684366958,\n\
            \"version\": 65537,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1651013234,\n\
            \"version\": 65537,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1734700909,\n\
            \"version\": 65537,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1835819624,\n\
            \"version\": 65536,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1919051362,\n\
            \"version\": 65536,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1919051365,\n\
            \"version\": 65536,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1784835428,\n\
            \"version\": 65536,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1953985399,\n\
            \"version\": 65536,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1835819621,\n\
            \"version\": 65536,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }, {\n\
            \"id\": 1685283685,\n\
            \"version\": 65536,\n\
            \"offset\": 0,\n\
            \"size\": 0\n\
        }]\n\
    },\n\
    \"desc1.1\": {\n\
        \"name\": \"\",\n\
        \"archetype\": 0,\n\
        \"gender\": 0,\n\
        \"age\": 0,\n\
        \"metadata\": [],\n\
        \"translationUnit\": 0,\n\
        \"rotationUnit\": 0,\n\
        \"coordinateSystem\": {\n\
            \"xAxis\": 0,\n\
            \"yAxis\": 0,\n\
            \"zAxis\": 0\n\
        },\n\
        \"lodCount\": 0,\n\
        \"maxLOD\": 0,\n\
        \"complexity\": \"\",\n\
        \"dbName\": \"\"\n\
    },\n\
    \"defn1.1\": {\n\
        \"lodJointMapping\": {\n\
            \"lods\": [],\n\
            \"indices\": []\n\
        },\n\
        \"lodBlendShapeMapping\": {\n\
            \"lods\": [],\n\
            \"indices\": []\n\
        },\n\
        \"lodAnimatedMapMapping\": {\n\
            \"lods\": [],\n\
            \"indices\": []\n\
        },\n\
        \"lodMeshMapping\": {\n\
            \"lods\": [],\n\
            \"indices\": []\n\
        },\n\
        \"guiControlNames\": [],\n\
        \"rawControlNames\": [],\n\
        \"jointNames\": [],\n\
        \"blendShapeChannelNames\": [],\n\
        \"animatedMapNames\": [],\n\
        \"meshNames\": [\"mesh0\"],\n\
        \"meshBlendShapeChannelMapping\": {\n\
            \"from\": [],\n\
            \"to\": []\n\
        },\n\
        \"jointHierarchy\": [],\n\
        \"neutralJointTranslations\": {\n\
            \"xs\": [],\n\
            \"ys\": [],\n\
            \"zs\": []\n\
        },\n\
        \"neutralJointRotations\": {\n\
            \"xs\": [],\n\
            \"ys\": [],\n\
            \"zs\": []\n\
        }\n\
    },\n\
    \"bhvr1.1\": {\n\
        \"controls\": {\n\
            \"psdCount\": 0,\n\
            \"conditionals\": {\n\
                \"inputIndices\": [],\n\
                \"outputIndices\": [],\n\
                \"fromValues\": [],\n\
                \"toValues\": [],\n\
                \"slopeValues\": [],\n\
                \"cutValues\": []\n\
            },\n\
            \"psds\": {\n\
                \"rows\": [],\n\
                \"columns\": [],\n\
                \"values\": []\n\
            }\n\
        },\n\
        \"joints\": {\n\
            \"rowCount\": 0,\n\
            \"colCount\": 0,\n\
            \"jointGroups\": []\n\
        },\n\
        \"blendShapeChannels\": {\n\
            \"lods\": [],\n\
            \"inputIndices\": [],\n\
            \"outputIndices\": []\n\
        },\n\
        \"animatedMaps\": {\n\
            \"lods\": [],\n\
            \"conditionals\": {\n\
                \"inputIndices\": [],\n\
                \"outputIndices\": [],\n\
                \"fromValues\": [],\n\
                \"toValues\": [],\n\
                \"slopeValues\": [],\n\
                \"cutValues\": []\n\
            }\n\
        }\n\
    },\n\
    \"geom1.1\": {\n\
        \"meshes\": [{\n\
            \"size\": 0,\n\
            \"positions\": {\n\
                \"xs\": [0, 3],\n\
                \"ys\": [1, 4],\n\
                \"zs\": [2, 5]\n\
            },\n\
            \"textureCoordinates\": {\n\
                \"us\": [],\n\
                \"vs\": []\n\
            },\n\
            \"normals\": {\n\
                \"xs\": [],\n\
                \"ys\": [],\n\
                \"zs\": []\n\
            },\n\
            \"layouts\": {\n\
                \"positions\": [],\n\
                \"textureCoordinates\": [],\n\
                \"normals\": []\n\
            },\n\
            \"faces\": [],\n\
            \"maximumInfluencePerVertex\": 0,\n\
            \"skinWeights\": [],\n\
            \"blendShapeTargets\": []\n\
        }]\n\
    },\n\
    \"mlbh1.0\": {\n\
        \"mlControlNames\": [],\n\
        \"lodNeuralNetworkMapping\": {\n\
            \"lods\": [],\n\
            \"indices\": []\n\
        },\n\
        \"neuralNetworkToMeshRegion\": {\n\
            \"regionNames\": [],\n\
            \"indices\": []\n\
        },\n\
        \"neuralNetworks\": []\n\
    },\n\
    \"rbfb1.0\": {\n\
        \"lodSolverMapping\": {\n\
            \"lods\": [],\n\
            \"indices\": []\n\
        },\n\
        \"solvers\": [],\n\
        \"poses\": []\n\
    },\n\
    \"rbfe1.0\": {\n\
        \"poseControlNames\": [],\n\
        \"poses\": []\n\
    },\n\
    \"jbmd1.0\": {\n\
        \"jointRepresentations\": []\n\
    },\n\
    \"twsw1.0\": {\n\
        \"twists\": [],\n\
        \"swings\": []\n\
    },\n\
    \"mlbe1.0\": {\n\
        \"mlbTypeData\": [],\n\
        \"mlbJoints\": {\n\
            \"parameterKeys\": [],\n\
            \"parameterValues\": [],\n\
            \"inputIndices\": [],\n\
            \"outputIndices\": []\n\
        }\n\
    },\n\
    \"dsce1.0\": {\n\
        \"rotationSequence\": {\n\
            \"value\": 0\n\
        },\n\
        \"rotationSign\": {\n\
            \"x\": 1,\n\
            \"y\": 1,\n\
            \"z\": 1\n\
        },\n\
        \"faceWindingOrder\": {\n\
            \"value\": 0\n\
        }\n\
    }\n\
}";

}  // namespace dna

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
