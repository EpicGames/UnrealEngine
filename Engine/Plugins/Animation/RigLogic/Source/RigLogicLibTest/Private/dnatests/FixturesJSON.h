// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// *INDENT-OFF*
#ifdef DNA_BUILD_WITH_JSON_SUPPORT

    #include "dnatests/Defs.h"

    #include <pma/TypeDefs.h>

namespace dna {

extern const char* jsonDNA;

}  // namespace dna

#endif  // DNA_BUILD_WITH_JSON_SUPPORT
// *INDENT-ON*
