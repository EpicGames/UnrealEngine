// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/Fixturesv22.h"

#include "dna/Reader.h"

namespace dna {

const unsigned char RawV22::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0x00,
    0x02,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x06,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0x6b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x57,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x00,
    0xc2,  // Definition offset
    0x00,
    0x00,
    0x03,
    0x1a,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x03,
    0xdc,  // Behavior offset
    0x00,
    0x00,
    0x05,
    0x46,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x09,
    0x22,  // Geometry offset
    0x00,
    0x00,
    0x04,
    0x38,  // Geometry size
    0x75,
    0x6e,
    0x6b,
    0x31,  // Unknown 1 id
    0x00,
    0x01,
    0x00,
    0x03,  // Unknown 1 version
    0x00,
    0x00,
    0x0d,
    0x5a,  // Unknown 1 offset
    0x00,
    0x00,
    0x00,
    0x10,  // Unknown 1 size
    0x75,
    0x6e,
    0x6b,
    0x32,  // Unknown 2 id
    0x00,
    0x01,
    0x00,
    0x02,  // Unknown 2 version
    0x00,
    0x00,
    0x0d,
    0x6a,  // Unknown 2 offset
    0x00,
    0x00,
    0x00,
    0x20  // Unknown 2 size
};

const unsigned char RawV22::descriptor[] = {
    0x00, 0x00, 0x00, 0x04,                    // Name length
    0x74, 0x65, 0x73, 0x74,                    // Name
    0x00, 0x05,                                // Archetype
    0x00, 0x02,                                // Gender
    0x00, 0x2a,                                // Age
    0x00, 0x00, 0x00, 0x02,                    // Metadata count
    0x00, 0x00, 0x00, 0x05,                    // Metadata key length
    0x6b, 0x65, 0x79, 0x2d, 0x41,              // Metadata key: "key-A"
    0x00, 0x00, 0x00, 0x07,                    // Metadata value length
    0x76, 0x61, 0x6c, 0x75, 0x65, 0x2d, 0x41,  // Metadata value: "value-A"
    0x00, 0x00, 0x00, 0x05,                    // Metadata key length
    0x6b, 0x65, 0x79, 0x2d, 0x42,              // Metadata key: "key-B"
    0x00, 0x00, 0x00, 0x07,                    // Metadata value length
    0x76, 0x61, 0x6c, 0x75, 0x65, 0x2d, 0x42,  // Metadata value: "value-B"
    0x00, 0x01,                                // Unit translation
    0x00, 0x01,                                // Unit rotation
    0x00, 0x01,                                // Coordinate system x-axis
    0x00, 0x02,                                // Coordinate system y-axis
    0x00, 0x04,                                // Coordinate system z-axis
    0x00, 0x02,                                // LOD Count
    0x00, 0x00,                                // MaxLOD: 0
    0x00, 0x00, 0x00, 0x01,                    // Complexity name length
    0x41,                                      // 'A' - Complexity name
    0x00, 0x00, 0x00, 0x06,                    // DB name length
    0x74, 0x65, 0x73, 0x74, 0x44, 0x42         // Name
};

const unsigned char RawV22::definition[] = {
    0x00, 0x00, 0x00, 0x02,  // Joint name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Joint name indices per LOD row count
    0x00, 0x00, 0x00, 0x09,  // Indices matrix row-0
    0x00, 0x00,              // Joint name index: 0
    0x00, 0x01,              // Joint name index: 1
    0x00, 0x02,              // Joint name index: 2
    0x00, 0x03,              // Joint name index: 3
    0x00, 0x04,              // Joint name index: 4
    0x00, 0x05,              // Joint name index: 5
    0x00, 0x06,              // Joint name index: 6
    0x00, 0x07,              // Joint name index: 7
    0x00, 0x08,              // Joint name index: 8
    0x00, 0x00, 0x00, 0x06,  // Indices matrix row-1
    0x00, 0x00,              // Joint name index: 0
    0x00, 0x01,              // Joint name index: 1
    0x00, 0x02,              // Joint name index: 2
    0x00, 0x03,              // Joint name index: 3
    0x00, 0x06,              // Joint name index: 6
    0x00, 0x08,              // Joint name index: 8
    0x00, 0x00, 0x00, 0x02,  // Blend shape name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Blend shape name indices per LOD row count
    0x00, 0x00, 0x00, 0x09,  // Indices matrix row-0
    0x00, 0x00,              // Blend shape name index: 0
    0x00, 0x01,              // Blend shape name index: 1
    0x00, 0x02,              // Blend shape name index: 2
    0x00, 0x03,              // Blend shape name index: 3
    0x00, 0x04,              // Blend shape name index: 4
    0x00, 0x05,              // Blend shape name index: 5
    0x00, 0x06,              // Blend shape name index: 6
    0x00, 0x07,              // Blend shape name index: 7
    0x00, 0x08,              // Blend shape name index: 8
    0x00, 0x00, 0x00, 0x04,  // Indices matrix row-1
    0x00, 0x02,              // Blend shape name index: 2
    0x00, 0x05,              // Blend shape name index: 5
    0x00, 0x07,              // Blend shape name index: 7
    0x00, 0x08,              // Blend shape name index: 8
    0x00, 0x00, 0x00, 0x02,  // Animated map name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Animated map name indices per LOD row count
    0x00, 0x00, 0x00, 0x0a,  // Indices matrix row-0
    0x00, 0x00,              // Animated map name index: 0
    0x00, 0x01,              // Animated map name index: 1
    0x00, 0x02,              // Animated map name index: 2
    0x00, 0x03,              // Animated map name index: 3
    0x00, 0x04,              // Animated map name index: 4
    0x00, 0x05,              // Animated map name index: 5
    0x00, 0x06,              // Animated map name index: 6
    0x00, 0x07,              // Animated map name index: 7
    0x00, 0x08,              // Animated map name index: 8
    0x00, 0x09,              // Animated map name index: 9
    0x00, 0x00, 0x00, 0x04,  // Indices matrix row-1
    0x00, 0x02,              // Animated map name index: 2
    0x00, 0x05,              // Animated map name index: 5
    0x00, 0x07,              // Animated map name index: 7
    0x00, 0x08,              // Animated map name index: 8
    0x00, 0x00, 0x00, 0x02,  // Mesh name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Mesh name indices per LOD row count
    0x00, 0x00, 0x00, 0x02,  // Indices matrix row-0
    0x00, 0x00,              // Mesh name index: 0
    0x00, 0x01,              // Mesh name index: 1
    0x00, 0x00, 0x00, 0x01,  // Indices matrix row-1
    0x00, 0x02,              // Mesh name index: 2
    0x00, 0x00, 0x00, 0x09,  // Gui control names length
    0x00, 0x00, 0x00, 0x02,  // Gui control name 0 length
    0x47, 0x41,              // Gui control name 0 : GA
    0x00, 0x00, 0x00, 0x02,  // Gui control name 1 length
    0x47, 0x42,              // Gui control name 1 : GB
    0x00, 0x00, 0x00, 0x02,  // Gui control name 2 length
    0x47, 0x43,              // Gui control name 2 : GC
    0x00, 0x00, 0x00, 0x02,  // Gui control name 3 length
    0x47, 0x44,              // Gui control name 3 : GD
    0x00, 0x00, 0x00, 0x02,  // Gui control name 4 length
    0x47, 0x45,              // Gui control name 4 : GE
    0x00, 0x00, 0x00, 0x02,  // Gui control name 5 length
    0x47, 0x46,              // Gui control name 5 : GF
    0x00, 0x00, 0x00, 0x02,  // Gui control name 6 length
    0x47, 0x47,              // Gui control name 6 : GG
    0x00, 0x00, 0x00, 0x02,  // Gui control name 7 length
    0x47, 0x48,              // Gui control name 7 : GH
    0x00, 0x00, 0x00, 0x02,  // Gui control name 8 length
    0x47, 0x49,              // Gui control name 8 : GI
    0x00, 0x00, 0x00, 0x09,  // Raw control names length
    0x00, 0x00, 0x00, 0x02,  // Raw control name 0 length
    0x52, 0x41,              // Raw control name 0 : RA
    0x00, 0x00, 0x00, 0x02,  // Raw control name 1 length
    0x52, 0x42,              // Raw control name 1 : RB
    0x00, 0x00, 0x00, 0x02,  // Raw control name 2 length
    0x52, 0x43,              // Raw control name 2 : RC
    0x00, 0x00, 0x00, 0x02,  // Raw control name 3 length
    0x52, 0x44,              // Raw control name 3 : RD
    0x00, 0x00, 0x00, 0x02,  // Raw control name 4 length
    0x52, 0x45,              // Raw control name 4 : RE
    0x00, 0x00, 0x00, 0x02,  // Raw control name 5 length
    0x52, 0x46,              // Raw control name 5 : RF
    0x00, 0x00, 0x00, 0x02,  // Raw control name 6 length
    0x52, 0x47,              // Raw control name 6 : RG
    0x00, 0x00, 0x00, 0x02,  // Raw control name 7 length
    0x52, 0x48,              // Raw control name 7 : RH
    0x00, 0x00, 0x00, 0x02,  // Raw control name 8 length
    0x52, 0x49,              // Raw control name 8 : RI
    0x00, 0x00, 0x00, 0x09,  // Joint names length
    0x00, 0x00, 0x00, 0x02,  // Joint name 0 length
    0x4a, 0x41,              // Joint name 0 : JA
    0x00, 0x00, 0x00, 0x02,  // Joint name 1 length
    0x4a, 0x42,              // Joint name 1 : JB
    0x00, 0x00, 0x00, 0x02,  // Joint name 2 length
    0x4a, 0x43,              // Joint name 2 : JC
    0x00, 0x00, 0x00, 0x02,  // Joint name 3 length
    0x4a, 0x44,              // Joint name 3 : JD
    0x00, 0x00, 0x00, 0x02,  // Joint name 4 length
    0x4a, 0x45,              // Joint name 4 : JE
    0x00, 0x00, 0x00, 0x02,  // Joint name 5 length
    0x4a, 0x46,              // Joint name 5 : JF
    0x00, 0x00, 0x00, 0x02,  // Joint name 6 length
    0x4a, 0x47,              // Joint name 6 : JG
    0x00, 0x00, 0x00, 0x02,  // Joint name 7 length
    0x4a, 0x48,              // Joint name 7 : JH
    0x00, 0x00, 0x00, 0x02,  // Joint name 8 length
    0x4a, 0x49,              // Joint name 8 : JI
    0x00, 0x00, 0x00, 0x09,  // BlendShape names length
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 0 length
    0x42, 0x41,              // Blendshape name 0 : BA
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 1 length
    0x42, 0x42,              // Blendshape name 1 : BB
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 2 length
    0x42, 0x43,              // Blendshape name 2 : BC
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 3 length
    0x42, 0x44,              // Blendshape name 3 : BD
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 4 length
    0x42, 0x45,              // Blendshape name 4 : BE
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 5 length
    0x42, 0x46,              // Blendshape name 5 : BF
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 6 length
    0x42, 0x47,              // Blendshape name 6 : BG
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 7 length
    0x42, 0x48,              // Blendshape name 7 : BH
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 8 length
    0x42, 0x49,              // Blendshape name 8 : BI
    0x00, 0x00, 0x00, 0x0a,  // Animated Map names length
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 0 length
    0x41, 0x41,              // Animated Map name 0 : AA
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 1 length
    0x41, 0x42,              // Animated Map name 1 : AB
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 2 length
    0x41, 0x43,              // Animated Map name 2 : AC
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 3 length
    0x41, 0x44,              // Animated Map name 3 : AD
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 4 length
    0x41, 0x45,              // Animated Map name 4 : AE
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 5 length
    0x41, 0x46,              // Animated Map name 5 : AF
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 6 length
    0x41, 0x47,              // Animated Map name 6 : AG
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 7 length
    0x41, 0x48,              // Animated Map name 7 : AH
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 8 length
    0x41, 0x49,              // Animated Map name 8 : AI
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 9 length
    0x41, 0x4a,              // Animated Map name 8 : AJ
    0x00, 0x00, 0x00, 0x03,  // Mesh names length
    0x00, 0x00, 0x00, 0x02,  // Mesh name 0 length
    0x4d, 0x41,              // Mesh name 0 : MA
    0x00, 0x00, 0x00, 0x02,  // Mesh name 1 length
    0x4d, 0x42,              // Mesh name 1 : MB
    0x00, 0x00, 0x00, 0x02,  // Mesh name 2 length
    0x4d, 0x43,              // Mesh name 2 : MC
    0x00, 0x00, 0x00, 0x09,  // Mesh indices length for mesh -> blendShape mapping
    0x00, 0x00,              // Mesh index 0
    0x00, 0x00,              // Mesh index 0
    0x00, 0x00,              // Mesh index 0
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x02,              // Mesh index 2
    0x00, 0x02,              // Mesh index 2
    0x00, 0x00, 0x00, 0x09,  // BlendShape indices length for mesh -> blendShape mapping
    0x00, 0x00,              // BlendShape 0
    0x00, 0x01,              // BlendShape 1
    0x00, 0x02,              // BlendShape 2
    0x00, 0x03,              // BlendShape 3
    0x00, 0x04,              // BlendShape 4
    0x00, 0x05,              // BlendShape 5
    0x00, 0x06,              // BlendShape 6
    0x00, 0x07,              // BlendShape 7
    0x00, 0x08,              // BlendShape 8
    0x00, 0x00, 0x00, 0x09,  // Joint hierarchy length
    0x00, 0x00,              // JA - root
    0x00, 0x00,              // JB
    0x00, 0x00,              // JC
    0x00, 0x01,              // JD
    0x00, 0x01,              // JE
    0x00, 0x04,              // JF
    0x00, 0x02,              // JG
    0x00, 0x04,              // JH
    0x00, 0x02,              // JI
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation X values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation Y values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation Z values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation X values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation Y values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation Z values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00   // 9.0f
};

const unsigned char RawV22::conditionals[]{
    // Input indices
    0x00,
    0x00,
    0x00,
    0x0f,  // Input indices count
    0x00,
    0x00,  // Index: 0      C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x02,  // Index: 2  C0      L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x05,  // Index: 5      C1  L0
    0x00,
    0x06,  // Index: 6      C1  L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x08,  // Index: 8  C0  C1  L0
    0x00,
    0x08,  // Index: 8      C1  L0
    // Output indices
    0x00,
    0x00,
    0x00,
    0x0f,  // Output indices count
    0x00,
    0x00,  // Index: 0      C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x02,  // Index: 2  C0      L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x05,  // Index: 5      C1  L0
    0x00,
    0x06,  // Index: 6      C1  L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x08,  // Index: 8  C0  C1  L0
    0x00,
    0x08,  // Index: 8      C1  L0
    // From values
    0x00,
    0x00,
    0x00,
    0x0f,  // From values count
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0  L1
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f  C0  C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0  C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0  L1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f      C1  L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f  C0      L0
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f      C1  L0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f  C0  C1  L0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0
    // To values
    0x00,
    0x00,
    0x00,
    0x0f,  // To values count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0  C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0  C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0      L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0  C1  L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    // Slope values
    0x00,
    0x00,
    0x00,
    0x0f,  // Slope values count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0  C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0  C1  L0  L1
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f      C1  L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f      C1  L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0  C1  L0
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f      C1  L0
    // Cut values
    0x00,
    0x00,
    0x00,
    0x0f,  // Cut values count
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0  L1
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f  C0  C1  L0  L1
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f  C0  C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0  L1
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f      C1  L0  L1
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f      C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f      C1  L0
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f      C1  L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0  C1  L0
    0x3e,
    0x4c,
    0xcc,
    0xcd  // 0.2f       C1  L0
};

const unsigned char RawV22::psds[]{
    // Rows
    0x00,
    0x00,
    0x00,
    0x18,  // Row index count
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x09,  // Index:  9      C2
    0x00,
    0x09,  // Index:  9      C2
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0b,  // Index: 11      C2
    0x00,
    0x0c,  // Index: 12      C2
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0e,  // Index: 14  C1
    0x00,
    0x0e,  // Index: 14  C1
    0x00,
    0x0f,  // Index: 15  C1
    0x00,
    0x10,  // Index: 16      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x13,  // Index: 19  C1
    0x00,
    0x13,  // Index: 19  C1
    0x00,
    0x14,  // Index: 20  C1
    // Columns
    0x00,
    0x00,
    0x00,
    0x18,  // Column index count
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x05,  // Index: 5      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x07,  // Index: 7  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x01,  // Index: 1  C1  C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x04,  // Index: 4  C1
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x04,  // Index: 4  C1
    0x00,
    0x05,  // Index: 5      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x07,  // Index: 7  C1
    0x00,
    0x02,  // Index: 2  C1
    // Values
    0x00,
    0x00,
    0x00,
    0x18,  // Value count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C2
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f      C2
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C1
    0x3f,
    0x80,
    0x00,
    0x00  // 1.0f  C1
};

const unsigned char RawV22::controls[] = {
    0x00,
    0x0c  // PSD count
};

const unsigned char RawV22::joints[] = {
    0x00,
    0x51,  // Rows = 81
    0x00,
    0x0a,  // Columns = 10
    // Joint groups
    0x00,
    0x00,
    0x00,
    0x04,  // Joint group count
    // Joint group-0
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x03,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x07,  // Input indices count
    0x00,
    0x00,  // Index: 0      C1
    0x00,
    0x01,  // Index: 1  C0  C1
    0x00,
    0x02,  // Index: 2  C0
    0x00,
    0x03,  // Index: 3      C1
    0x00,
    0x06,  // Index: 6      C1
    0x00,
    0x07,  // Index: 7  C0
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x02,  // Index: 2
    0x00,
    0x03,  // Index: 3
    0x00,
    0x05,  // Index: 5
    0x00,
    0x00,
    0x00,
    0x15,  // Float value count: 21
    // Row 0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.00f      C1
    0x3d,
    0x4c,
    0xcc,
    0xcd,  // 0.05f  C0  C1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.10f  C0
    0x3e,
    0x19,
    0x99,
    0x9a,  // 0.15f      C1
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.20f      C1
    0x3e,
    0x80,
    0x00,
    0x00,  // 0.25f  C0
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.30f  C0  C1
    // Row 1
    0x3e,
    0xb3,
    0x33,
    0x33,  // 0.35f      C1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.40f  C0  C1
    0x3e,
    0xe6,
    0x66,
    0x66,  // 0.45f  C0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.50f      C1
    0x3f,
    0x0c,
    0xcc,
    0xcd,  // 0.55f      C1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.60f  C0
    0x3f,
    0x26,
    0x66,
    0x66,  // 0.65f  C0  C1
    // Row 2
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.70f      C1
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0  C1
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f  C0
    0x3f,
    0x59,
    0x99,
    0x9a,  // 0.85f      C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.90f      C1
    0x3f,
    0x73,
    0x33,
    0x33,  // 0.95f  C0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.00f  C0  C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x01,  // Joint index count: 1
    0x00,
    0x00,  // Index: 0
    // Joint group-1
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x04,  // LOD-0 row-count
    0x00,
    0x02,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x05,  // Input indices count
    0x00,
    0x03,  // Index: 3      C1
    0x00,
    0x04,  // Index: 4  C0
    0x00,
    0x07,  // Index: 7  C0
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x09,  // Index: 9      C1
    0x00,
    0x00,
    0x00,
    0x04,  // Output indices count
    0x00,
    0x12,  // Index: 18
    0x00,
    0x14,  // Index: 20
    0x00,
    0x24,  // Index: 36
    0x00,
    0x26,  // Index: 38
    0x00,
    0x00,
    0x00,
    0x14,  // Float value count: 20
    // Row 0
    0x3c,
    0x23,
    0xd7,
    0x0a,  // 0.01f      C1
    0x3c,
    0xa3,
    0xd7,
    0x0a,  // 0.02f  C0
    0x3c,
    0xf5,
    0xc2,
    0x8f,  // 0.03f  C0
    0x3d,
    0x23,
    0xd7,
    0x0a,  // 0.04f  C0  C1
    0x3d,
    0x4c,
    0xcc,
    0xcd,  // 0.05f      C1
    // Row 1
    0x3d,
    0x75,
    0xc2,
    0x8f,  // 0.06f      C1
    0x3d,
    0x8f,
    0x5c,
    0x29,  // 0.07f  C0
    0x3d,
    0xa3,
    0xd7,
    0x0a,  // 0.08f  C0
    0x3d,
    0xb8,
    0x51,
    0xec,  // 0.09f  C0  C1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.10f      C1
    // Row 2
    0x3d,
    0xe1,
    0x47,
    0xae,  // 0.11f      C1
    0x3d,
    0xf5,
    0xc2,
    0x8f,  // 0.12f  C0
    0x3e,
    0x05,
    0x1e,
    0xb8,  // 0.13f  C0
    0x3e,
    0x0f,
    0x5c,
    0x29,  // 0.14f  C0  C1
    0x3e,
    0x19,
    0x99,
    0x9a,  // 0.15f      C1
    // Row 3
    0x3e,
    0x23,
    0xd7,
    0x0a,  // 0.16f      C1
    0x3e,
    0x2e,
    0x14,
    0x7b,  // 0.17f  C0
    0x3e,
    0x38,
    0x51,
    0xec,  // 0.18f  C0
    0x3e,
    0x42,
    0x8f,
    0x5c,  // 0.19f  C0  C1
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.20f      C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x02,  // Index: 2
    0x00,
    0x04,  // Index: 4
    // Joint group-2
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x02,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x04,  // Input indices count
    0x00,
    0x04,  // Index: 4  C0
    0x00,
    0x05,  // Index: 5      C1
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x09,  // Index: 9      C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x37,  // Index: 55
    0x00,
    0x38,  // Index: 56
    0x00,
    0x3f,  // Index: 63
    0x00,
    0x00,
    0x00,
    0x0c,  // Float value count: 12
    // Row 0
    0x3e,
    0x9e,
    0xb8,
    0x52,  // 0.31f  C0
    0x3e,
    0xb8,
    0x51,
    0xec,  // 0.36f      C1
    0x3e,
    0xd7,
    0x0a,
    0x3d,  // 0.42f  C0  C1
    0x3e,
    0xf0,
    0xa3,
    0xd7,  // 0.47f      C1
    // Row 1
    0x3f,
    0x07,
    0xae,
    0x14,  // 0.53f  C0
    0x3f,
    0x14,
    0x7a,
    0xe1,  // 0.58f      C1
    0x3f,
    0x23,
    0xd7,
    0x0a,  // 0.64f  C0  C1
    0x3f,
    0x30,
    0xa3,
    0xd7,  // 0.69f      C1
    // Row 2
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f      C1
    0x3f,
    0x5c,
    0x28,
    0xf6,  // 0.86f  C0  C1
    0x3f,
    0x68,
    0xf5,
    0xc3,  // 0.91f       C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x06,  // Index: 6
    0x00,
    0x07,  // Index: 7
    // Joint group-3
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x00,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x04,  // Input indices count
    0x00,
    0x02,  // Index: 2  C0
    0x00,
    0x05,  // Index: 5      C1
    0x00,
    0x06,  // Index: 6  C0  C1
    0x00,
    0x08,  // Index: 8      C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x2d,  // Index: 45
    0x00,
    0x2e,  // Index: 46
    0x00,
    0x47,  // Index: 71
    0x00,
    0x00,
    0x00,
    0x0c,  // Float value count: 12
    // Row 0
    0x3e,
    0x9e,
    0xb8,
    0x52,  // 0.31f  C0
    0x3e,
    0xb8,
    0x51,
    0xec,  // 0.36f      C1
    0x3e,
    0xd7,
    0x0a,
    0x3d,  // 0.42f  C0  C1
    0x3e,
    0xf0,
    0xa3,
    0xd7,  // 0.47f      C1
    // Row 1
    0x3f,
    0x07,
    0xae,
    0x14,  // 0.53f  C0
    0x3f,
    0x14,
    0x7a,
    0xe1,  // 0.58f      C1
    0x3f,
    0x23,
    0xd7,
    0x0a,  // 0.64f  C0  C1
    0x3f,
    0x30,
    0xa3,
    0xd7,  // 0.69f      C1
    // Row 2
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f      C1
    0x3f,
    0x5c,
    0x28,
    0xf6,  // 0.86f  C0  C1
    0x3f,
    0x68,
    0xf5,
    0xc3,  // 0.91f       C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x05,  // Index: 5
    0x00,
    0x07  // Index: 7
};

const unsigned char RawV22::blendshapes[] = {
    0x00, 0x00, 0x00, 0x02,  // LOD count
    0x00, 0x07,              // LOD-0 row-count
    0x00, 0x04,              // LOD-1 row-count
    0x00, 0x00, 0x00, 0x07,  // Input indices count
    0x00, 0x00,              // Index: 0      C1  L0  L1
    0x00, 0x01,              // Index: 1  C0  C1  L0  L1
    0x00, 0x02,              // Index: 2  C0      L0  L1
    0x00, 0x03,              // Index: 3      C1  L0  L1
    0x00, 0x06,              // Index: 6      C1  L0
    0x00, 0x07,              // Index: 7  C0      L0
    0x00, 0x08,              // Index: 8  C0  C1  L0
    0x00, 0x00, 0x00, 0x07,  // Output indices count
    0x00, 0x00,              // Index: 0      C1  L0  L1
    0x00, 0x01,              // Index: 1  C0  C1  L0  L1
    0x00, 0x02,              // Index: 2  C0      L0  L1
    0x00, 0x03,              // Index: 3      C1  L0  L1
    0x00, 0x06,              // Index: 6      C1  L0
    0x00, 0x07,              // Index: 7  C0      L0
    0x00, 0x08               // Index: 8  C0  C1  L0
};

const unsigned char RawV22::animatedmaps[] = {
    // LOD sizes
    0x00,
    0x00,
    0x00,
    0x02,  // Row count per LOD
    0x00,
    0x0f,  // LOD-0 row-count
    0x00,
    0x06  // LOD-1 row-count
};

const unsigned char RawV22::geometry[] = {
    0x00,
    0x00,
    0x00,
    0x03,  // Mesh count
    // Mesh-0
    0x00,
    0x00,
    0x01,
    0x52,  // Mesh-0 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x01,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    // Mesh-1
    0x00,
    0x00,
    0x01,
    0x52,  // Mesh-1 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x01,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    // Mesh-2
    0x00,
    0x00,
    0x01,
    0x84,  // Mesh-2 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x02,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x03  // Blend shape index in Definition
};

const unsigned char RawV22::unknownLayer1[] =
    {0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0a, 0x0b, 0x0c, 0x0d, 0x0e, 0x0f, 0x10};

const unsigned char RawV22::unknownLayer2[] = {0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0a, 0x0b,
                                               0x0c, 0x0d, 0x0e, 0x0f, 0x10, 0x11, 0x12, 0x13, 0x14, 0x15, 0x16,
                                               0x17, 0x18, 0x19, 0x1a, 0x1b, 0x1c, 0x1d, 0x1e, 0x1f, 0x20};

std::vector<char> RawV22::getBytes() {
    std::vector<char> bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Descriptor
    bytes.insert(bytes.end(), descriptor, descriptor + sizeof(descriptor));
    // Definition
    bytes.insert(bytes.end(), definition, definition + sizeof(definition));
    // Behavior
    // > Controls
    bytes.insert(bytes.end(), controls, controls + sizeof(controls));
    bytes.insert(bytes.end(), conditionals, conditionals + sizeof(conditionals));
    bytes.insert(bytes.end(), psds, psds + sizeof(psds));
    // > Joints
    bytes.insert(bytes.end(), joints, joints + sizeof(joints));
    // > BlendShapes
    bytes.insert(bytes.end(), blendshapes, blendshapes + sizeof(blendshapes));
    // > AnimatedMaps
    bytes.insert(bytes.end(), animatedmaps, animatedmaps + sizeof(animatedmaps));
    bytes.insert(bytes.end(), conditionals, conditionals + sizeof(conditionals));
    // Geometry
    bytes.insert(bytes.end(), geometry, geometry + sizeof(geometry));
    // Unknown layer 1
    bytes.insert(bytes.end(), unknownLayer1, unknownLayer1 + sizeof(unknownLayer1));
    // Unknown layer 2
    bytes.insert(bytes.end(), unknownLayer2, unknownLayer2 + sizeof(unknownLayer2));
    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

const unsigned char RawV22WithUnknownDataIgnoredAndDNARewritten::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0x00,
    0x02,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x04,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0x4b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x57,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x00,
    0xa2,  // Definition offset
    0x00,
    0x00,
    0x03,
    0x1a,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x03,
    0xbc,  // Behavior offset
    0x00,
    0x00,
    0x05,
    0x46,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x09,
    0x02,  // Geometry offset
    0x00,
    0x00,
    0x04,
    0x38  // Geometry size
};

std::vector<char> RawV22WithUnknownDataIgnoredAndDNARewritten::getBytes() {
    std::vector<char> bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Descriptor
    bytes.insert(bytes.end(), RawV22::descriptor, RawV22::descriptor + sizeof(RawV22::descriptor));
    // Definition
    bytes.insert(bytes.end(), RawV22::definition, RawV22::definition + sizeof(RawV22::definition));
    // Behavior
    // > Controls
    bytes.insert(bytes.end(), RawV22::controls, RawV22::controls + sizeof(RawV22::controls));
    bytes.insert(bytes.end(), RawV22::conditionals, RawV22::conditionals + sizeof(RawV22::conditionals));
    bytes.insert(bytes.end(), RawV22::psds, RawV22::psds + sizeof(RawV22::psds));
    // > Joints
    bytes.insert(bytes.end(), RawV22::joints, RawV22::joints + sizeof(RawV22::joints));
    // > BlendShapes
    bytes.insert(bytes.end(), RawV22::blendshapes, RawV22::blendshapes + sizeof(RawV22::blendshapes));
    // > AnimatedMaps
    bytes.insert(bytes.end(), RawV22::animatedmaps, RawV22::animatedmaps + sizeof(RawV22::animatedmaps));
    bytes.insert(bytes.end(), RawV22::conditionals, RawV22::conditionals + sizeof(RawV22::conditionals));
    // Geometry
    bytes.insert(bytes.end(), RawV22::geometry, RawV22::geometry + sizeof(RawV22::geometry));
    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

const unsigned char RawV2xNewer::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0xff,
    0xff,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x06,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x02,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0x6b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x10,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x02,  // Definition version
    0x00,
    0x00,
    0x00,
    0x7b,  // Definition offset
    0x00,
    0x00,
    0x00,
    0x10,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x02,  // Behavior version
    0x00,
    0x00,
    0x00,
    0x8b,  // Behavior offset
    0x00,
    0x00,
    0x00,
    0x20,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x02,  // Geometry version
    0x00,
    0x00,
    0x00,
    0xab,  // Geometry offset
    0x00,
    0x00,
    0x00,
    0x20,  // Geometry size
    0x75,
    0x6e,
    0x6b,
    0x31,  // Unknown 1 id
    0x00,
    0x01,
    0x00,
    0x03,  // Unknown 1 version
    0x00,
    0x00,
    0x00,
    0xcb,  // Unknown 1 offset
    0x00,
    0x00,
    0x00,
    0x10,  // Unknown 1 size
    0x75,
    0x6e,
    0x6b,
    0x32,  // Unknown 2 id
    0x00,
    0x01,
    0x00,
    0x02,  // Unknown 2 version
    0x00,
    0x00,
    0x00,
    0xdb,  // Unknown 2 offset
    0x00,
    0x00,
    0x00,
    0x20  // Unknown 2 size
};

const unsigned char RawV2xNewer::unknownDescriptor[] =
    {0xfc, 0x9b, 0x3e, 0x11, 0x46, 0xcb, 0xf9, 0x77, 0x92, 0x46, 0x30, 0xa1, 0xb4, 0xfd, 0xe9, 0x5f};

const unsigned char RawV2xNewer::unknownDefinition[] =
    {0x67, 0x20, 0x3e, 0x13, 0xa7, 0xd3, 0x03, 0x3d, 0x78, 0xe0, 0xfd, 0xc3, 0xc5, 0xe3, 0xa4, 0x7a};

const unsigned char RawV2xNewer::unknownBehavior[] = {0x5c, 0xb9, 0xe1, 0x7e, 0x50, 0x2f, 0x3b, 0x8a, 0x72, 0x3c, 0xd9,
                                                      0xbf, 0x6f, 0x10, 0xcc, 0xee, 0xa4, 0xd8, 0x65, 0x98, 0xc0, 0xb2,
                                                      0xff, 0xa2, 0x0d, 0x4d, 0xc5, 0x79, 0x99, 0x3a, 0xef, 0xc8};

const unsigned char RawV2xNewer::unknownGeometry[] = {0x9e, 0x60, 0x76, 0xe8, 0x04, 0x15, 0x99, 0xda, 0x38, 0xdd, 0xc9,
                                                      0x4a, 0xe8, 0x86, 0x83, 0x42, 0x4b, 0x32, 0x59, 0x5d, 0xf8, 0x07,
                                                      0x33, 0x7c, 0x01, 0xa4, 0x05, 0x65, 0x22, 0x25, 0x57, 0x8e};

const unsigned char RawV2xNewer::unknownCustom1[] =
    {0x69, 0x11, 0x44, 0x0a, 0x36, 0x7a, 0x2c, 0xa8, 0x31, 0x64, 0xcb, 0x8f, 0x98, 0x50, 0x0a, 0x56};

const unsigned char RawV2xNewer::unknownCustom2[] = {0xf1, 0x12, 0xca, 0xcb, 0x29, 0x3f, 0xeb, 0x0d, 0x19, 0x3e, 0x86,
                                                     0x73, 0xd9, 0x6d, 0x76, 0xd3, 0xa7, 0x70, 0xfb, 0x06, 0x6a, 0xe1,
                                                     0x8e, 0x4b, 0xb9, 0x3d, 0xf9, 0xfe, 0x49, 0xc9, 0xbe, 0x3c};

std::vector<char> RawV2xNewer::getBytes() {
    std::vector<char> bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Unknown Descriptor
    bytes.insert(bytes.end(), unknownDescriptor, unknownDescriptor + sizeof(unknownDescriptor));
    // Unknown Definition
    bytes.insert(bytes.end(), unknownDefinition, unknownDefinition + sizeof(unknownDefinition));
    // Unknown Behavior
    bytes.insert(bytes.end(), unknownBehavior, unknownBehavior + sizeof(unknownBehavior));
    // Unknown Geometry
    bytes.insert(bytes.end(), unknownGeometry, unknownGeometry + sizeof(unknownGeometry));
    // Unknown layer 1
    bytes.insert(bytes.end(), unknownCustom1, unknownCustom1 + sizeof(unknownCustom1));
    // Unknown layer 2
    bytes.insert(bytes.end(), unknownCustom2, unknownCustom2 + sizeof(unknownCustom2));
    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0xff,
    0xff,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x11,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x02,  // Descriptor version
    0x00,
    0x00,
    0x01,
    0x1b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x10,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x02,  // Definition version
    0x00,
    0x00,
    0x01,
    0x2b,  // Definition offset
    0x00,
    0x00,
    0x00,
    0x10,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x02,  // Behavior version
    0x00,
    0x00,
    0x01,
    0x3b,  // Behavior offset
    0x00,
    0x00,
    0x00,
    0x20,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x02,  // Geometry version
    0x00,
    0x00,
    0x01,
    0x5b,  // Geometry offset
    0x00,
    0x00,
    0x00,
    0x20,  // Geometry size
    0x75,
    0x6e,
    0x6b,
    0x31,  // Unknown 1 id
    0x00,
    0x01,
    0x00,
    0x03,  // Unknown 1 version
    0x00,
    0x00,
    0x01,
    0x7b,  // Unknown 1 offset
    0x00,
    0x00,
    0x00,
    0x10,  // Unknown 1 size
    0x75,
    0x6e,
    0x6b,
    0x32,  // Unknown 2 id
    0x00,
    0x01,
    0x00,
    0x02,  // Unknown 2 version
    0x00,
    0x00,
    0x01,
    0x8b,  // Unknown 2 offset
    0x00,
    0x00,
    0x00,
    0x20,  // Unknown 2 size
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x01,
    0xab,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x24,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x01,
    0xcf,  // Definition offset
    0x00,
    0x00,
    0x00,
    0x5c,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x02,
    0x2b,  // Behavior offset
    0x00,
    0x00,
    0x00,
    0x56,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x02,
    0x81,  // Geometry offset
    0x00,
    0x00,
    0x00,
    0x04,  // Geometry size
    0x6d,
    0x6c,
    0x62,
    0x68,  // Machine learned behavior id
    0x00,
    0x01,
    0x00,
    0x00,  // Machine learned behavior version
    0x00,
    0x00,
    0x02,
    0x85,  // Machine learned behavior offset
    0x00,
    0x00,
    0x00,
    0x18,  // Machine learned behavior size
    0x72,
    0x62,
    0x66,
    0x62,  // RBF behavior id
    0x00,
    0x01,
    0x00,
    0x00,  // RBF behavior version
    0x00,
    0x00,
    0x02,
    0x9d,  // RBF behavior offset
    0x00,
    0x00,
    0x00,
    0x10,  // RBF behavior size
    0x72,
    0x62,
    0x66,
    0x65,  // RBF behavior ext id
    0x00,
    0x01,
    0x00,
    0x00,  // RBF behavior ext version
    0x00,
    0x00,
    0x02,
    0xad,  // RBF behavior ext offset
    0x00,
    0x00,
    0x00,
    0x08,  // RBF behavior ext size
    0x6a,
    0x62,
    0x6d,
    0x64,  // Joint behavior metadata id
    0x00,
    0x01,
    0x00,
    0x00,  // Joint behavior metadata version
    0x00,
    0x00,
    0x02,
    0xb5,  // Joint behavior metadata offset
    0x00,
    0x00,
    0x00,
    0x04,  // Joint behavior metadata size
    0x74,
    0x77,
    0x73,
    0x77,  // Twist swing setups id
    0x00,
    0x01,
    0x00,
    0x00,  // Twist swing setups version
    0x00,
    0x00,
    0x02,
    0xb9,  // Twist swing setups offset
    0x00,
    0x00,
    0x00,
    0x08,  // Twist swing setups size
    0x6d,
    0x6c,
    0x62,
    0x65,  // Machine learned behavior ext id
    0x00,
    0x01,
    0x00,
    0x00,  // Machine learned behavior ext version
    0x00,
    0x00,
    0x02,
    0xc1,  // Machine learned behavior ext offset
    0x00,
    0x00,
    0x00,
    0x14,  // Machine learned behavior ext size
    0x64,
    0x73,
    0x63,
    0x65,  // Descriptor ext id
    0x00,
    0x01,
    0x00,
    0x00,  // Descriptor ext version
    0x00,
    0x00,
    0x02,
    0xd5,  // Descriptor ext offset
    0x00,
    0x00,
    0x00,
    0x11  // Descriptor ext size
};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor[] = {
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition[] = {
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior[] = {
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry[] = {0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehavior[] = {
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehavior[] =
    {0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehaviorExt[] =
    {0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::jointBehaviorMetaData[] = {0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::twistSwingBehavior[] =
    {0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehaviorExt[] = {
    0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00};

const unsigned char RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptorExt[] =
    {0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00, 0x01, 0x00};

std::vector<char> RawV2xNewerWithUnknownDataPreservedAndDNARewritten::getBytes() {
    std::vector<char> bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Unknown Descriptor
    bytes.insert(bytes.end(),
                 RawV2xNewer::unknownDescriptor,
                 RawV2xNewer::unknownDescriptor + sizeof(RawV2xNewer::unknownDescriptor));
    // Unknown Definition
    bytes.insert(bytes.end(),
                 RawV2xNewer::unknownDefinition,
                 RawV2xNewer::unknownDefinition + sizeof(RawV2xNewer::unknownDefinition));
    // Unknown Behavior
    bytes.insert(bytes.end(), RawV2xNewer::unknownBehavior, RawV2xNewer::unknownBehavior + sizeof(RawV2xNewer::unknownBehavior));
    // Unknown Geometry
    bytes.insert(bytes.end(), RawV2xNewer::unknownGeometry, RawV2xNewer::unknownGeometry + sizeof(RawV2xNewer::unknownGeometry));
    // Unknown layer 1
    bytes.insert(bytes.end(), RawV2xNewer::unknownCustom1, RawV2xNewer::unknownCustom1 + sizeof(RawV2xNewer::unknownCustom1));
    // Unknown layer 2
    bytes.insert(bytes.end(), RawV2xNewer::unknownCustom2, RawV2xNewer::unknownCustom2 + sizeof(RawV2xNewer::unknownCustom2));
    // Descriptor
    bytes.insert(bytes.end(), descriptor, descriptor + sizeof(descriptor));
    // Definition
    bytes.insert(bytes.end(), definition, definition + sizeof(definition));
    // Behavior
    bytes.insert(bytes.end(), behavior, behavior + sizeof(behavior));
    // Geometry
    bytes.insert(bytes.end(), geometry, geometry + sizeof(geometry));
    // Machine learned behavior
    bytes.insert(bytes.end(), machineLearnedBehavior, machineLearnedBehavior + sizeof(machineLearnedBehavior));
    // RBF behavior
    bytes.insert(bytes.end(), rbfBehavior, rbfBehavior + sizeof(rbfBehavior));
    // RBF behavior ext
    bytes.insert(bytes.end(), rbfBehaviorExt, rbfBehaviorExt + sizeof(rbfBehaviorExt));
    // Joint behavior metadata
    bytes.insert(bytes.end(), jointBehaviorMetaData, jointBehaviorMetaData + sizeof(jointBehaviorMetaData));
    // Twist swing behavior
    bytes.insert(bytes.end(), twistSwingBehavior, twistSwingBehavior + sizeof(twistSwingBehavior));
    // Machine learned behavior ext
    bytes.insert(bytes.end(), machineLearnedBehaviorExt, machineLearnedBehaviorExt + sizeof(machineLearnedBehaviorExt));
    // Descriptor ext
    bytes.insert(bytes.end(), descriptorExt, descriptorExt + sizeof(descriptorExt));

    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

const unsigned char RawV2xNewerWithUnknownDataIgnoredAndDNARewritten::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0xff,
    0xff,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x0b,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0xbb,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x24,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x00,
    0xdf,  // Definition offset
    0x00,
    0x00,
    0x00,
    0x5c,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x01,
    0x3b,  // Behavior offset
    0x00,
    0x00,
    0x00,
    0x56,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x01,
    0x91,  // Geometry offset
    0x00,
    0x00,
    0x00,
    0x04,  // Geometry size
    0x6d,
    0x6c,
    0x62,
    0x68,  // Machine learned behavior id
    0x00,
    0x01,
    0x00,
    0x00,  // Machine learned behavior version
    0x00,
    0x00,
    0x01,
    0x95,  // Machine learned behavior offset
    0x00,
    0x00,
    0x00,
    0x18,  // Machine learned behavior size
    0x72,
    0x62,
    0x66,
    0x62,  // RBF behavior id
    0x00,
    0x01,
    0x00,
    0x00,  // RBF behavior version
    0x00,
    0x00,
    0x01,
    0xad,  // RBF behavior offset
    0x00,
    0x00,
    0x00,
    0x10,  // RBF behavior size
    0x72,
    0x62,
    0x66,
    0x65,  // RBF behavior ext id
    0x00,
    0x01,
    0x00,
    0x00,  // RBF behavior ext version
    0x00,
    0x00,
    0x01,
    0xbd,  // RBF behavior ext offset
    0x00,
    0x00,
    0x00,
    0x08,  // RBF behavior ext size
    0x6a,
    0x62,
    0x6d,
    0x64,  // Joint behavior metadata id
    0x00,
    0x01,
    0x00,
    0x00,  // Joint behavior metadata version
    0x00,
    0x00,
    0x01,
    0xc5,  // Joint behavior metadata offset
    0x00,
    0x00,
    0x00,
    0x04,  // Joint behavior metadata size
    0x74,
    0x77,
    0x73,
    0x77,  // Twist swing setups id
    0x00,
    0x01,
    0x00,
    0x00,  // Twist swing setups version
    0x00,
    0x00,
    0x01,
    0xc9,  // Twist swing setups offset
    0x00,
    0x00,
    0x00,
    0x08,  // Twist swing setups size
    0x6d,
    0x6c,
    0x62,
    0x65,  // Machine learned behavior ext id
    0x00,
    0x01,
    0x00,
    0x00,  // Machine learned behavior ext version
    0x00,
    0x00,
    0x01,
    0xd1,  // Machine learned behavior ext offset
    0x00,
    0x00,
    0x00,
    0x14,  // Machine learned behavior ext size
    0x64,
    0x73,
    0x63,
    0x65,  // Descriptor ext id
    0x00,
    0x01,
    0x00,
    0x00,  // Descriptor ext version
    0x00,
    0x00,
    0x01,
    0xe5,  // Descriptor ext offset
    0x00,
    0x00,
    0x00,
    0x11  // Descriptor ext size
};

std::vector<char> RawV2xNewerWithUnknownDataIgnoredAndDNARewritten::getBytes() {
    std::vector<char> bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Descriptor
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor));
    // Definition
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition));
    // Behavior
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior));
    // Geometry
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry));
    // Machine learned behavior
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehavior,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehavior +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehavior));
    // RBF behavior
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehavior,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehavior +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehavior));
    // RBF behavior ext
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehaviorExt,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehaviorExt +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::rbfBehaviorExt));
    // Joint behavior metadata
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::jointBehaviorMetaData,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::jointBehaviorMetaData +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::jointBehaviorMetaData));
    // Twist swing behavior
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::twistSwingBehavior,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::twistSwingBehavior +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::twistSwingBehavior));
    // Machine learned behavior ext
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehaviorExt,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehaviorExt +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::machineLearnedBehaviorExt));
    // Descriptor ext
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptorExt,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptorExt +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptorExt));

    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

const unsigned char RawV22Empty::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0x00,
    0x02,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x04,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0x4b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x24,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x00,
    0x6f,  // Definition offset
    0x00,
    0x00,
    0x00,
    0x5c,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x00,
    0xcb,  // Behavior offset
    0x00,
    0x00,
    0x00,
    0x56,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x01,
    0x21,  // Geometry offset
    0x00,
    0x00,
    0x00,
    0x04  // Geometry size
};

std::vector<char> RawV22Empty::getBytes() {
    std::vector<char> bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Descriptor
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor));
    // Definition
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition));
    // Behavior
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior));
    // Geometry
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry));

    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

const unsigned char RawV22WithUnknownDataFromNewer2x::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0x00,
    0x02,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x0a,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x02,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0xab,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x10,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x02,  // Definition version
    0x00,
    0x00,
    0x00,
    0xbb,  // Definition offset
    0x00,
    0x00,
    0x00,
    0x10,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x02,  // Behavior version
    0x00,
    0x00,
    0x00,
    0xcb,  // Behavior offset
    0x00,
    0x00,
    0x00,
    0x20,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x02,  // Geometry version
    0x00,
    0x00,
    0x00,
    0xeb,  // Geometry offset
    0x00,
    0x00,
    0x00,
    0x20,  // Geometry size
    0x75,
    0x6e,
    0x6b,
    0x31,  // Unknown 1 id
    0x00,
    0x01,
    0x00,
    0x03,  // Unknown 1 version
    0x00,
    0x00,
    0x01,
    0x0b,  // Unknown 1 offset
    0x00,
    0x00,
    0x00,
    0x10,  // Unknown 1 size
    0x75,
    0x6e,
    0x6b,
    0x32,  // Unknown 2 id
    0x00,
    0x01,
    0x00,
    0x02,  // Unknown 2 version
    0x00,
    0x00,
    0x01,
    0x1b,  // Unknown 2 offset
    0x00,
    0x00,
    0x00,
    0x20,  // Unknown 2 size
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x01,
    0x3b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x24,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x01,
    0x5f,  // Definition offset
    0x00,
    0x00,
    0x00,
    0x5c,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x01,
    0xbb,  // Behavior offset
    0x00,
    0x00,
    0x00,
    0x56,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x02,
    0x11,  // Geometry offset
    0x00,
    0x00,
    0x00,
    0x04  // Geometry size
};

std::vector<char> RawV22WithUnknownDataFromNewer2x::getBytes() {
    std::vector<char> bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Unknown Descriptor
    bytes.insert(bytes.end(),
                 RawV2xNewer::unknownDescriptor,
                 RawV2xNewer::unknownDescriptor + sizeof(RawV2xNewer::unknownDescriptor));
    // Unknown Definition
    bytes.insert(bytes.end(),
                 RawV2xNewer::unknownDefinition,
                 RawV2xNewer::unknownDefinition + sizeof(RawV2xNewer::unknownDefinition));
    // Unknown Behavior
    bytes.insert(bytes.end(), RawV2xNewer::unknownBehavior, RawV2xNewer::unknownBehavior + sizeof(RawV2xNewer::unknownBehavior));
    // Unknown Geometry
    bytes.insert(bytes.end(), RawV2xNewer::unknownGeometry, RawV2xNewer::unknownGeometry + sizeof(RawV2xNewer::unknownGeometry));
    // Unknown layer 1
    bytes.insert(bytes.end(), RawV2xNewer::unknownCustom1, RawV2xNewer::unknownCustom1 + sizeof(RawV2xNewer::unknownCustom1));
    // Unknown layer 2
    bytes.insert(bytes.end(), RawV2xNewer::unknownCustom2, RawV2xNewer::unknownCustom2 + sizeof(RawV2xNewer::unknownCustom2));
    // Descriptor
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::descriptor));
    // Definition
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::definition));
    // Behavior
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::behavior));
    // Geometry
    bytes.insert(bytes.end(),
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry,
                 RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry +
                     sizeof(RawV2xNewerWithUnknownDataPreservedAndDNARewritten::geometry));
    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif
// Descriptor
const pma::String<char> DecodedV22::name = "test";
const Archetype DecodedV22::archetype = Archetype::other;
const Gender DecodedV22::gender = Gender::other;
const std::uint16_t DecodedV22::age = 42u;
const pma::Vector<DecodedV22::StringPair> DecodedV22::metadata = {{"key-A", "value-A"}, {"key-B", "value-B"}};
const TranslationUnit DecodedV22::translationUnit = TranslationUnit::m;
const RotationUnit DecodedV22::rotationUnit = RotationUnit::radians;
const CoordinateSystem DecodedV22::coordinateSystem = {Direction::right, Direction::up, Direction::front};
const std::uint16_t DecodedV22::lodCount[] = {
    2u,  // MaxLOD-0 - MinLOD-1
    1u,  // MaxLOD-1 - MinLOD-1
    1u   // MaxLOD-0 - MinLOD-0
};
const std::uint16_t DecodedV22::maxLODs[] = {
    0u,  // MaxLOD-0 - MinLOD-1
    1u,  // MaxLOD-1 - MinLOD-0
    0u   // MaxLOD-0 - MinLOD-0
};
const pma::String<char> DecodedV22::complexity = "A";
const pma::String<char> DecodedV22::dbName = "testDB";

// Definition
const pma::Vector<pma::String<char>> DecodedV22::guiControlNames = {"GA", "GB", "GC", "GD", "GE", "GF", "GG", "GH", "GI"};
const pma::Vector<pma::String<char>> DecodedV22::rawControlNames = {"RA", "RB", "RC", "RD", "RE", "RF", "RG", "RH", "RI"};
const DecodedV22::VectorOfCharStringMatrix DecodedV22::jointNames = {{// MaxLOD-0 - MinLOD-1
                                                                      {"JA", "JB", "JC", "JD", "JE", "JF", "JG", "JH", "JI"},
                                                                      {"JA", "JB", "JC", "JD", "JG", "JI"}},
                                                                     {// MaxLOD-1 - MinLOD-0
                                                                      {"JA", "JB", "JC", "JD", "JG", "JI"}},
                                                                     {
                                                                         // MaxLOD-0 - MinLOD-0
                                                                         {"JA", "JB", "JC", "JD", "JE", "JF", "JG", "JH", "JI"},
                                                                     }};
const DecodedV22::VectorOfCharStringMatrix DecodedV22::blendShapeNames = {
    {// MaxLOD-0 - MinLOD-1
     {"BA", "BB", "BC", "BD", "BE", "BF", "BG", "BH", "BI"},
     {"BC", "BF", "BH", "BI"}},
    {// MaxLOD-1 - MinLOD-1
     {"BC", "BF", "BH", "BI"}},
    {
        // MaxLOD-0 - MinLOD-0
        {"BA", "BB", "BC", "BD", "BE", "BF", "BG", "BH", "BI"},
    }};
const DecodedV22::VectorOfCharStringMatrix DecodedV22::animatedMapNames = {
    {// MaxLOD-0 - MinLOD-1
     {"AA", "AB", "AC", "AD", "AE", "AF", "AG", "AH", "AI", "AJ"},
     {"AC", "AF", "AH", "AI"}},
    {// MaxLOD-1 - MinLOD-1
     {"AC", "AF", "AH", "AI"}},
    {
        // MaxLOD-0 - MinLOD-0
        {"AA", "AB", "AC", "AD", "AE", "AF", "AG", "AH", "AI", "AJ"},
    }};
const DecodedV22::VectorOfCharStringMatrix DecodedV22::meshNames = {{// MaxLOD-0 - MinLOD-1
                                                                     {"MA", "MB"},
                                                                     {"MC"}},
                                                                    {// MaxLOD-1 - MinLOD-1
                                                                     {"MC"}},
                                                                    {// MaxLOD-0 - MinLOD-0
                                                                     {"MA", "MB"}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::meshBlendShapeIndices = {{// MaxLOD-0 - MinLOD-1
                                                                                    {0, 1, 2, 3, 4, 5, 6},
                                                                                    {7, 8}},
                                                                                   {// MaxLOD-1 - MinLOD-1
                                                                                    {0, 1}},
                                                                                   {
                                                                                       // MaxLOD-0 - MinLOD-0
                                                                                       {0, 1, 2, 3, 4, 5, 6},
                                                                                   }};
const pma::Matrix<std::uint16_t> DecodedV22::jointHierarchy = {{// MaxLOD-0 - MinLOD-1
                                                                {0, 0, 0, 1, 1, 4, 2, 4, 2}},
                                                               {// MaxLOD-1 - MinLOD-1
                                                                {0, 0, 0, 1, 2, 2}},
                                                               {// MaxLOD-0 - MinLOD-0
                                                                {0, 0, 0, 1, 1, 4, 2, 4, 2}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV22::neutralJointTranslations = {
    {// MaxLOD-0 - MinLOD-1
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}},
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-1 - MinLOD-1
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-0 - MinLOD-0
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV22::neutralJointRotations = {
    {// MaxLOD-0 - MinLOD-1
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}},
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-1 - MinLOD-1
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-0 - MinLOD-0
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}}}};

// Behavior
const std::uint16_t DecodedV22::guiControlCount = 9u;
const std::uint16_t DecodedV22::rawControlCount = 9u;
const std::uint16_t DecodedV22::psdCount = 12u;
// Behavior->Conditionals
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::conditionalInputIndices = {
    {
        // MaxLOD-0 - MinLOD-1
        {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8},
        {0, 1, 1, 2, 3, 3},
    },
    {
        // MaxLOD-1 - MinLOD-1
        {0, 1, 1, 2, 3, 3},
    },
    {// MaxLOD-0 - MinLOD-0
     {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::conditionalOutputIndices = {
    {
        // MaxLOD-0 - MinLOD-1
        {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8},
        {0, 1, 1, 2, 3, 3},
    },
    {
        // MaxLOD-1 - MinLOD-1
        {0, 1, 1, 2, 3, 3},
    },
    {// MaxLOD-0 - MinLOD-0
     {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8}}};
const pma::Vector<pma::Matrix<float>> DecodedV22::conditionalFromValues = {
    {// MaxLOD-0 - MinLOD-1
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f, 0.0f, 0.4f, 0.7f, 0.5f, 0.0f, 0.1f, 0.6f, 0.2f, 0.0f},
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f}},
    {// MaxLOD-1 - MinLOD-1
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f}},
    {// MaxLOD-0 - MinLOD-0
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f, 0.0f, 0.4f, 0.7f, 0.5f, 0.0f, 0.1f, 0.6f, 0.2f, 0.0f}}};
const pma::Vector<pma::Matrix<float>> DecodedV22::conditionalToValues = {
    {// MaxLOD-0 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f, 0.4f, 0.7f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f, 0.8f, 1.0f},
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f, 0.4f, 0.7f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f, 0.8f, 1.0f}}};
const pma::Vector<pma::Matrix<float>> DecodedV22::conditionalSlopeValues = {
    {// MaxLOD-0 - MinLOD-1
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f, 0.6f, 0.6f, 0.6f, 0.5f, 0.6f, 0.7f, 0.7f, 0.8f, 0.9f},
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f}},
    {// MaxLOD-0 - MinLOD-0
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f, 0.6f, 0.6f, 0.6f, 0.5f, 0.6f, 0.7f, 0.7f, 0.8f, 0.9f}}};
const pma::Vector<pma::Matrix<float>> DecodedV22::conditionalCutValues = {
    {// MaxLOD-0 - MinLOD-1
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f, 1.0f, 1.0f, 1.0f, 0.2f, 0.4f, 0.8f, 0.8f, 1.0f, 0.2f},
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f}},
    {// MaxLOD-1 - MinLOD-1
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f}},
    {// MaxLOD-0 - MinLOD-0
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f, 1.0f, 1.0f, 1.0f, 0.2f, 0.4f, 0.8f, 0.8f, 1.0f, 0.2f}}};
// Behavior->PSDs
const pma::Vector<std::uint16_t> DecodedV22::psdRowIndices = {8,  8,  8,  9,  9,  10, 10, 10, 11, 12, 13, 13,
                                                              13, 14, 14, 15, 16, 18, 18, 18, 18, 19, 19, 20};
const pma::Vector<std::uint16_t> DecodedV22::psdColumnIndices = {0, 3, 6, 2, 5, 2, 3, 7, 3, 2, 0, 1,
                                                                 2, 3, 6, 0, 4, 0, 3, 4, 5, 6, 7, 2};
const pma::Vector<float> DecodedV22::psdValues = {1.0f, 0.9f, 0.9f, 0.6f, 1.0f, 0.8f, 0.9f, 0.8f, 1.0f, 0.3f, 1.0f, 0.9f,
                                                  1.0f, 0.9f, 0.5f, 0.5f, 0.9f, 0.7f, 0.6f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f};
// Behavior->Joints
const pma::Vector<std::uint16_t> DecodedV22::jointRowCount = {
    81u,  // MaxLOD-0 - MinLOD-1
    54u,  // MaxLOD-1 - MinLOD-1
    81u   // MaxLOD-0 - MinLOD-0
};
const std::uint16_t DecodedV22::jointColumnCount = 10u;
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::jointVariableIndices = {
    {// MaxLOD-0 - MinLOD-1
     {2, 3, 5, 18, 20, 36, 38, 45, 46, 55, 56, 63, 71},
     {2, 3, 5, 18, 20, 55, 56}},
    {// MaxLOD-1 - MinLOD-1
     {2, 3, 5, 18, 20, 37, 38}},
    {// MaxLOD-0 - MinLOD-0
     {2, 3, 5, 18, 20, 36, 38, 45, 46, 55, 56, 63, 71}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::jointGroupLODs = {{
                                                                                // Joint Group 0
                                                                                {3, 3},  // MaxLOD-0 - MaxLOD-1
                                                                                {3},     // MaxLOD-1 - MaxLOD-1
                                                                                {3},     // MaxLOD-0 - MaxLOD-1
                                                                            },
                                                                            {
                                                                                // Joint group 1
                                                                                {4, 2},  // MaxLOD-0 - MaxLOD-1
                                                                                {2},     // MaxLOD-1 - MaxLOD-1
                                                                                {4}      // MaxLOD-0 - MaxLOD-0
                                                                            },
                                                                            {
                                                                                // Joint group 2
                                                                                {3, 2},  // MaxLOD-0 - MinLOD-1
                                                                                {2},     // MaxLOD-1 - MinLOD-1
                                                                                {3}      // MaxLOD-0 - MinLOD-0
                                                                            },
                                                                            {
                                                                                // Joint group 3
                                                                                {3, 0},  // MaxLOD-0 - MinLOD-1
                                                                                {0},     // MaxLOD-1 - MinLOD-1
                                                                                {3}      // MaxLOD-0 - MinLOD-0
                                                                            }};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV22::jointGroupInputIndices = {{ // Joint Group 0
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {0, 1, 2, 3, 6, 7, 8},
                                                                                                   {0, 1, 2, 3, 6, 7, 8}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {0, 1, 2, 3, 6, 7, 8}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {0, 1, 2, 3, 6, 7, 8}}},
                                                                                                 { // Joint Group 1
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {3, 4, 7, 8, 9},
                                                                                                   {3, 4, 7, 8, 9}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {3, 4, 7, 8, 9}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {3, 4, 7, 8, 9}}},
                                                                                                 { // Joint Group 2
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {4, 5, 8, 9},
                                                                                                   {4, 5, 8, 9}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {4, 5, 8, 9}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {4, 5, 8, 9}}},
                                                                                                 { // Joint Group 3
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {2, 5, 6, 8},
                                                                                                   {2, 5, 6, 8}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {2, 5, 6, 8}}}};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV22::jointGroupOutputIndices = {{ // Joint Group 0
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {2, 3, 5},
                                                                                                    {2, 3, 5}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {2, 3, 5}},
                                                                                                   {
                                                                                                       // MaxLOD-0 - MaxLOD-0
                                                                                                       {2, 3, 5},
                                                                                                   }},
                                                                                                  { // Joint Group 1
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {18, 20, 36, 38},
                                                                                                    {18, 20}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {18, 20}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {18, 20, 36, 38}}},
                                                                                                  { // Joint Group 2
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {55, 56, 63},
                                                                                                    {55, 56}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {37, 38}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {55, 56, 63}}},
                                                                                                  { // Joint Group 3
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {45, 46, 71},
                                                                                                    {}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {45, 46, 71}}}};
const pma::Vector<pma::Vector<pma::Matrix<float>>> DecodedV22::jointGroupValues = {
    { // Joint Group 0
     {// MaxLOD-0 - MaxLOD-1
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f},
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}},
     {// MaxLOD-1 - MinLOD-1
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}},
     {// MaxLOD-0 - MinLOD-0
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}}},
    { // Joint group 1
     {// MaxLOD-0 - MaxLOD-1
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f,
       0.11f, 0.12f, 0.13f, 0.14f, 0.15f, 0.16f, 0.17f, 0.18f, 0.19f, 0.20f},
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f}},
     {// MaxLOD-1 - MinLOD-1
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f}},
     {// MaxLOD-0 - MinLOD-0
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f,
       0.11f, 0.12f, 0.13f, 0.14f, 0.15f, 0.16f, 0.17f, 0.18f, 0.19f, 0.20f}}},
    { // Joint group 2
     {// MaxLOD-0 - MaxLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f},
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f}},
     {// MaxLOD-1 - MinLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f}},
     {// MaxLOD-0 - MinLOD-0
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f}}},
    { // Joint group 3
     {// MaxLOD-0 - MaxLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f},
      {}},
     {// MaxLOD-1 - MinLOD-1
      {}},
     {// MaxLOD-0 - MinLOD-0
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f}}}};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV22::jointGroupJointIndices = {{ // Joint Group 0
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {0},
                                                                                                   {0}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {0}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {0}}},
                                                                                                 { // Joint Group 1
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {2, 4},
                                                                                                   {2}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {2}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {2, 4}}},
                                                                                                 { // Joint Group 2
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {6, 7},
                                                                                                   {6}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {4}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {6, 7}}},
                                                                                                 { // Joint Group 3
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {5, 7},
                                                                                                   {}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {5, 7}}}};
// Behavior->BlendShapes
const pma::Matrix<std::uint16_t> DecodedV22::blendShapeLODs = {{
    {7, 4},  // MaxLOD-0 - MaxLOD-1
    {4},     // MaxLOD-1 - MinLOD-1
    {7}      // MaxLOD-0 - MinLOD-0
}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::blendShapeInputIndices = {{// MaxLOD-0 - MaxLOD-1
                                                                                     {0, 1, 2, 3, 6, 7, 8},
                                                                                     {0, 1, 2, 3}},
                                                                                    {// MaxLOD-1 - MinLOD-1
                                                                                     {0, 1, 2, 3}},
                                                                                    {// MaxLOD-0 - MinLOD-0
                                                                                     {0, 1, 2, 3, 6, 7, 8}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::blendShapeOutputIndices = {{// MaxLOD-0 - MaxLOD-1
                                                                                      {0, 1, 2, 3, 6, 7, 8},
                                                                                      {0, 1, 2, 3}},
                                                                                     {// MaxLOD-1 - MinLOD-1
                                                                                      {0, 1, 2, 3}},
                                                                                     {// MaxLOD-0 - MinLOD-0
                                                                                      {0, 1, 2, 3, 6, 7, 8}}};
// Behavior->AnimatedMaps
const pma::Vector<std::uint16_t> DecodedV22::animatedMapCount = {
    10,  // MaxLOD-0 - MaxLOD-1
    4,   // MaxLOD-1 - MinLOD-1
    10   // MaxLOD-0 - MinLOD-0
};
const pma::Matrix<std::uint16_t> DecodedV22::animatedMapLODs = {{
    {15, 6},  // MaxLOD-0 - MaxLOD-1
    {6},      // MaxLOD-1 - MinLOD-1
    {15}      // MaxLOD-0 - MinLOD-0
}};
// Geometry
const pma::Vector<std::uint32_t> DecodedV22::meshCount = {
    3u,  // MaxLOD-0 - MaxLOD-1
    1u,  // MaxLOD-1 - MinLOD-1
    2u   // MaxLOD-0 - MinLOD-0
};
const pma::Vector<pma::Matrix<Vector3>> DecodedV22::vertexPositions = {{ // MaxLOD-0 - MaxLOD-1
                                                                        {// Mesh-0
                                                                         {7.0f, 7.0f, 7.0f},
                                                                         {8.0f, 8.0f, 8.0f},
                                                                         {9.0f, 9.0f, 9.0f}},
                                                                        {// Mesh-1
                                                                         {4.0f, 4.0f, 4.0f},
                                                                         {5.0f, 5.0f, 5.0f},
                                                                         {6.0f, 6.0f, 6.0f}},
                                                                        {// Mesh-2
                                                                         {1.0f, 1.0f, 1.0f},
                                                                         {2.0f, 2.0f, 2.0f},
                                                                         {3.0f, 3.0f, 3.0f}}},
                                                                       { // MaxLOD-1 - MinLOD-1
                                                                        {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                         {1.0f, 1.0f, 1.0f},
                                                                         {2.0f, 2.0f, 2.0f},
                                                                         {3.0f, 3.0f, 3.0f}}},
                                                                       { // MaxLOD-0 - MinLOD-0
                                                                        {// Mesh-0
                                                                         {7.0f, 7.0f, 7.0f},
                                                                         {8.0f, 8.0f, 8.0f},
                                                                         {9.0f, 9.0f, 9.0f}},
                                                                        {// Mesh-1
                                                                         {4.0f, 4.0f, 4.0f},
                                                                         {5.0f, 5.0f, 5.0f},
                                                                         {6.0f, 6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<TextureCoordinate>> DecodedV22::vertexTextureCoordinates = {{ // MaxLOD-0 - MaxLOD-1
                                                                                           {// Mesh-0
                                                                                            {7.0f, 7.0f},
                                                                                            {8.0f, 8.0f},
                                                                                            {9.0f, 9.0f}},
                                                                                           {// Mesh-1
                                                                                            {4.0f, 4.0f},
                                                                                            {5.0f, 5.0f},
                                                                                            {6.0f, 6.0f}},
                                                                                           {// Mesh-2
                                                                                            {1.0f, 1.0f},
                                                                                            {2.0f, 2.0f},
                                                                                            {3.0f, 3.0f}}},
                                                                                          { // MaxLOD-1 - MinLOD-1
                                                                                           {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                            {1.0f, 1.0f},
                                                                                            {2.0f, 2.0f},
                                                                                            {3.0f, 3.0f}}},
                                                                                          { // MaxLOD-0 - MinLOD-0
                                                                                           {// Mesh-0
                                                                                            {7.0f, 7.0f},
                                                                                            {8.0f, 8.0f},
                                                                                            {9.0f, 9.0f}},
                                                                                           {// Mesh-1
                                                                                            {4.0f, 4.0f},
                                                                                            {5.0f, 5.0f},
                                                                                            {6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV22::vertexNormals = {{ // MaxLOD-0 - MaxLOD-1
                                                                      {// Mesh-0
                                                                       {7.0f, 7.0f, 7.0f},
                                                                       {8.0f, 8.0f, 8.0f},
                                                                       {9.0f, 9.0f, 9.0f}},
                                                                      {// Mesh-1
                                                                       {4.0f, 4.0f, 4.0f},
                                                                       {5.0f, 5.0f, 5.0f},
                                                                       {6.0f, 6.0f, 6.0f}},
                                                                      {// Mesh-2
                                                                       {1.0f, 1.0f, 1.0f},
                                                                       {2.0f, 2.0f, 2.0f},
                                                                       {3.0f, 3.0f, 3.0f}}},
                                                                     { // MaxLOD-1 - MinLOD-1
                                                                      {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                       {1.0f, 1.0f, 1.0f},
                                                                       {2.0f, 2.0f, 2.0f},
                                                                       {3.0f, 3.0f, 3.0f}}},
                                                                     { // MaxLOD-0 - MinLOD-0
                                                                      {// Mesh-0
                                                                       {7.0f, 7.0f, 7.0f},
                                                                       {8.0f, 8.0f, 8.0f},
                                                                       {9.0f, 9.0f, 9.0f}},
                                                                      {// Mesh-1
                                                                       {4.0f, 4.0f, 4.0f},
                                                                       {5.0f, 5.0f, 5.0f},
                                                                       {6.0f, 6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<VertexLayout>> DecodedV22::vertexLayouts = {{ // MaxLOD-0 - MaxLOD-1
                                                                           {// Mesh-0
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-1
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-2
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}},
                                                                          { // MaxLOD-1 - MinLOD-1
                                                                           {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}},
                                                                          { // MaxLOD-0 - MinLOD-0
                                                                           {// Mesh-0
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-1
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}}};
const pma::Matrix<pma::Matrix<std::uint32_t>> DecodedV22::faces = {{ // MaxLOD-0 - MaxLOD-1
                                                                    {// Mesh-0
                                                                     {0, 1, 2}},
                                                                    {// Mesh-1
                                                                     {0, 1, 2}},
                                                                    {// Mesh-2
                                                                     {0, 1, 2}}},
                                                                   { // MaxLOD-1 - MinLOD-1
                                                                    {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                     {0, 1, 2}}},
                                                                   { // MaxLOD-0 - MinLOD-0
                                                                    {// Mesh-0
                                                                     {0, 1, 2}},
                                                                    {// Mesh-1
                                                                     {0, 1, 2}}}};
const pma::Matrix<std::uint16_t> DecodedV22::maxInfluencePerVertex = {{
                                                                          // MaxLOD-0 - MaxLOD-1
                                                                          8u,  // Mesh-0
                                                                          8u,  // Mesh-1
                                                                          8u   // Mesh-2
                                                                      },
                                                                      {
                                                                          // MaxLOD-1 - MinLOD-1
                                                                          8u  // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                      },
                                                                      {
                                                                          // MaxLOD-0 - MinLOD-0
                                                                          8u,  // Mesh-0
                                                                          8u   // Mesh-1
                                                                      }};
const pma::Matrix<pma::Matrix<float>> DecodedV22::skinWeightsValues = {{ // MaxLOD-0 - MinLOD-1
                                                                        {// Mesh-0
                                                                         {0.7f, 0.1f, 0.2f},
                                                                         {0.5f, 0.5f},
                                                                         {0.4f, 0.6f}},
                                                                        {// Mesh-1
                                                                         {0.4f, 0.3f, 0.3f},
                                                                         {0.8f, 0.2f},
                                                                         {0.1f, 0.9f}},
                                                                        {// Mesh-2
                                                                         {0.1f, 0.3f, 0.6f},
                                                                         {0.3f, 0.7f},
                                                                         {0.2f, 0.8f}}},
                                                                       {// MaxLOD-1 - MinLOD-1
                                                                        {
                                                                            // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                            {0.1f, 0.3f, 0.6f},
                                                                            {1.0f},  // 0.3f normalized to 1.0f
                                                                            {1.0f}   // 0.8f normalized to 1.0f
                                                                        }},
                                                                       { // MaxLOD-0 - MinLOD-0
                                                                        {// Mesh-0
                                                                         {0.7f, 0.1f, 0.2f},
                                                                         {0.5f, 0.5f},
                                                                         {0.4f, 0.6f}},
                                                                        {// Mesh-1
                                                                         {0.4f, 0.3f, 0.3f},
                                                                         {0.8f, 0.2f},
                                                                         {0.1f, 0.9f}}}};
const pma::Matrix<pma::Matrix<std::uint16_t>> DecodedV22::skinWeightsJointIndices = {{ // MaxLOD-0 - MaxLOD-1
                                                                                      {// Mesh-0
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-1
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-2
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}}},
                                                                                     { // MaxLOD-1 - MinLOD-1
                                                                                      {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                       {0, 1, 2},
                                                                                       {3},
                                                                                       {4}}},
                                                                                     { // MaxLOD-0 - MinLOD-0
                                                                                      {// Mesh-0
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-1
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV22::correctiveBlendShapeIndices = {{ // MaxLOD-0 - MaxLOD-1
                                                                                          {// Mesh-0
                                                                                           2},
                                                                                          {// Mesh-1
                                                                                           2},
                                                                                          {// Mesh-2
                                                                                           2,
                                                                                           3}},
                                                                                         { // MaxLOD-1 - MinLOD-1
                                                                                          {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                           2}},
                                                                                         { // MaxLOD-0 - MinLOD-0
                                                                                          {// Mesh-0
                                                                                           2},
                                                                                          {// Mesh-1
                                                                                           2}}};
const pma::Matrix<pma::Matrix<Vector3>> DecodedV22::correctiveBlendShapeDeltas = {{  // MaxLOD-0 - MaxLOD-1
                                                                                   { // Mesh-0
                                                                                    {// Blendshape-0
                                                                                     {7.0f, 7.0f, 7.0f},
                                                                                     {8.0f, 8.0f, 8.0f},
                                                                                     {9.0f, 9.0f, 9.0f}}},
                                                                                   { // Mesh-1
                                                                                    {// Blendshape-0
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f},
                                                                                     {6.0f, 6.0f, 6.0f}}},
                                                                                   { // Mesh-2
                                                                                    {// Blendshape-0
                                                                                     {1.0f, 1.0f, 1.0f},
                                                                                     {2.0f, 2.0f, 2.0f},
                                                                                     {3.0f, 3.0f, 3.0f}},
                                                                                    {// Blendshape-1
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f}}}},
                                                                                  {  // MaxLOD-1 - MinLOD-1
                                                                                   { // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                    {// Blendshape-0
                                                                                     {1.0f, 1.0f, 1.0f},
                                                                                     {2.0f, 2.0f, 2.0f},
                                                                                     {3.0f, 3.0f, 3.0f}}}},
                                                                                  {  // MaxLOD-0 - MinLOD-0
                                                                                   { // Mesh-0
                                                                                    {// Blendshape-0
                                                                                     {7.0f, 7.0f, 7.0f},
                                                                                     {8.0f, 8.0f, 8.0f},
                                                                                     {9.0f, 9.0f, 9.0f}}},
                                                                                   { // Mesh-1
                                                                                    {// Blendshape-0
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f},
                                                                                     {6.0f, 6.0f, 6.0f}}}}};
const pma::Matrix<pma::Matrix<std::uint32_t>> DecodedV22::correctiveBlendShapeVertexIndices = {
    {// MaxLOD-0 - MaxLOD-1
     {
         // Mesh-0
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-1
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-2
         {0, 1, 2},  // Blendshape-0
         {0, 2}      // Blendshape-1
     }},
    {// MaxLOD-1 - MinLOD-1
     {
         // Mesh-0 (Mesh-2 under MaxLOD-0)
         {0, 1, 2}  // Blendshape-0
     }},
    {// MaxLOD-0 - MinLOD-0
     {
         // Mesh-0
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-1
         {0, 1, 2},  // Blendshape-0
     }}};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

std::size_t DecodedV22::lodConstraintToIndex(std::uint16_t maxLOD, std::uint16_t minLOD) {
    // Relies on having only TWO available LODs (0, 1)
    return (minLOD == 1u ? maxLOD : 2ul);
}

RawJoints DecodedV22::getJoints(std::uint16_t currentMaxLOD, std::uint16_t currentMinLOD, pma::MemoryResource* memRes) {
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    RawJoints joints{memRes};
    joints.rowCount = jointRowCount[srcIndex];
    joints.colCount = jointColumnCount;
    for (std::size_t i = 0ul; i < jointGroupLODs.size(); ++i) {
        RawJointGroup jntGrp{memRes};
        jntGrp.lods.assign(jointGroupLODs[i][srcIndex].begin(), jointGroupLODs[i][srcIndex].end());
        jntGrp.inputIndices.assign(jointGroupInputIndices[i][srcIndex][0ul].begin(),
                                   jointGroupInputIndices[i][srcIndex][0ul].end());
        jntGrp.outputIndices.assign(jointGroupOutputIndices[i][srcIndex][0ul].begin(),
                                    jointGroupOutputIndices[i][srcIndex][0ul].end());
        jntGrp.values.assign(jointGroupValues[i][srcIndex][0ul].begin(), jointGroupValues[i][srcIndex][0ul].end());
        jntGrp.jointIndices.assign(jointGroupJointIndices[i][srcIndex][0ul].begin(),
                                   jointGroupJointIndices[i][srcIndex][0ul].end());
        joints.jointGroups.push_back(std::move(jntGrp));
    }

    return joints;
}

RawBlendShapeChannels DecodedV22::getBlendShapes(std::uint16_t currentMaxLOD,
                                                 std::uint16_t currentMinLOD,
                                                 pma::MemoryResource* memRes) {
    RawBlendShapeChannels blendShapes{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    blendShapes.lods.assign(blendShapeLODs[srcIndex].begin(), blendShapeLODs[srcIndex].end());
    blendShapes.inputIndices.assign(blendShapeInputIndices[srcIndex][0ul].begin(), blendShapeInputIndices[srcIndex][0ul].end());
    blendShapes.outputIndices.assign(blendShapeOutputIndices[srcIndex][0ul].begin(),
                                     blendShapeOutputIndices[srcIndex][0ul].end());
    return blendShapes;
}

RawConditionalTable DecodedV22::getConditionals(std::uint16_t currentMaxLOD,
                                                std::uint16_t currentMinLOD,
                                                pma::MemoryResource* memRes) {
    RawConditionalTable conditionals{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    conditionals.inputIndices.assign(conditionalInputIndices[srcIndex][0ul].begin(),
                                     conditionalInputIndices[srcIndex][0ul].end());
    conditionals.outputIndices.assign(conditionalOutputIndices[srcIndex][0ul].begin(),
                                      conditionalOutputIndices[srcIndex][0ul].end());
    conditionals.fromValues.assign(conditionalFromValues[srcIndex][0ul].begin(), conditionalFromValues[srcIndex][0ul].end());
    conditionals.toValues.assign(conditionalToValues[srcIndex][0ul].begin(), conditionalToValues[srcIndex][0ul].end());
    conditionals.slopeValues.assign(conditionalSlopeValues[srcIndex][0ul].begin(), conditionalSlopeValues[srcIndex][0ul].end());
    conditionals.cutValues.assign(conditionalCutValues[srcIndex][0ul].begin(), conditionalCutValues[srcIndex][0ul].end());
    return conditionals;
}

RawAnimatedMaps DecodedV22::getAnimatedMaps(std::uint16_t currentMaxLOD,
                                            std::uint16_t currentMinLOD,
                                            pma::MemoryResource* memRes) {
    RawAnimatedMaps animatedMaps{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    animatedMaps.lods.assign(animatedMapLODs[srcIndex].begin(), animatedMapLODs[srcIndex].end());
    animatedMaps.conditionals = getConditionals(currentMaxLOD, currentMinLOD, memRes);
    return animatedMaps;
}

}  // namespace dna
