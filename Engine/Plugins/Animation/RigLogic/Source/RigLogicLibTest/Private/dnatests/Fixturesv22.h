// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnatests/Defs.h"

#include "dna/DNA.h"
#include "dna/Reader.h"

#include <pma/TypeDefs.h>

namespace dna {

struct RawV22 {
    static const unsigned char header[107];
    static const unsigned char descriptor[87];
    static const unsigned char definition[794];
    static const unsigned char controls[2];
    static const unsigned char conditionals[324];
    static const unsigned char psds[204];
    static const unsigned char joints[444];
    static const unsigned char blendshapes[44];
    static const unsigned char animatedmaps[8];
    static const unsigned char geometry[1080];
    static const unsigned char unknownLayer1[16];
    static const unsigned char unknownLayer2[32];

    static std::vector<char> getBytes();
};

struct RawV22WithUnknownDataIgnoredAndDNARewritten {
    static const unsigned char header[75];

    static std::vector<char> getBytes();
};

struct RawV2xNewer {
    static const unsigned char header[107];
    static const unsigned char unknownDescriptor[16];
    static const unsigned char unknownDefinition[16];
    static const unsigned char unknownBehavior[32];
    static const unsigned char unknownGeometry[32];
    static const unsigned char unknownCustom1[16];
    static const unsigned char unknownCustom2[32];

    static std::vector<char> getBytes();
};

struct RawV2xNewerWithUnknownDataPreservedAndDNARewritten {
    static const unsigned char header[283];
    static const unsigned char descriptor[36];
    static const unsigned char definition[92];
    static const unsigned char behavior[86];
    static const unsigned char geometry[4];
    static const unsigned char machineLearnedBehavior[24];
    static const unsigned char rbfBehavior[16];
    static const unsigned char rbfBehaviorExt[8];
    static const unsigned char jointBehaviorMetaData[4];
    static const unsigned char twistSwingBehavior[8];
    static const unsigned char machineLearnedBehaviorExt[20];
    static const unsigned char descriptorExt[17];

    static std::vector<char> getBytes();
};

struct RawV2xNewerWithUnknownDataIgnoredAndDNARewritten {
    static const unsigned char header[187];

    static std::vector<char> getBytes();
};

struct RawV22Empty {
    static const unsigned char header[75];

    static std::vector<char> getBytes();
};

struct RawV22WithUnknownDataFromNewer2x {
    static const unsigned char header[171];

    static std::vector<char> getBytes();
};

struct DecodedV22 {
    using VectorOfCharStringMatrix = pma::Vector<pma::Matrix<pma::String<char>>>;
    using StringPair = std::pair<pma::String<char>, pma::String<char>>;

    // Descriptor
    static const pma::String<char> name;
    static const Archetype archetype;
    static const Gender gender;
    static const std::uint16_t age;

    static const pma::Vector<StringPair> metadata;
    static const TranslationUnit translationUnit;
    static const RotationUnit rotationUnit;
    static const CoordinateSystem coordinateSystem;
    static const std::uint16_t lodCount[3ul];
    static const std::uint16_t maxLODs[3ul];
    static const pma::String<char> complexity;
    static const pma::String<char> dbName;

    // Definition
    static const pma::Vector<pma::String<char>> guiControlNames;
    static const pma::Vector<pma::String<char>> rawControlNames;
    static const VectorOfCharStringMatrix jointNames;
    static const VectorOfCharStringMatrix blendShapeNames;
    static const VectorOfCharStringMatrix animatedMapNames;
    static const VectorOfCharStringMatrix meshNames;
    static const pma::Vector<pma::Matrix<std::uint16_t>> meshBlendShapeIndices;
    static const pma::Matrix<std::uint16_t> jointHierarchy;
    static const pma::Vector<pma::Matrix<Vector3>> neutralJointTranslations;
    static const pma::Vector<pma::Matrix<Vector3>> neutralJointRotations;

    // Behavior
    static const std::uint16_t guiControlCount;
    static const std::uint16_t rawControlCount;
    static const std::uint16_t psdCount;
    static const pma::Vector<pma::Matrix<std::uint16_t>> conditionalInputIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> conditionalOutputIndices;
    static const pma::Vector<pma::Matrix<float>> conditionalFromValues;
    static const pma::Vector<pma::Matrix<float>> conditionalToValues;
    static const pma::Vector<pma::Matrix<float>> conditionalSlopeValues;
    static const pma::Vector<pma::Matrix<float>> conditionalCutValues;
    static const pma::Vector<std::uint16_t> psdRowIndices;
    static const pma::Vector<std::uint16_t> psdColumnIndices;
    static const pma::Vector<float> psdValues;
    static const pma::Vector<std::uint16_t> jointRowCount;
    static const std::uint16_t jointColumnCount;
    static const pma::Vector<pma::Matrix<std::uint16_t>> jointVariableIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> jointGroupLODs;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> jointGroupInputIndices;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> jointGroupOutputIndices;
    static const pma::Matrix<pma::Matrix<float>> jointGroupValues;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> jointGroupJointIndices;
    static const pma::Matrix<std::uint16_t> blendShapeLODs;
    static const pma::Vector<pma::Matrix<std::uint16_t>> blendShapeInputIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> blendShapeOutputIndices;
    static const pma::Vector<std::uint16_t> animatedMapCount;
    static const pma::Matrix<std::uint16_t> animatedMapLODs;

    // Geometry
    static const pma::Vector<std::uint32_t> meshCount;
    static const pma::Vector<pma::Matrix<Vector3>> vertexPositions;
    static const pma::Vector<pma::Matrix<TextureCoordinate>> vertexTextureCoordinates;
    static const pma::Vector<pma::Matrix<Vector3>> vertexNormals;
    static const pma::Vector<pma::Matrix<VertexLayout>> vertexLayouts;
    static const pma::Matrix<pma::Matrix<std::uint32_t>> faces;
    static const pma::Matrix<std::uint16_t> maxInfluencePerVertex;
    static const pma::Matrix<pma::Matrix<float>> skinWeightsValues;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> skinWeightsJointIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> correctiveBlendShapeIndices;
    static const pma::Matrix<pma::Matrix<Vector3>> correctiveBlendShapeDeltas;
    static const pma::Matrix<pma::Matrix<std::uint32_t>> correctiveBlendShapeVertexIndices;

    static std::size_t lodConstraintToIndex(std::uint16_t maxLOD, std::uint16_t minLOD);
    static RawJoints getJoints(std::uint16_t currentMaxLOD, std::uint16_t currentMinLOD, pma::MemoryResource* memRes);
    static RawBlendShapeChannels getBlendShapes(std::uint16_t currentMaxLOD,
                                                std::uint16_t currentMinLOD,
                                                pma::MemoryResource* memRes);
    static RawConditionalTable getConditionals(std::uint16_t currentMaxLOD,
                                               std::uint16_t currentMinLOD,
                                               pma::MemoryResource* memRes);
    static RawAnimatedMaps getAnimatedMaps(std::uint16_t currentMaxLOD, std::uint16_t currentMinLOD, pma::MemoryResource* memRes);
};

}  // namespace dna
