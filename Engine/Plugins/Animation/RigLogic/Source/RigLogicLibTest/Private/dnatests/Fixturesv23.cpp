// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/Fixturesv23.h"

#include "dna/Reader.h"

#pragma warning(disable : 4503)

namespace dna {

const unsigned char RawV23::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0x00,
    0x03,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x05,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0x5b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x57,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x00,
    0xb2,  // Definition offset
    0x00,
    0x00,
    0x03,
    0x1a,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x03,
    0xcc,  // Behavior offset
    0x00,
    0x00,
    0x05,
    0x46,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x09,
    0x12,  // Geometry offset
    0x00,
    0x00,
    0x04,
    0x38,  // Geometry size
    0x6d,
    0x6c,
    0x62,
    0x68,  // Machine learned behavior id
    0x00,
    0x01,
    0x00,
    0x00,  // Machine learned behavior version
    0x00,
    0x00,
    0x0d,
    0x4a,  // Machine learned behavior offset
    0x00,
    0x00,
    0x02,
    0xfa  // Machine learned behavior size
};

const unsigned char RawV23::descriptor[] = {
    0x00, 0x00, 0x00, 0x04,                    // Name length
    0x74, 0x65, 0x73, 0x74,                    // Name
    0x00, 0x05,                                // Archetype
    0x00, 0x02,                                // Gender
    0x00, 0x2a,                                // Age
    0x00, 0x00, 0x00, 0x02,                    // Metadata count
    0x00, 0x00, 0x00, 0x05,                    // Metadata key length
    0x6b, 0x65, 0x79, 0x2d, 0x41,              // Metadata key: "key-A"
    0x00, 0x00, 0x00, 0x07,                    // Metadata value length
    0x76, 0x61, 0x6c, 0x75, 0x65, 0x2d, 0x41,  // Metadata value: "value-A"
    0x00, 0x00, 0x00, 0x05,                    // Metadata key length
    0x6b, 0x65, 0x79, 0x2d, 0x42,              // Metadata key: "key-B"
    0x00, 0x00, 0x00, 0x07,                    // Metadata value length
    0x76, 0x61, 0x6c, 0x75, 0x65, 0x2d, 0x42,  // Metadata value: "value-B"
    0x00, 0x01,                                // Unit translation
    0x00, 0x01,                                // Unit rotation
    0x00, 0x01,                                // Coordinate system x-axis
    0x00, 0x02,                                // Coordinate system y-axis
    0x00, 0x04,                                // Coordinate system z-axis
    0x00, 0x02,                                // LOD Count
    0x00, 0x00,                                // MaxLOD: 0
    0x00, 0x00, 0x00, 0x01,                    // Complexity name length
    0x41,                                      // 'A' - Complexity name
    0x00, 0x00, 0x00, 0x06,                    // DB name length
    0x74, 0x65, 0x73, 0x74, 0x44, 0x42         // Name
};

const unsigned char RawV23::definition[] = {
    0x00, 0x00, 0x00, 0x02,  // Joint name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Joint name indices per LOD row count
    0x00, 0x00, 0x00, 0x09,  // Indices matrix row-0
    0x00, 0x00,              // Joint name index: 0
    0x00, 0x01,              // Joint name index: 1
    0x00, 0x02,              // Joint name index: 2
    0x00, 0x03,              // Joint name index: 3
    0x00, 0x04,              // Joint name index: 4
    0x00, 0x05,              // Joint name index: 5
    0x00, 0x06,              // Joint name index: 6
    0x00, 0x07,              // Joint name index: 7
    0x00, 0x08,              // Joint name index: 8
    0x00, 0x00, 0x00, 0x06,  // Indices matrix row-1
    0x00, 0x00,              // Joint name index: 0
    0x00, 0x01,              // Joint name index: 1
    0x00, 0x02,              // Joint name index: 2
    0x00, 0x03,              // Joint name index: 3
    0x00, 0x06,              // Joint name index: 6
    0x00, 0x08,              // Joint name index: 8
    0x00, 0x00, 0x00, 0x02,  // Blend shape name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Blend shape name indices per LOD row count
    0x00, 0x00, 0x00, 0x09,  // Indices matrix row-0
    0x00, 0x00,              // Blend shape name index: 0
    0x00, 0x01,              // Blend shape name index: 1
    0x00, 0x02,              // Blend shape name index: 2
    0x00, 0x03,              // Blend shape name index: 3
    0x00, 0x04,              // Blend shape name index: 4
    0x00, 0x05,              // Blend shape name index: 5
    0x00, 0x06,              // Blend shape name index: 6
    0x00, 0x07,              // Blend shape name index: 7
    0x00, 0x08,              // Blend shape name index: 8
    0x00, 0x00, 0x00, 0x04,  // Indices matrix row-1
    0x00, 0x02,              // Blend shape name index: 2
    0x00, 0x05,              // Blend shape name index: 5
    0x00, 0x07,              // Blend shape name index: 7
    0x00, 0x08,              // Blend shape name index: 8
    0x00, 0x00, 0x00, 0x02,  // Animated map name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Animated map name indices per LOD row count
    0x00, 0x00, 0x00, 0x0a,  // Indices matrix row-0
    0x00, 0x00,              // Animated map name index: 0
    0x00, 0x01,              // Animated map name index: 1
    0x00, 0x02,              // Animated map name index: 2
    0x00, 0x03,              // Animated map name index: 3
    0x00, 0x04,              // Animated map name index: 4
    0x00, 0x05,              // Animated map name index: 5
    0x00, 0x06,              // Animated map name index: 6
    0x00, 0x07,              // Animated map name index: 7
    0x00, 0x08,              // Animated map name index: 8
    0x00, 0x09,              // Animated map name index: 9
    0x00, 0x00, 0x00, 0x04,  // Indices matrix row-1
    0x00, 0x02,              // Animated map name index: 2
    0x00, 0x05,              // Animated map name index: 5
    0x00, 0x07,              // Animated map name index: 7
    0x00, 0x08,              // Animated map name index: 8
    0x00, 0x00, 0x00, 0x02,  // Mesh name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Mesh name indices per LOD row count
    0x00, 0x00, 0x00, 0x02,  // Indices matrix row-0
    0x00, 0x00,              // Mesh name index: 0
    0x00, 0x01,              // Mesh name index: 1
    0x00, 0x00, 0x00, 0x01,  // Indices matrix row-1
    0x00, 0x02,              // Mesh name index: 2
    0x00, 0x00, 0x00, 0x09,  // Gui control names length
    0x00, 0x00, 0x00, 0x02,  // Gui control name 0 length
    0x47, 0x41,              // Gui control name 0 : GA
    0x00, 0x00, 0x00, 0x02,  // Gui control name 1 length
    0x47, 0x42,              // Gui control name 1 : GB
    0x00, 0x00, 0x00, 0x02,  // Gui control name 2 length
    0x47, 0x43,              // Gui control name 2 : GC
    0x00, 0x00, 0x00, 0x02,  // Gui control name 3 length
    0x47, 0x44,              // Gui control name 3 : GD
    0x00, 0x00, 0x00, 0x02,  // Gui control name 4 length
    0x47, 0x45,              // Gui control name 4 : GE
    0x00, 0x00, 0x00, 0x02,  // Gui control name 5 length
    0x47, 0x46,              // Gui control name 5 : GF
    0x00, 0x00, 0x00, 0x02,  // Gui control name 6 length
    0x47, 0x47,              // Gui control name 6 : GG
    0x00, 0x00, 0x00, 0x02,  // Gui control name 7 length
    0x47, 0x48,              // Gui control name 7 : GH
    0x00, 0x00, 0x00, 0x02,  // Gui control name 8 length
    0x47, 0x49,              // Gui control name 8 : GI
    0x00, 0x00, 0x00, 0x09,  // Raw control names length
    0x00, 0x00, 0x00, 0x02,  // Raw control name 0 length
    0x52, 0x41,              // Raw control name 0 : RA
    0x00, 0x00, 0x00, 0x02,  // Raw control name 1 length
    0x52, 0x42,              // Raw control name 1 : RB
    0x00, 0x00, 0x00, 0x02,  // Raw control name 2 length
    0x52, 0x43,              // Raw control name 2 : RC
    0x00, 0x00, 0x00, 0x02,  // Raw control name 3 length
    0x52, 0x44,              // Raw control name 3 : RD
    0x00, 0x00, 0x00, 0x02,  // Raw control name 4 length
    0x52, 0x45,              // Raw control name 4 : RE
    0x00, 0x00, 0x00, 0x02,  // Raw control name 5 length
    0x52, 0x46,              // Raw control name 5 : RF
    0x00, 0x00, 0x00, 0x02,  // Raw control name 6 length
    0x52, 0x47,              // Raw control name 6 : RG
    0x00, 0x00, 0x00, 0x02,  // Raw control name 7 length
    0x52, 0x48,              // Raw control name 7 : RH
    0x00, 0x00, 0x00, 0x02,  // Raw control name 8 length
    0x52, 0x49,              // Raw control name 8 : RI
    0x00, 0x00, 0x00, 0x09,  // Joint names length
    0x00, 0x00, 0x00, 0x02,  // Joint name 0 length
    0x4a, 0x41,              // Joint name 0 : JA
    0x00, 0x00, 0x00, 0x02,  // Joint name 1 length
    0x4a, 0x42,              // Joint name 1 : JB
    0x00, 0x00, 0x00, 0x02,  // Joint name 2 length
    0x4a, 0x43,              // Joint name 2 : JC
    0x00, 0x00, 0x00, 0x02,  // Joint name 3 length
    0x4a, 0x44,              // Joint name 3 : JD
    0x00, 0x00, 0x00, 0x02,  // Joint name 4 length
    0x4a, 0x45,              // Joint name 4 : JE
    0x00, 0x00, 0x00, 0x02,  // Joint name 5 length
    0x4a, 0x46,              // Joint name 5 : JF
    0x00, 0x00, 0x00, 0x02,  // Joint name 6 length
    0x4a, 0x47,              // Joint name 6 : JG
    0x00, 0x00, 0x00, 0x02,  // Joint name 7 length
    0x4a, 0x48,              // Joint name 7 : JH
    0x00, 0x00, 0x00, 0x02,  // Joint name 8 length
    0x4a, 0x49,              // Joint name 8 : JI
    0x00, 0x00, 0x00, 0x09,  // BlendShape names length
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 0 length
    0x42, 0x41,              // Blendshape name 0 : BA
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 1 length
    0x42, 0x42,              // Blendshape name 1 : BB
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 2 length
    0x42, 0x43,              // Blendshape name 2 : BC
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 3 length
    0x42, 0x44,              // Blendshape name 3 : BD
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 4 length
    0x42, 0x45,              // Blendshape name 4 : BE
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 5 length
    0x42, 0x46,              // Blendshape name 5 : BF
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 6 length
    0x42, 0x47,              // Blendshape name 6 : BG
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 7 length
    0x42, 0x48,              // Blendshape name 7 : BH
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 8 length
    0x42, 0x49,              // Blendshape name 8 : BI
    0x00, 0x00, 0x00, 0x0a,  // Animated Map names length
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 0 length
    0x41, 0x41,              // Animated Map name 0 : AA
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 1 length
    0x41, 0x42,              // Animated Map name 1 : AB
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 2 length
    0x41, 0x43,              // Animated Map name 2 : AC
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 3 length
    0x41, 0x44,              // Animated Map name 3 : AD
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 4 length
    0x41, 0x45,              // Animated Map name 4 : AE
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 5 length
    0x41, 0x46,              // Animated Map name 5 : AF
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 6 length
    0x41, 0x47,              // Animated Map name 6 : AG
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 7 length
    0x41, 0x48,              // Animated Map name 7 : AH
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 8 length
    0x41, 0x49,              // Animated Map name 8 : AI
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 9 length
    0x41, 0x4a,              // Animated Map name 8 : AJ
    0x00, 0x00, 0x00, 0x03,  // Mesh names length
    0x00, 0x00, 0x00, 0x02,  // Mesh name 0 length
    0x4d, 0x41,              // Mesh name 0 : MA
    0x00, 0x00, 0x00, 0x02,  // Mesh name 1 length
    0x4d, 0x42,              // Mesh name 1 : MB
    0x00, 0x00, 0x00, 0x02,  // Mesh name 2 length
    0x4d, 0x43,              // Mesh name 2 : MC
    0x00, 0x00, 0x00, 0x09,  // Mesh indices length for mesh -> blendShape mapping
    0x00, 0x00,              // Mesh index 0
    0x00, 0x00,              // Mesh index 0
    0x00, 0x00,              // Mesh index 0
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x02,              // Mesh index 2
    0x00, 0x02,              // Mesh index 2
    0x00, 0x00, 0x00, 0x09,  // BlendShape indices length for mesh -> blendShape mapping
    0x00, 0x00,              // BlendShape 0
    0x00, 0x01,              // BlendShape 1
    0x00, 0x02,              // BlendShape 2
    0x00, 0x03,              // BlendShape 3
    0x00, 0x04,              // BlendShape 4
    0x00, 0x05,              // BlendShape 5
    0x00, 0x06,              // BlendShape 6
    0x00, 0x07,              // BlendShape 7
    0x00, 0x08,              // BlendShape 8
    0x00, 0x00, 0x00, 0x09,  // Joint hierarchy length
    0x00, 0x00,              // JA - root
    0x00, 0x00,              // JB
    0x00, 0x00,              // JC
    0x00, 0x01,              // JD
    0x00, 0x01,              // JE
    0x00, 0x04,              // JF
    0x00, 0x02,              // JG
    0x00, 0x04,              // JH
    0x00, 0x02,              // JI
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation X values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation Y values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation Z values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation X values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation Y values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation Z values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00   // 9.0f
};

const unsigned char RawV23::conditionals[]{
    // Input indices
    0x00,
    0x00,
    0x00,
    0x0f,  // Input indices count
    0x00,
    0x00,  // Index: 0      C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x02,  // Index: 2  C0      L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x05,  // Index: 5      C1  L0
    0x00,
    0x06,  // Index: 6      C1  L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x08,  // Index: 8  C0  C1  L0
    0x00,
    0x08,  // Index: 8      C1  L0
    // Output indices
    0x00,
    0x00,
    0x00,
    0x0f,  // Output indices count
    0x00,
    0x00,  // Index: 0      C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x02,  // Index: 2  C0      L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x05,  // Index: 5      C1  L0
    0x00,
    0x06,  // Index: 6      C1  L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x08,  // Index: 8  C0  C1  L0
    0x00,
    0x08,  // Index: 8      C1  L0
    // From values
    0x00,
    0x00,
    0x00,
    0x0f,  // From values count
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0  L1
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f  C0  C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0  C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0  L1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f      C1  L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f  C0      L0
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f      C1  L0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f  C0  C1  L0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0
    // To values
    0x00,
    0x00,
    0x00,
    0x0f,  // To values count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0  C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0  C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0      L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0  C1  L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    // Slope values
    0x00,
    0x00,
    0x00,
    0x0f,  // Slope values count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0  C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0  C1  L0  L1
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f      C1  L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f      C1  L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0  C1  L0
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f      C1  L0
    // Cut values
    0x00,
    0x00,
    0x00,
    0x0f,  // Cut values count
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0  L1
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f  C0  C1  L0  L1
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f  C0  C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0  L1
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f      C1  L0  L1
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f      C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f      C1  L0
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f      C1  L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0  C1  L0
    0x3e,
    0x4c,
    0xcc,
    0xcd  // 0.2f       C1  L0
};

const unsigned char RawV23::psds[]{
    // Rows
    0x00,
    0x00,
    0x00,
    0x18,  // Row index count
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x09,  // Index:  9      C2
    0x00,
    0x09,  // Index:  9      C2
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0b,  // Index: 11      C2
    0x00,
    0x0c,  // Index: 12      C2
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0e,  // Index: 14  C1
    0x00,
    0x0e,  // Index: 14  C1
    0x00,
    0x0f,  // Index: 15  C1
    0x00,
    0x10,  // Index: 16      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x13,  // Index: 19  C1
    0x00,
    0x13,  // Index: 19  C1
    0x00,
    0x14,  // Index: 20  C1
    // Columns
    0x00,
    0x00,
    0x00,
    0x18,  // Column index count
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x05,  // Index: 5      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x07,  // Index: 7  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x01,  // Index: 1  C1  C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x04,  // Index: 4  C1
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x04,  // Index: 4  C1
    0x00,
    0x05,  // Index: 5      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x07,  // Index: 7  C1
    0x00,
    0x02,  // Index: 2  C1
    // Values
    0x00,
    0x00,
    0x00,
    0x18,  // Value count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C2
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f      C2
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C1
    0x3f,
    0x80,
    0x00,
    0x00  // 1.0f  C1
};

const unsigned char RawV23::controls[] = {
    0x00,
    0x0c  // PSD count
};

const unsigned char RawV23::joints[] = {
    0x00,
    0x51,  // Rows = 81
    0x00,
    0x0a,  // Columns = 10
    // Joint groups
    0x00,
    0x00,
    0x00,
    0x04,  // Joint group count
    // Joint group-0
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x03,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x07,  // Input indices count
    0x00,
    0x00,  // Index: 0      C1
    0x00,
    0x01,  // Index: 1  C0  C1
    0x00,
    0x02,  // Index: 2  C0
    0x00,
    0x03,  // Index: 3      C1
    0x00,
    0x06,  // Index: 6      C1
    0x00,
    0x07,  // Index: 7  C0
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x02,  // Index: 2
    0x00,
    0x03,  // Index: 3
    0x00,
    0x05,  // Index: 5
    0x00,
    0x00,
    0x00,
    0x15,  // Float value count: 21
    // Row 0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.00f      C1
    0x3d,
    0x4c,
    0xcc,
    0xcd,  // 0.05f  C0  C1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.10f  C0
    0x3e,
    0x19,
    0x99,
    0x9a,  // 0.15f      C1
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.20f      C1
    0x3e,
    0x80,
    0x00,
    0x00,  // 0.25f  C0
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.30f  C0  C1
    // Row 1
    0x3e,
    0xb3,
    0x33,
    0x33,  // 0.35f      C1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.40f  C0  C1
    0x3e,
    0xe6,
    0x66,
    0x66,  // 0.45f  C0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.50f      C1
    0x3f,
    0x0c,
    0xcc,
    0xcd,  // 0.55f      C1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.60f  C0
    0x3f,
    0x26,
    0x66,
    0x66,  // 0.65f  C0  C1
    // Row 2
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.70f      C1
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0  C1
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f  C0
    0x3f,
    0x59,
    0x99,
    0x9a,  // 0.85f      C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.90f      C1
    0x3f,
    0x73,
    0x33,
    0x33,  // 0.95f  C0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.00f  C0  C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x01,  // Joint index count: 1
    0x00,
    0x00,  // Index: 0
    // Joint group-1
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x04,  // LOD-0 row-count
    0x00,
    0x02,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x05,  // Input indices count
    0x00,
    0x03,  // Index: 3      C1
    0x00,
    0x04,  // Index: 4  C0
    0x00,
    0x07,  // Index: 7  C0
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x09,  // Index: 9      C1
    0x00,
    0x00,
    0x00,
    0x04,  // Output indices count
    0x00,
    0x12,  // Index: 18
    0x00,
    0x14,  // Index: 20
    0x00,
    0x24,  // Index: 36
    0x00,
    0x26,  // Index: 38
    0x00,
    0x00,
    0x00,
    0x14,  // Float value count: 20
    // Row 0
    0x3c,
    0x23,
    0xd7,
    0x0a,  // 0.01f      C1
    0x3c,
    0xa3,
    0xd7,
    0x0a,  // 0.02f  C0
    0x3c,
    0xf5,
    0xc2,
    0x8f,  // 0.03f  C0
    0x3d,
    0x23,
    0xd7,
    0x0a,  // 0.04f  C0  C1
    0x3d,
    0x4c,
    0xcc,
    0xcd,  // 0.05f      C1
    // Row 1
    0x3d,
    0x75,
    0xc2,
    0x8f,  // 0.06f      C1
    0x3d,
    0x8f,
    0x5c,
    0x29,  // 0.07f  C0
    0x3d,
    0xa3,
    0xd7,
    0x0a,  // 0.08f  C0
    0x3d,
    0xb8,
    0x51,
    0xec,  // 0.09f  C0  C1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.10f      C1
    // Row 2
    0x3d,
    0xe1,
    0x47,
    0xae,  // 0.11f      C1
    0x3d,
    0xf5,
    0xc2,
    0x8f,  // 0.12f  C0
    0x3e,
    0x05,
    0x1e,
    0xb8,  // 0.13f  C0
    0x3e,
    0x0f,
    0x5c,
    0x29,  // 0.14f  C0  C1
    0x3e,
    0x19,
    0x99,
    0x9a,  // 0.15f      C1
    // Row 3
    0x3e,
    0x23,
    0xd7,
    0x0a,  // 0.16f      C1
    0x3e,
    0x2e,
    0x14,
    0x7b,  // 0.17f  C0
    0x3e,
    0x38,
    0x51,
    0xec,  // 0.18f  C0
    0x3e,
    0x42,
    0x8f,
    0x5c,  // 0.19f  C0  C1
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.20f      C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x02,  // Index: 2
    0x00,
    0x04,  // Index: 4
    // Joint group-2
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x02,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x04,  // Input indices count
    0x00,
    0x04,  // Index: 4  C0
    0x00,
    0x05,  // Index: 5      C1
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x09,  // Index: 9      C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x37,  // Index: 55
    0x00,
    0x38,  // Index: 56
    0x00,
    0x3f,  // Index: 63
    0x00,
    0x00,
    0x00,
    0x0c,  // Float value count: 12
    // Row 0
    0x3e,
    0x9e,
    0xb8,
    0x52,  // 0.31f  C0
    0x3e,
    0xb8,
    0x51,
    0xec,  // 0.36f      C1
    0x3e,
    0xd7,
    0x0a,
    0x3d,  // 0.42f  C0  C1
    0x3e,
    0xf0,
    0xa3,
    0xd7,  // 0.47f      C1
    // Row 1
    0x3f,
    0x07,
    0xae,
    0x14,  // 0.53f  C0
    0x3f,
    0x14,
    0x7a,
    0xe1,  // 0.58f      C1
    0x3f,
    0x23,
    0xd7,
    0x0a,  // 0.64f  C0  C1
    0x3f,
    0x30,
    0xa3,
    0xd7,  // 0.69f      C1
    // Row 2
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f      C1
    0x3f,
    0x5c,
    0x28,
    0xf6,  // 0.86f  C0  C1
    0x3f,
    0x68,
    0xf5,
    0xc3,  // 0.91f       C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x06,  // Index: 6
    0x00,
    0x07,  // Index: 7
    // Joint group-3
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x00,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x04,  // Input indices count
    0x00,
    0x02,  // Index: 2  C0
    0x00,
    0x05,  // Index: 5      C1
    0x00,
    0x06,  // Index: 6  C0  C1
    0x00,
    0x08,  // Index: 8      C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x2d,  // Index: 45
    0x00,
    0x2e,  // Index: 46
    0x00,
    0x47,  // Index: 71
    0x00,
    0x00,
    0x00,
    0x0c,  // Float value count: 12
    // Row 0
    0x3e,
    0x9e,
    0xb8,
    0x52,  // 0.31f  C0
    0x3e,
    0xb8,
    0x51,
    0xec,  // 0.36f      C1
    0x3e,
    0xd7,
    0x0a,
    0x3d,  // 0.42f  C0  C1
    0x3e,
    0xf0,
    0xa3,
    0xd7,  // 0.47f      C1
    // Row 1
    0x3f,
    0x07,
    0xae,
    0x14,  // 0.53f  C0
    0x3f,
    0x14,
    0x7a,
    0xe1,  // 0.58f      C1
    0x3f,
    0x23,
    0xd7,
    0x0a,  // 0.64f  C0  C1
    0x3f,
    0x30,
    0xa3,
    0xd7,  // 0.69f      C1
    // Row 2
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f      C1
    0x3f,
    0x5c,
    0x28,
    0xf6,  // 0.86f  C0  C1
    0x3f,
    0x68,
    0xf5,
    0xc3,  // 0.91f       C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x05,  // Index: 5
    0x00,
    0x07  // Index: 7
};

const unsigned char RawV23::blendshapes[] = {
    0x00, 0x00, 0x00, 0x02,  // LOD count
    0x00, 0x07,              // LOD-0 row-count
    0x00, 0x04,              // LOD-1 row-count
    0x00, 0x00, 0x00, 0x07,  // Input indices count
    0x00, 0x00,              // Index: 0      C1  L0  L1
    0x00, 0x01,              // Index: 1  C0  C1  L0  L1
    0x00, 0x02,              // Index: 2  C0      L0  L1
    0x00, 0x03,              // Index: 3      C1  L0  L1
    0x00, 0x06,              // Index: 6      C1  L0
    0x00, 0x07,              // Index: 7  C0      L0
    0x00, 0x08,              // Index: 8  C0  C1  L0
    0x00, 0x00, 0x00, 0x07,  // Output indices count
    0x00, 0x00,              // Index: 0      C1  L0  L1
    0x00, 0x01,              // Index: 1  C0  C1  L0  L1
    0x00, 0x02,              // Index: 2  C0      L0  L1
    0x00, 0x03,              // Index: 3      C1  L0  L1
    0x00, 0x06,              // Index: 6      C1  L0
    0x00, 0x07,              // Index: 7  C0      L0
    0x00, 0x08               // Index: 8  C0  C1  L0
};

const unsigned char RawV23::animatedmaps[] = {
    // LOD sizes
    0x00,
    0x00,
    0x00,
    0x02,  // Row count per LOD
    0x00,
    0x0f,  // LOD-0 row-count
    0x00,
    0x06  // LOD-1 row-count
};

const unsigned char RawV23::geometry[] = {
    0x00,
    0x00,
    0x00,
    0x03,  // Mesh count
    // Mesh-0
    0x00,
    0x00,
    0x01,
    0x52,  // Mesh-0 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x01,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    // Mesh-1
    0x00,
    0x00,
    0x01,
    0x52,  // Mesh-1 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x01,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    // Mesh-2
    0x00,
    0x00,
    0x01,
    0x84,  // Mesh-2 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x02,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x03  // Blend shape index in Definition
};

const unsigned char RawV23::machineLearnedBehavior[] = {
    0x00,
    0x00,
    0x00,
    0x09,  // Raw control names length
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 0 length
    0x4d,
    0x41,  // Raw control name 0 : MA
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 1 length
    0x4d,
    0x42,  // Raw control name 1 : MB
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 2 length
    0x4d,
    0x43,  // Raw control name 2 : MC
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 3 length
    0x4d,
    0x44,  // Raw control name 3 : MD
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 4 length
    0x4d,
    0x45,  // Raw control name 4 : ME
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 5 length
    0x4d,
    0x46,  // Raw control name 5 : MF
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 6 length
    0x4d,
    0x47,  // Raw control name 6 : MG
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 7 length
    0x4d,
    0x48,  // Raw control name 7 : MH
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 8 length
    0x4d,
    0x49,  // Raw control name 8 : MI
    0x00,
    0x00,
    0x00,
    0x02,  // Neural network indices lod to row mapping length
    0x00,
    0x00,  // Map from LOD-0 to row 0 in below defined matrix
    0x00,
    0x01,  // Map from LOD-1 to row 1 in below defined matrix
    0x00,
    0x00,
    0x00,
    0x02,  // Neural network indices per LOD row count
    0x00,
    0x00,
    0x00,
    0x04,  // Indices matrix row-0
    0x00,
    0x00,  // Neural network index: 0
    0x00,
    0x01,  // Neural network index: 1
    0x00,
    0x02,  // Neural network index: 2
    0x00,
    0x03,  // Neural network index: 3
    0x00,
    0x00,
    0x00,
    0x02,  // Indices matrix row-1
    0x00,
    0x04,  // Neural network index: 4
    0x00,
    0x05,  // Neural network index: 5
    0x00,
    0x00,
    0x00,
    0x03,  // Region names length
    0x00,
    0x00,
    0x00,
    0x02,  // Region names length for mesh 0
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 0 length
    0x52,
    0x41,  // Region name 0 : RA
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 1 length
    0x52,
    0x42,  // Region name 1 : RB
    0x00,
    0x00,
    0x00,
    0x02,  // Region names length for mesh 1
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 0 length
    0x52,
    0x43,  // Region name 0 : RC
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 1 length
    0x52,
    0x44,  // Region name 1 : RD
    0x00,
    0x00,
    0x00,
    0x02,  // Region names length for mesh 2
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 0 length
    0x52,
    0x45,  // Region name 0 : RE
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 1 length
    0x52,
    0x46,  // Region name 1 : RF
    0x00,
    0x00,
    0x00,
    0x03,  // Mesh count
    0x00,
    0x00,
    0x00,
    0x02,  // Region count for Mesh-0
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-0 Region-0
    0x00,
    0x00,  // Neural network index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-0 Region-1
    0x00,
    0x01,  // Neural network index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Region count for Mesh-1
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-1 Region-0
    0x00,
    0x02,  // Neural network index: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-1 Region-1
    0x00,
    0x03,  // Neural network index: 3
    0x00,
    0x00,
    0x00,
    0x02,  // Region count for Mesh-2
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-2 Region-0
    0x00,
    0x04,  // Neural network index: 4
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-2 Region-1
    0x00,
    0x05,  // Neural network index: 5
    0x00,
    0x00,
    0x00,
    0x06,  // Neural network count
    // Mesh-0 Region-0 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-0 Region-0 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network output index count
    0x00,
    0x09,  // Mesh-0 Region-0 neural network output index-9
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network input index count
    0x00,
    0x00,  // Mesh-0 Region-0 neural network input index-0
    0x00,
    0x01,  // Mesh-0 Region-0 neural network input index-1
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-0 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-0 neural network layer-0 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-0 Region-0 neural network layer-0 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-0 neural network layer-1 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    // Mesh-0 Region-1 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-0 Region-1 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network output index count
    0x00,
    0x0a,  // Mesh-0 Region-1 neural network output index-10
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network input index count
    0x00,
    0x02,  // Mesh-0 Region-1 neural network input index-2
    0x00,
    0x03,  // Mesh-0 Region-1 neural network input index-3
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network layer-0 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-0 Region-1 neural network layer-0 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network layer-1 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    // Mesh-1 Region-0 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-1 Region-0 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network output index count
    0x00,
    0x0b,  // Mesh-1 Region-0 neural network output index-11
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network input index count
    0x00,
    0x04,  // Mesh-1 Region-0 neural network input index-4
    0x00,
    0x05,  // Mesh-1 Region-0 neural network input index-5
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-0 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-0 neural network layer-0 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-1 Region-0 neural network layer-0 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-0 neural network layer-1 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    // Mesh-1 Region-1 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-1 Region-1 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network output index count
    0x00,
    0x0c,  // Mesh-1 Region-1 neural network output index-12
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network input index count
    0x00,
    0x06,  // Mesh-1 Region-1 neural network input index-6
    0x00,
    0x07,  // Mesh-1 Region-1 neural network input index-7
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network layer-0 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-1 Region-1 neural network layer-0 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network layer-1 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    // Mesh-2 Region-0 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-2 Region-0 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network output index count
    0x00,
    0x0d,  // Mesh-2 Region-0 neural network output index-13
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network input index count
    0x00,
    0x08,  // Mesh-2 Region-0 neural network input index-8
    0x00,
    0x00,  // Mesh-2 Region-0 neural network input index-0
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-0 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-0 neural network layer-0 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-2 Region-0 neural network layer-0 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-0 neural network layer-1 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    // Mesh-2 Region-1 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-2 Region-1 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network output index count
    0x00,
    0x0e,  // Mesh-2 Region-1 neural network output index-14
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network input index count
    0x00,
    0x04,  // Mesh-2 Region-1 neural network input index-4
    0x00,
    0x07,  // Mesh-2 Region-1 neural network input index-7
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network layer-0 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-2 Region-1 neural network layer-0 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network layer-1 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00  // 1.0f
};

std::vector<char> RawV23::getBytes() {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    std::vector<char> bytes;
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Descriptor
    bytes.insert(bytes.end(), descriptor, descriptor + sizeof(descriptor));
    // Definition
    bytes.insert(bytes.end(), definition, definition + sizeof(definition));
    // Behavior
    // > Controls
    bytes.insert(bytes.end(), controls, controls + sizeof(controls));
    bytes.insert(bytes.end(), conditionals, conditionals + sizeof(conditionals));
    bytes.insert(bytes.end(), psds, psds + sizeof(psds));
    // > Joints
    bytes.insert(bytes.end(), joints, joints + sizeof(joints));
    // > BlendShapes
    bytes.insert(bytes.end(), blendshapes, blendshapes + sizeof(blendshapes));
    // > AnimatedMaps
    bytes.insert(bytes.end(), animatedmaps, animatedmaps + sizeof(animatedmaps));
    bytes.insert(bytes.end(), conditionals, conditionals + sizeof(conditionals));
    // Geometry
    bytes.insert(bytes.end(), geometry, geometry + sizeof(geometry));
    // Machine learned behavior
    bytes.insert(bytes.end(), machineLearnedBehavior, machineLearnedBehavior + sizeof(machineLearnedBehavior));
    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

std::vector<char> RawV22DowngradedFromV23::getBytes() {
    auto bytes = RawV23::getBytes();
    bytes[5] = 0x00;
    bytes[6] = 0x02;
    return bytes;
}

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif
// Descriptor
const pma::String<char> DecodedV23::name = "test";
const Archetype DecodedV23::archetype = Archetype::other;
const Gender DecodedV23::gender = Gender::other;
const std::uint16_t DecodedV23::age = 42u;
const pma::Vector<DecodedV23::StringPair> DecodedV23::metadata = {{"key-A", "value-A"}, {"key-B", "value-B"}};
const TranslationUnit DecodedV23::translationUnit = TranslationUnit::m;
const RotationUnit DecodedV23::rotationUnit = RotationUnit::radians;
const CoordinateSystem DecodedV23::coordinateSystem = {Direction::right, Direction::up, Direction::front};
const std::uint16_t DecodedV23::lodCount[] = {
    2u,  // MaxLOD-0 - MinLOD-1
    1u,  // MaxLOD-1 - MinLOD-1
    1u   // MaxLOD-0 - MinLOD-0
};
const std::uint16_t DecodedV23::maxLODs[] = {
    0u,  // MaxLOD-0 - MinLOD-1
    1u,  // MaxLOD-1 - MinLOD-0
    0u   // MaxLOD-0 - MinLOD-0
};
const pma::String<char> DecodedV23::complexity = "A";
const pma::String<char> DecodedV23::dbName = "testDB";

// Definition
const pma::Vector<pma::String<char>> DecodedV23::guiControlNames = {"GA", "GB", "GC", "GD", "GE", "GF", "GG", "GH", "GI"};
const pma::Vector<pma::String<char>> DecodedV23::rawControlNames = {"RA", "RB", "RC", "RD", "RE", "RF", "RG", "RH", "RI"};
const DecodedV23::VectorOfCharStringMatrix DecodedV23::jointNames = {{// MaxLOD-0 - MinLOD-1
                                                                      {"JA", "JB", "JC", "JD", "JE", "JF", "JG", "JH", "JI"},
                                                                      {"JA", "JB", "JC", "JD", "JG", "JI"}},
                                                                     {// MaxLOD-1 - MinLOD-0
                                                                      {"JA", "JB", "JC", "JD", "JG", "JI"}},
                                                                     {
                                                                         // MaxLOD-0 - MinLOD-0
                                                                         {"JA", "JB", "JC", "JD", "JE", "JF", "JG", "JH", "JI"},
                                                                     }};
const DecodedV23::VectorOfCharStringMatrix DecodedV23::blendShapeNames = {
    {// MaxLOD-0 - MinLOD-1
     {"BA", "BB", "BC", "BD", "BE", "BF", "BG", "BH", "BI"},
     {"BC", "BF", "BH", "BI"}},
    {// MaxLOD-1 - MinLOD-1
     {"BC", "BF", "BH", "BI"}},
    {
        // MaxLOD-0 - MinLOD-0
        {"BA", "BB", "BC", "BD", "BE", "BF", "BG", "BH", "BI"},
    }};
const DecodedV23::VectorOfCharStringMatrix DecodedV23::animatedMapNames = {
    {// MaxLOD-0 - MinLOD-1
     {"AA", "AB", "AC", "AD", "AE", "AF", "AG", "AH", "AI", "AJ"},
     {"AC", "AF", "AH", "AI"}},
    {// MaxLOD-1 - MinLOD-1
     {"AC", "AF", "AH", "AI"}},
    {
        // MaxLOD-0 - MinLOD-0
        {"AA", "AB", "AC", "AD", "AE", "AF", "AG", "AH", "AI", "AJ"},
    }};
const DecodedV23::VectorOfCharStringMatrix DecodedV23::meshNames = {{// MaxLOD-0 - MinLOD-1
                                                                     {"MA", "MB"},
                                                                     {"MC"}},
                                                                    {// MaxLOD-1 - MinLOD-1
                                                                     {"MC"}},
                                                                    {// MaxLOD-0 - MinLOD-0
                                                                     {"MA", "MB"}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::meshBlendShapeIndices = {{// MaxLOD-0 - MinLOD-1
                                                                                    {0, 1, 2, 3, 4, 5, 6},
                                                                                    {7, 8}},
                                                                                   {// MaxLOD-1 - MinLOD-1
                                                                                    {0, 1}},
                                                                                   {
                                                                                       // MaxLOD-0 - MinLOD-0
                                                                                       {0, 1, 2, 3, 4, 5, 6},
                                                                                   }};
const pma::Matrix<std::uint16_t> DecodedV23::jointHierarchy = {{// MaxLOD-0 - MinLOD-1
                                                                {0, 0, 0, 1, 1, 4, 2, 4, 2}},
                                                               {// MaxLOD-1 - MinLOD-1
                                                                {0, 0, 0, 1, 2, 2}},
                                                               {// MaxLOD-0 - MinLOD-0
                                                                {0, 0, 0, 1, 1, 4, 2, 4, 2}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV23::neutralJointTranslations = {
    {// MaxLOD-0 - MinLOD-1
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}},
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-1 - MinLOD-1
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-0 - MinLOD-0
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV23::neutralJointRotations = {
    {// MaxLOD-0 - MinLOD-1
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}},
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-1 - MinLOD-1
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-0 - MinLOD-0
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}}}};

// Behavior
const std::uint16_t DecodedV23::guiControlCount = 9u;
const std::uint16_t DecodedV23::rawControlCount = 9u;
const std::uint16_t DecodedV23::psdCount = 12u;
// Behavior->Conditionals
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::conditionalInputIndices = {
    {
        // MaxLOD-0 - MinLOD-1
        {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8},
        {0, 1, 1, 2, 3, 3},
    },
    {
        // MaxLOD-1 - MinLOD-1
        {0, 1, 1, 2, 3, 3},
    },
    {// MaxLOD-0 - MinLOD-0
     {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::conditionalOutputIndices = {
    {
        // MaxLOD-0 - MinLOD-1
        {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8},
        {0, 1, 1, 2, 3, 3},
    },
    {
        // MaxLOD-1 - MinLOD-1
        {0, 1, 1, 2, 3, 3},
    },
    {// MaxLOD-0 - MinLOD-0
     {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8}}};
const pma::Vector<pma::Matrix<float>> DecodedV23::conditionalFromValues = {
    {// MaxLOD-0 - MinLOD-1
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f, 0.0f, 0.4f, 0.7f, 0.5f, 0.0f, 0.1f, 0.6f, 0.2f, 0.0f},
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f}},
    {// MaxLOD-1 - MinLOD-1
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f}},
    {// MaxLOD-0 - MinLOD-0
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f, 0.0f, 0.4f, 0.7f, 0.5f, 0.0f, 0.1f, 0.6f, 0.2f, 0.0f}}};
const pma::Vector<pma::Matrix<float>> DecodedV23::conditionalToValues = {
    {// MaxLOD-0 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f, 0.4f, 0.7f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f, 0.8f, 1.0f},
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f, 0.4f, 0.7f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f, 0.8f, 1.0f}}};
const pma::Vector<pma::Matrix<float>> DecodedV23::conditionalSlopeValues = {
    {// MaxLOD-0 - MinLOD-1
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f, 0.6f, 0.6f, 0.6f, 0.5f, 0.6f, 0.7f, 0.7f, 0.8f, 0.9f},
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f}},
    {// MaxLOD-0 - MinLOD-0
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f, 0.6f, 0.6f, 0.6f, 0.5f, 0.6f, 0.7f, 0.7f, 0.8f, 0.9f}}};
const pma::Vector<pma::Matrix<float>> DecodedV23::conditionalCutValues = {
    {// MaxLOD-0 - MinLOD-1
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f, 1.0f, 1.0f, 1.0f, 0.2f, 0.4f, 0.8f, 0.8f, 1.0f, 0.2f},
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f}},
    {// MaxLOD-1 - MinLOD-1
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f}},
    {// MaxLOD-0 - MinLOD-0
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f, 1.0f, 1.0f, 1.0f, 0.2f, 0.4f, 0.8f, 0.8f, 1.0f, 0.2f}}};
// Behavior->PSDs
const pma::Vector<std::uint16_t> DecodedV23::psdRowIndices = {8,  8,  8,  9,  9,  10, 10, 10, 11, 12, 13, 13,
                                                              13, 14, 14, 15, 16, 18, 18, 18, 18, 19, 19, 20};
const pma::Vector<std::uint16_t> DecodedV23::psdColumnIndices = {0, 3, 6, 2, 5, 2, 3, 7, 3, 2, 0, 1,
                                                                 2, 3, 6, 0, 4, 0, 3, 4, 5, 6, 7, 2};
const pma::Vector<float> DecodedV23::psdValues = {1.0f, 0.9f, 0.9f, 0.6f, 1.0f, 0.8f, 0.9f, 0.8f, 1.0f, 0.3f, 1.0f, 0.9f,
                                                  1.0f, 0.9f, 0.5f, 0.5f, 0.9f, 0.7f, 0.6f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f};
// Behavior->Joints
const pma::Vector<std::uint16_t> DecodedV23::jointRowCount = {
    81u,  // MaxLOD-0 - MinLOD-1
    54u,  // MaxLOD-1 - MinLOD-1
    81u   // MaxLOD-0 - MinLOD-0
};
const std::uint16_t DecodedV23::jointColumnCount = 10u;
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::jointVariableIndices = {
    {// MaxLOD-0 - MinLOD-1
     {2, 3, 5, 18, 20, 36, 38, 45, 46, 55, 56, 63, 71},
     {2, 3, 5, 18, 20, 55, 56}},
    {// MaxLOD-1 - MinLOD-1
     {2, 3, 5, 18, 20, 37, 38}},
    {// MaxLOD-0 - MinLOD-0
     {2, 3, 5, 18, 20, 36, 38, 45, 46, 55, 56, 63, 71}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::jointGroupLODs = {{
                                                                                // Joint Group 0
                                                                                {3, 3},  // MaxLOD-0 - MaxLOD-1
                                                                                {3},     // MaxLOD-1 - MaxLOD-1
                                                                                {3},     // MaxLOD-0 - MaxLOD-1
                                                                            },
                                                                            {
                                                                                // Joint group 1
                                                                                {4, 2},  // MaxLOD-0 - MaxLOD-1
                                                                                {2},     // MaxLOD-1 - MaxLOD-1
                                                                                {4}      // MaxLOD-0 - MaxLOD-0
                                                                            },
                                                                            {
                                                                                // Joint group 2
                                                                                {3, 2},  // MaxLOD-0 - MinLOD-1
                                                                                {2},     // MaxLOD-1 - MinLOD-1
                                                                                {3}      // MaxLOD-0 - MinLOD-0
                                                                            },
                                                                            {
                                                                                // Joint group 3
                                                                                {3, 0},  // MaxLOD-0 - MinLOD-1
                                                                                {0},     // MaxLOD-1 - MinLOD-1
                                                                                {3}      // MaxLOD-0 - MinLOD-0
                                                                            }};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV23::jointGroupInputIndices = {{ // Joint Group 0
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {0, 1, 2, 3, 6, 7, 8},
                                                                                                   {0, 1, 2, 3, 6, 7, 8}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {0, 1, 2, 3, 6, 7, 8}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {0, 1, 2, 3, 6, 7, 8}}},
                                                                                                 { // Joint Group 1
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {3, 4, 7, 8, 9},
                                                                                                   {3, 4, 7, 8, 9}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {3, 4, 7, 8, 9}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {3, 4, 7, 8, 9}}},
                                                                                                 { // Joint Group 2
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {4, 5, 8, 9},
                                                                                                   {4, 5, 8, 9}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {4, 5, 8, 9}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {4, 5, 8, 9}}},
                                                                                                 { // Joint Group 3
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {2, 5, 6, 8},
                                                                                                   {2, 5, 6, 8}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {2, 5, 6, 8}}}};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV23::jointGroupOutputIndices = {{ // Joint Group 0
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {2, 3, 5},
                                                                                                    {2, 3, 5}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {2, 3, 5}},
                                                                                                   {
                                                                                                       // MaxLOD-0 - MaxLOD-0
                                                                                                       {2, 3, 5},
                                                                                                   }},
                                                                                                  { // Joint Group 1
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {18, 20, 36, 38},
                                                                                                    {18, 20}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {18, 20}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {18, 20, 36, 38}}},
                                                                                                  { // Joint Group 2
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {55, 56, 63},
                                                                                                    {55, 56}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {37, 38}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {55, 56, 63}}},
                                                                                                  { // Joint Group 3
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {45, 46, 71},
                                                                                                    {}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {45, 46, 71}}}};
const pma::Vector<pma::Vector<pma::Matrix<float>>> DecodedV23::jointGroupValues = {
    { // Joint Group 0
     {// MaxLOD-0 - MaxLOD-1
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f},
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}},
     {// MaxLOD-1 - MinLOD-1
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}},
     {// MaxLOD-0 - MinLOD-0
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}}},
    { // Joint group 1
     {// MaxLOD-0 - MaxLOD-1
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f,
       0.11f, 0.12f, 0.13f, 0.14f, 0.15f, 0.16f, 0.17f, 0.18f, 0.19f, 0.20f},
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f}},
     {// MaxLOD-1 - MinLOD-1
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f}},
     {// MaxLOD-0 - MinLOD-0
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f,
       0.11f, 0.12f, 0.13f, 0.14f, 0.15f, 0.16f, 0.17f, 0.18f, 0.19f, 0.20f}}},
    { // Joint group 2
     {// MaxLOD-0 - MaxLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f},
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f}},
     {// MaxLOD-1 - MinLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f}},
     {// MaxLOD-0 - MinLOD-0
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f}}},
    { // Joint group 3
     {// MaxLOD-0 - MaxLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f},
      {}},
     {// MaxLOD-1 - MinLOD-1
      {}},
     {// MaxLOD-0 - MinLOD-0
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f}}}};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV23::jointGroupJointIndices = {{ // Joint Group 0
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {0},
                                                                                                   {0}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {0}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {0}}},
                                                                                                 { // Joint Group 1
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {2, 4},
                                                                                                   {2}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {2}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {2, 4}}},
                                                                                                 { // Joint Group 2
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {6, 7},
                                                                                                   {6}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {4}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {6, 7}}},
                                                                                                 { // Joint Group 3
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {5, 7},
                                                                                                   {}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {5, 7}}}};
// Behavior->BlendShapes
const pma::Matrix<std::uint16_t> DecodedV23::blendShapeLODs = {{
    {7, 4},  // MaxLOD-0 - MaxLOD-1
    {4},     // MaxLOD-1 - MinLOD-1
    {7}      // MaxLOD-0 - MinLOD-0
}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::blendShapeInputIndices = {{// MaxLOD-0 - MaxLOD-1
                                                                                     {0, 1, 2, 3, 6, 7, 8},
                                                                                     {0, 1, 2, 3}},
                                                                                    {// MaxLOD-1 - MinLOD-1
                                                                                     {0, 1, 2, 3}},
                                                                                    {// MaxLOD-0 - MinLOD-0
                                                                                     {0, 1, 2, 3, 6, 7, 8}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::blendShapeOutputIndices = {{// MaxLOD-0 - MaxLOD-1
                                                                                      {0, 1, 2, 3, 6, 7, 8},
                                                                                      {0, 1, 2, 3}},
                                                                                     {// MaxLOD-1 - MinLOD-1
                                                                                      {0, 1, 2, 3}},
                                                                                     {// MaxLOD-0 - MinLOD-0
                                                                                      {0, 1, 2, 3, 6, 7, 8}}};
// Behavior->AnimatedMaps
const pma::Vector<std::uint16_t> DecodedV23::animatedMapCount = {
    10,  // MaxLOD-0 - MaxLOD-1
    4,   // MaxLOD-1 - MinLOD-1
    10   // MaxLOD-0 - MinLOD-0
};
const pma::Matrix<std::uint16_t> DecodedV23::animatedMapLODs = {{
    {15, 6},  // MaxLOD-0 - MaxLOD-1
    {6},      // MaxLOD-1 - MinLOD-1
    {15}      // MaxLOD-0 - MinLOD-0
}};
// Geometry
const pma::Vector<std::uint32_t> DecodedV23::meshCount = {
    3u,  // MaxLOD-0 - MaxLOD-1
    1u,  // MaxLOD-1 - MinLOD-1
    2u   // MaxLOD-0 - MinLOD-0
};
const pma::Vector<pma::Matrix<Vector3>> DecodedV23::vertexPositions = {{ // MaxLOD-0 - MaxLOD-1
                                                                        {// Mesh-0
                                                                         {7.0f, 7.0f, 7.0f},
                                                                         {8.0f, 8.0f, 8.0f},
                                                                         {9.0f, 9.0f, 9.0f}},
                                                                        {// Mesh-1
                                                                         {4.0f, 4.0f, 4.0f},
                                                                         {5.0f, 5.0f, 5.0f},
                                                                         {6.0f, 6.0f, 6.0f}},
                                                                        {// Mesh-2
                                                                         {1.0f, 1.0f, 1.0f},
                                                                         {2.0f, 2.0f, 2.0f},
                                                                         {3.0f, 3.0f, 3.0f}}},
                                                                       { // MaxLOD-1 - MinLOD-1
                                                                        {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                         {1.0f, 1.0f, 1.0f},
                                                                         {2.0f, 2.0f, 2.0f},
                                                                         {3.0f, 3.0f, 3.0f}}},
                                                                       { // MaxLOD-0 - MinLOD-0
                                                                        {// Mesh-0
                                                                         {7.0f, 7.0f, 7.0f},
                                                                         {8.0f, 8.0f, 8.0f},
                                                                         {9.0f, 9.0f, 9.0f}},
                                                                        {// Mesh-1
                                                                         {4.0f, 4.0f, 4.0f},
                                                                         {5.0f, 5.0f, 5.0f},
                                                                         {6.0f, 6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<TextureCoordinate>> DecodedV23::vertexTextureCoordinates = {{ // MaxLOD-0 - MaxLOD-1
                                                                                           {// Mesh-0
                                                                                            {7.0f, 7.0f},
                                                                                            {8.0f, 8.0f},
                                                                                            {9.0f, 9.0f}},
                                                                                           {// Mesh-1
                                                                                            {4.0f, 4.0f},
                                                                                            {5.0f, 5.0f},
                                                                                            {6.0f, 6.0f}},
                                                                                           {// Mesh-2
                                                                                            {1.0f, 1.0f},
                                                                                            {2.0f, 2.0f},
                                                                                            {3.0f, 3.0f}}},
                                                                                          { // MaxLOD-1 - MinLOD-1
                                                                                           {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                            {1.0f, 1.0f},
                                                                                            {2.0f, 2.0f},
                                                                                            {3.0f, 3.0f}}},
                                                                                          { // MaxLOD-0 - MinLOD-0
                                                                                           {// Mesh-0
                                                                                            {7.0f, 7.0f},
                                                                                            {8.0f, 8.0f},
                                                                                            {9.0f, 9.0f}},
                                                                                           {// Mesh-1
                                                                                            {4.0f, 4.0f},
                                                                                            {5.0f, 5.0f},
                                                                                            {6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV23::vertexNormals = {{ // MaxLOD-0 - MaxLOD-1
                                                                      {// Mesh-0
                                                                       {7.0f, 7.0f, 7.0f},
                                                                       {8.0f, 8.0f, 8.0f},
                                                                       {9.0f, 9.0f, 9.0f}},
                                                                      {// Mesh-1
                                                                       {4.0f, 4.0f, 4.0f},
                                                                       {5.0f, 5.0f, 5.0f},
                                                                       {6.0f, 6.0f, 6.0f}},
                                                                      {// Mesh-2
                                                                       {1.0f, 1.0f, 1.0f},
                                                                       {2.0f, 2.0f, 2.0f},
                                                                       {3.0f, 3.0f, 3.0f}}},
                                                                     { // MaxLOD-1 - MinLOD-1
                                                                      {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                       {1.0f, 1.0f, 1.0f},
                                                                       {2.0f, 2.0f, 2.0f},
                                                                       {3.0f, 3.0f, 3.0f}}},
                                                                     { // MaxLOD-0 - MinLOD-0
                                                                      {// Mesh-0
                                                                       {7.0f, 7.0f, 7.0f},
                                                                       {8.0f, 8.0f, 8.0f},
                                                                       {9.0f, 9.0f, 9.0f}},
                                                                      {// Mesh-1
                                                                       {4.0f, 4.0f, 4.0f},
                                                                       {5.0f, 5.0f, 5.0f},
                                                                       {6.0f, 6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<VertexLayout>> DecodedV23::vertexLayouts = {{ // MaxLOD-0 - MaxLOD-1
                                                                           {// Mesh-0
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-1
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-2
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}},
                                                                          { // MaxLOD-1 - MinLOD-1
                                                                           {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}},
                                                                          { // MaxLOD-0 - MinLOD-0
                                                                           {// Mesh-0
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-1
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}}};
const pma::Matrix<pma::Matrix<std::uint32_t>> DecodedV23::faces = {{ // MaxLOD-0 - MaxLOD-1
                                                                    {// Mesh-0
                                                                     {0, 1, 2}},
                                                                    {// Mesh-1
                                                                     {0, 1, 2}},
                                                                    {// Mesh-2
                                                                     {0, 1, 2}}},
                                                                   { // MaxLOD-1 - MinLOD-1
                                                                    {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                     {0, 1, 2}}},
                                                                   { // MaxLOD-0 - MinLOD-0
                                                                    {// Mesh-0
                                                                     {0, 1, 2}},
                                                                    {// Mesh-1
                                                                     {0, 1, 2}}}};
const pma::Matrix<std::uint16_t> DecodedV23::maxInfluencePerVertex = {{
                                                                          // MaxLOD-0 - MaxLOD-1
                                                                          8u,  // Mesh-0
                                                                          8u,  // Mesh-1
                                                                          8u   // Mesh-2
                                                                      },
                                                                      {
                                                                          // MaxLOD-1 - MinLOD-1
                                                                          8u  // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                      },
                                                                      {
                                                                          // MaxLOD-0 - MinLOD-0
                                                                          8u,  // Mesh-0
                                                                          8u   // Mesh-1
                                                                      }};
const pma::Matrix<pma::Matrix<float>> DecodedV23::skinWeightsValues = {{ // MaxLOD-0 - MinLOD-1
                                                                        {// Mesh-0
                                                                         {0.7f, 0.1f, 0.2f},
                                                                         {0.5f, 0.5f},
                                                                         {0.4f, 0.6f}},
                                                                        {// Mesh-1
                                                                         {0.4f, 0.3f, 0.3f},
                                                                         {0.8f, 0.2f},
                                                                         {0.1f, 0.9f}},
                                                                        {// Mesh-2
                                                                         {0.1f, 0.3f, 0.6f},
                                                                         {0.3f, 0.7f},
                                                                         {0.2f, 0.8f}}},
                                                                       {// MaxLOD-1 - MinLOD-1
                                                                        {
                                                                            // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                            {0.1f, 0.3f, 0.6f},
                                                                            {1.0f},  // 0.3f normalized to 1.0f
                                                                            {1.0f}   // 0.8f normalized to 1.0f
                                                                        }},
                                                                       { // MaxLOD-0 - MinLOD-0
                                                                        {// Mesh-0
                                                                         {0.7f, 0.1f, 0.2f},
                                                                         {0.5f, 0.5f},
                                                                         {0.4f, 0.6f}},
                                                                        {// Mesh-1
                                                                         {0.4f, 0.3f, 0.3f},
                                                                         {0.8f, 0.2f},
                                                                         {0.1f, 0.9f}}}};
const pma::Matrix<pma::Matrix<std::uint16_t>> DecodedV23::skinWeightsJointIndices = {{ // MaxLOD-0 - MaxLOD-1
                                                                                      {// Mesh-0
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-1
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-2
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}}},
                                                                                     { // MaxLOD-1 - MinLOD-1
                                                                                      {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                       {0, 1, 2},
                                                                                       {3},
                                                                                       {4}}},
                                                                                     { // MaxLOD-0 - MinLOD-0
                                                                                      {// Mesh-0
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-1
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::correctiveBlendShapeIndices = {{ // MaxLOD-0 - MaxLOD-1
                                                                                          {// Mesh-0
                                                                                           2},
                                                                                          {// Mesh-1
                                                                                           2},
                                                                                          {// Mesh-2
                                                                                           2,
                                                                                           3}},
                                                                                         { // MaxLOD-1 - MinLOD-1
                                                                                          {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                           2}},
                                                                                         { // MaxLOD-0 - MinLOD-0
                                                                                          {// Mesh-0
                                                                                           2},
                                                                                          {// Mesh-1
                                                                                           2}}};
const pma::Matrix<pma::Matrix<Vector3>> DecodedV23::correctiveBlendShapeDeltas = {{  // MaxLOD-0 - MaxLOD-1
                                                                                   { // Mesh-0
                                                                                    {// Blendshape-0
                                                                                     {7.0f, 7.0f, 7.0f},
                                                                                     {8.0f, 8.0f, 8.0f},
                                                                                     {9.0f, 9.0f, 9.0f}}},
                                                                                   { // Mesh-1
                                                                                    {// Blendshape-0
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f},
                                                                                     {6.0f, 6.0f, 6.0f}}},
                                                                                   { // Mesh-2
                                                                                    {// Blendshape-0
                                                                                     {1.0f, 1.0f, 1.0f},
                                                                                     {2.0f, 2.0f, 2.0f},
                                                                                     {3.0f, 3.0f, 3.0f}},
                                                                                    {// Blendshape-1
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f}}}},
                                                                                  {  // MaxLOD-1 - MinLOD-1
                                                                                   { // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                    {// Blendshape-0
                                                                                     {1.0f, 1.0f, 1.0f},
                                                                                     {2.0f, 2.0f, 2.0f},
                                                                                     {3.0f, 3.0f, 3.0f}}}},
                                                                                  {  // MaxLOD-0 - MinLOD-0
                                                                                   { // Mesh-0
                                                                                    {// Blendshape-0
                                                                                     {7.0f, 7.0f, 7.0f},
                                                                                     {8.0f, 8.0f, 8.0f},
                                                                                     {9.0f, 9.0f, 9.0f}}},
                                                                                   { // Mesh-1
                                                                                    {// Blendshape-0
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f},
                                                                                     {6.0f, 6.0f, 6.0f}}}}};
const pma::Matrix<pma::Matrix<std::uint32_t>> DecodedV23::correctiveBlendShapeVertexIndices = {
    {// MaxLOD-0 - MaxLOD-1
     {
         // Mesh-0
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-1
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-2
         {0, 1, 2},  // Blendshape-0
         {0, 2}      // Blendshape-1
     }},
    {// MaxLOD-1 - MinLOD-1
     {
         // Mesh-0 (Mesh-2 under MaxLOD-0)
         {0, 1, 2}  // Blendshape-0
     }},
    {// MaxLOD-0 - MinLOD-0
     {
         // Mesh-0
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-1
         {0, 1, 2},  // Blendshape-0
     }}};
// Machine learned behavior
const pma::Vector<pma::String<char>> DecodedV23::mlControlNames = {"MA", "MB", "MC", "MD", "ME", "MF", "MG", "MH", "MI"};
const pma::Matrix<std::uint16_t> DecodedV23::neuralNetworkIndicesPerLOD = {{
                                                                               // MaxLOD-0 - MaxLOD-1
                                                                               0,  // Mesh-0 Region-0
                                                                               1,  // Mesh-0 Region-1
                                                                               2,  // Mesh-1 Region-0
                                                                               3,  // Mesh-1 Region-1
                                                                               4,  // Mesh-2 Region-0
                                                                               5   // Mesh-2 Region-1
                                                                           },
                                                                           {
                                                                               // MaxLOD-1 - MinLOD-1
                                                                               0,  // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                               1   // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
                                                                           },
                                                                           {
                                                                               // MaxLOD-0 - MinLOD-0
                                                                               0,  // Mesh-0 Region-0
                                                                               1,  // Mesh-0 Region-1
                                                                               2,  // Mesh-1 Region-0
                                                                               3   // Mesh-1 Region-1
                                                                           }};
const DecodedV23::VectorOfCharStringMatrix DecodedV23::regionNames = {{ // MaxLOD-0 - MinLOD-1
                                                                       {// Mesh-0
                                                                        "RA",
                                                                        "RB"},
                                                                       {// Mesh-1
                                                                        "RC",
                                                                        "RD"},
                                                                       {// Mesh-2
                                                                        "RE",
                                                                        "RF"}},
                                                                      { // MaxLOD-1 - MinLOD-1
                                                                       {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                        "RE",
                                                                        "RF"}},
                                                                      { // MaxLOD-0 - MinLOD-0
                                                                       {// Mesh-0
                                                                        "RA",
                                                                        "RB"},
                                                                       {// Mesh-1
                                                                        "RC",
                                                                        "RD"}}};
const pma::Matrix<pma::Matrix<std::uint16_t>> DecodedV23::neuralNetworkIndicesPerMeshRegion = {{  // MaxLOD-0 - MaxLOD-1
                                                                                                { // Mesh-0
                                                                                                 {// Region-0
                                                                                                  0},
                                                                                                 {// Region-1
                                                                                                  1}},
                                                                                                { // Mesh-1
                                                                                                 {// Region-0
                                                                                                  2},
                                                                                                 {// Region-1
                                                                                                  3}},
                                                                                                { // Mesh-2
                                                                                                 {// Region-0
                                                                                                  4},
                                                                                                 {// Region-1
                                                                                                  5}}},
                                                                                               { // MaxLOD-1 - MinLOD-1
                                                                                                {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                                 {
                                                                                                     // Region-0
                                                                                                     0  // (4 under MaxLOD-0)
                                                                                                 },
                                                                                                 {
                                                                                                     // Region-1
                                                                                                     1  // (5 under MaxLOD-0)
                                                                                                 }}},
                                                                                               {  // MaxLOD-0 - MinLOD-0
                                                                                                { // Mesh-0
                                                                                                 {// Region-0
                                                                                                  0},
                                                                                                 {// Region-1
                                                                                                  1}},
                                                                                                { // Mesh-1
                                                                                                 {// Region-0
                                                                                                  2},
                                                                                                 {// Region-1
                                                                                                  3}}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::neuralNetworkInputIndices = {
    { // MaxLOD-0 - MaxLOD-1
     {// Mesh-0 Region-0
      0,
      1},
     {// Mesh-0 Region-1
      2,
      3},
     {// Mesh-1 Region-0
      4,
      5},
     {// Mesh-1 Region-1
      6,
      7},
     {// Mesh-2 Region-0
      8,
      0},
     {// Mesh-2 Region-1
      4,
      7}},
    { // MaxLOD-1 - MinLOD-1
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
      8,
      0},
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
      4,
      7}},
    { // MaxLOD-0 - MinLOD-0
     {// Mesh-0 Region-0
      0,
      1},
     {// Mesh-0 Region-1
      2,
      3},
     {// Mesh-1 Region-0
      4,
      5},
     {// Mesh-1 Region-1
      6,
      7}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::neuralNetworkOutputIndices = {
    { // MaxLOD-0 - MaxLOD-1
     {// Mesh-0 Region-0
      9},
     {// Mesh-0 Region-1
      10},
     {// Mesh-1 Region-0
      11},
     {// Mesh-1 Region-1
      12},
     {// Mesh-2 Region-0
      13},
     {// Mesh-2 Region-1
      14}},
    { // MaxLOD-1 - MinLOD-1
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
      13},
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
      14}},
    { // MaxLOD-0 - MinLOD-0
     {// Mesh-0 Region-0
      9},
     {// Mesh-0 Region-1
      10},
     {// Mesh-1 Region-0
      11},
     {// Mesh-1 Region-1
      12}}};
const pma::Matrix<std::uint16_t> DecodedV23::neuralNetworkLayerCount = {{
                                                                            // MaxLOD-0 - MaxLOD-1
                                                                            2,  // Mesh-0 Region-0
                                                                            2,  // Mesh-0 Region-1
                                                                            2,  // Mesh-1 Region-0
                                                                            2,  // Mesh-1 Region-1
                                                                            2,  // Mesh-2 Region-0
                                                                            2   // Mesh-2 Region-1
                                                                        },
                                                                        {
                                                                            // MaxLOD-1 - MinLOD-1
                                                                            2,  // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                            2   // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
                                                                        },
                                                                        {
                                                                            // MaxLOD-0 - MinLOD-0
                                                                            2,  // Mesh-0 Region-0
                                                                            2,  // Mesh-0 Region-1
                                                                            2,  // Mesh-1 Region-0
                                                                            2   // Mesh-1 Region-1
                                                                        }};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV23::neuralNetworkActivationFunction = {
    { // MaxLOD-0 - MaxLOD-1
     {// Mesh-0 Region-0
      1,
      1},
     {// Mesh-0 Region-1
      1,
      1},
     {// Mesh-1  Region-0
      1,
      1},
     {// Mesh-1 Region-1
      1,
      1},
     {// Mesh-2 Region-0
      1,
      1},
     {// Mesh-2 Region-1
      1,
      1}},
    { // MaxLOD-1 - MinLOD-1
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
      1,
      1},
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
      1,
      1}},
    { // MaxLOD-0 - MinLOD-0
     {// Mesh-0  Region-0
      1,
      1},
     {// Mesh-0 Region-1
      1,
      1},
     {// Mesh-1 Region-0
      1,
      1},
     {// Mesh-1 Region-1
      1,
      1}}};
const pma::Matrix<pma::Matrix<float>> DecodedV23::neuralNetworkActivationFunctionParameters = {
    {// MaxLOD-0 - MaxLOD-1
     {
         // Mesh-0  Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-0 Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-0
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-1
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-2 Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-2 Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     }},
    {// MaxLOD-1 - MinLOD-1
     {
         // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     }},
    {// MaxLOD-0 - MinLOD-0
     {
         // Mesh-0 Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-0 Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-0
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-1
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     }}};
const pma::Matrix<pma::Matrix<float>> DecodedV23::neuralNetworkBiases = {{// MaxLOD-0 - MaxLOD-1
                                                                          {
                                                                              // Mesh-0 Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-0 Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-0
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-1
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-2 Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-2 Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          }},
                                                                         {// MaxLOD-1 - MinLOD-1
                                                                          {
                                                                              // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          }},
                                                                         {// MaxLOD-0 - MinLOD-0
                                                                          {
                                                                              // Mesh-0 Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-0 Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-0
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-1
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          }}};
const pma::Matrix<pma::Matrix<float>> DecodedV23::neuralNetworkWeights = {{// MaxLOD-0 - MaxLOD-1
                                                                           {
                                                                               // Mesh-0 Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-0 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-0
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-1
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-2 Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-2 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           }},
                                                                          {// MaxLOD-1 - MinLOD-1
                                                                           {
                                                                               // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-0 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           }},
                                                                          {// MaxLOD-0 - MinLOD-0
                                                                           {
                                                                               // Mesh-0 Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-0 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-0
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-1
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           }}};
#ifdef __clang__
    #pragma clang diagnostic pop
#endif

std::size_t DecodedV23::lodConstraintToIndex(std::uint16_t maxLOD, std::uint16_t minLOD) {
    // Relies on having only TWO available LODs (0, 1)
    return (minLOD == 1u ? maxLOD : 2ul);
}

RawJoints DecodedV23::getJoints(std::uint16_t currentMaxLOD, std::uint16_t currentMinLOD, pma::MemoryResource* memRes) {
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    RawJoints joints{memRes};
    joints.rowCount = jointRowCount[srcIndex];
    joints.colCount = jointColumnCount;
    for (std::size_t i = 0ul; i < jointGroupLODs.size(); ++i) {
        RawJointGroup jntGrp{memRes};
        jntGrp.lods.assign(jointGroupLODs[i][srcIndex].begin(), jointGroupLODs[i][srcIndex].end());
        jntGrp.inputIndices.assign(jointGroupInputIndices[i][srcIndex][0ul].begin(),
                                   jointGroupInputIndices[i][srcIndex][0ul].end());
        jntGrp.outputIndices.assign(jointGroupOutputIndices[i][srcIndex][0ul].begin(),
                                    jointGroupOutputIndices[i][srcIndex][0ul].end());
        jntGrp.values.assign(jointGroupValues[i][srcIndex][0ul].begin(), jointGroupValues[i][srcIndex][0ul].end());
        jntGrp.jointIndices.assign(jointGroupJointIndices[i][srcIndex][0ul].begin(),
                                   jointGroupJointIndices[i][srcIndex][0ul].end());
        joints.jointGroups.push_back(std::move(jntGrp));
    }

    return joints;
}

RawBlendShapeChannels DecodedV23::getBlendShapes(std::uint16_t currentMaxLOD,
                                                 std::uint16_t currentMinLOD,
                                                 pma::MemoryResource* memRes) {
    RawBlendShapeChannels blendShapes{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    blendShapes.lods.assign(blendShapeLODs[srcIndex].begin(), blendShapeLODs[srcIndex].end());
    blendShapes.inputIndices.assign(blendShapeInputIndices[srcIndex][0ul].begin(), blendShapeInputIndices[srcIndex][0ul].end());
    blendShapes.outputIndices.assign(blendShapeOutputIndices[srcIndex][0ul].begin(),
                                     blendShapeOutputIndices[srcIndex][0ul].end());
    return blendShapes;
}

RawConditionalTable DecodedV23::getConditionals(std::uint16_t currentMaxLOD,
                                                std::uint16_t currentMinLOD,
                                                pma::MemoryResource* memRes) {
    RawConditionalTable conditionals{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    conditionals.inputIndices.assign(conditionalInputIndices[srcIndex][0ul].begin(),
                                     conditionalInputIndices[srcIndex][0ul].end());
    conditionals.outputIndices.assign(conditionalOutputIndices[srcIndex][0ul].begin(),
                                      conditionalOutputIndices[srcIndex][0ul].end());
    conditionals.fromValues.assign(conditionalFromValues[srcIndex][0ul].begin(), conditionalFromValues[srcIndex][0ul].end());
    conditionals.toValues.assign(conditionalToValues[srcIndex][0ul].begin(), conditionalToValues[srcIndex][0ul].end());
    conditionals.slopeValues.assign(conditionalSlopeValues[srcIndex][0ul].begin(), conditionalSlopeValues[srcIndex][0ul].end());
    conditionals.cutValues.assign(conditionalCutValues[srcIndex][0ul].begin(), conditionalCutValues[srcIndex][0ul].end());
    return conditionals;
}

RawAnimatedMaps DecodedV23::getAnimatedMaps(std::uint16_t currentMaxLOD,
                                            std::uint16_t currentMinLOD,
                                            pma::MemoryResource* memRes) {
    RawAnimatedMaps animatedMaps{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    animatedMaps.lods.assign(animatedMapLODs[srcIndex].begin(), animatedMapLODs[srcIndex].end());
    animatedMaps.conditionals = getConditionals(currentMaxLOD, currentMinLOD, memRes);
    return animatedMaps;
}

}  // namespace dna
