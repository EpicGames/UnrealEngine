// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/Fixturesv25.h"

#include "dna/Reader.h"
#include "pma/TypeDefs.h"

#pragma warning(disable : 4503)

namespace dna {

const unsigned char RawV25::header[] = {
    0x44,
    0x4e,
    0x41,  // DNA signature
    0x00,
    0x02,  // Generation
    0x00,
    0x05,  // Version
    // Index Table
    0x00,
    0x00,
    0x00,
    0x09,  // Index table entry count
    0x64,
    0x65,
    0x73,
    0x63,  // Descriptor id
    0x00,
    0x01,
    0x00,
    0x01,  // Descriptor version
    0x00,
    0x00,
    0x00,
    0x9b,  // Descriptor offset
    0x00,
    0x00,
    0x00,
    0x57,  // Descriptor size
    0x64,
    0x65,
    0x66,
    0x6e,  // Definition id
    0x00,
    0x01,
    0x00,
    0x01,  // Definition version
    0x00,
    0x00,
    0x00,
    0xf2,  // Definition offset
    0x00,
    0x00,
    0x03,
    0x1a,  // Definition size
    0x62,
    0x68,
    0x76,
    0x72,  // Behavior id
    0x00,
    0x01,
    0x00,
    0x01,  // Behavior version
    0x00,
    0x00,
    0x04,
    0x0c,  // Behavior offset
    0x00,
    0x00,
    0x05,
    0x46,  // Behavior size
    0x67,
    0x65,
    0x6f,
    0x6d,  // Geometry id
    0x00,
    0x01,
    0x00,
    0x01,  // Geometry version
    0x00,
    0x00,
    0x09,
    0x52,  // Geometry offset
    0x00,
    0x00,
    0x04,
    0x38,  // Geometry size
    0x6d,
    0x6c,
    0x62,
    0x68,  // Machine learned behavior id
    0x00,
    0x01,
    0x00,
    0x00,  // Machine learned behavior version
    0x00,
    0x00,
    0x0d,
    0x8a,  // Machine learned behavior offset
    0x00,
    0x00,
    0x02,
    0xfa,  // Machine learned behavior size
    0x72,
    0x62,
    0x66,
    0x62,  // RBF behavior id
    0x00,
    0x01,
    0x00,
    0x00,  // RBF behavior version
    0x00,
    0x00,
    0x10,
    0x84,  // RBF behavior offset
    0x00,
    0x00,
    0x01,
    0x47,  // RBF behavior size
    0x72,
    0x62,
    0x66,
    0x65,  // RBF behavior ext id
    0x00,
    0x01,
    0x00,
    0x00,  // RBF behavior ext version
    0x00,
    0x00,
    0x11,
    0xcb,  // RBF behavior ext offset
    0x00,
    0x00,
    0x00,
    0xe4,  // RBF behavior ext size
    0x6a,
    0x62,
    0x6d,
    0x64,  // Joint behavior metadata id
    0x00,
    0x01,
    0x00,
    0x00,  // Joint behavior metadata version
    0x00,
    0x00,
    0x12,
    0xaf,  // Joint behavior metadata offset
    0x00,
    0x00,
    0x00,
    0x3a,  // Joint behavior metadata size
    0x74,
    0x77,
    0x73,
    0x77,  // Twist swing setups id
    0x00,
    0x01,
    0x00,
    0x00,  // Twist swing setups version
    0x00,
    0x00,
    0x12,
    0xe9,  // Twist swing setups offset
    0x00,
    0x00,
    0x00,
    0xc8  // Twist swing setups size
};

const unsigned char RawV25::descriptor[] = {
    0x00, 0x00, 0x00, 0x04,                    // Name length
    0x74, 0x65, 0x73, 0x74,                    // Name
    0x00, 0x05,                                // Archetype
    0x00, 0x02,                                // Gender
    0x00, 0x2a,                                // Age
    0x00, 0x00, 0x00, 0x02,                    // Metadata count
    0x00, 0x00, 0x00, 0x05,                    // Metadata key length
    0x6b, 0x65, 0x79, 0x2d, 0x41,              // Metadata key: "key-A"
    0x00, 0x00, 0x00, 0x07,                    // Metadata value length
    0x76, 0x61, 0x6c, 0x75, 0x65, 0x2d, 0x41,  // Metadata value: "value-A"
    0x00, 0x00, 0x00, 0x05,                    // Metadata key length
    0x6b, 0x65, 0x79, 0x2d, 0x42,              // Metadata key: "key-B"
    0x00, 0x00, 0x00, 0x07,                    // Metadata value length
    0x76, 0x61, 0x6c, 0x75, 0x65, 0x2d, 0x42,  // Metadata value: "value-B"
    0x00, 0x01,                                // Unit translation
    0x00, 0x01,                                // Unit rotation
    0x00, 0x01,                                // Coordinate system x-axis
    0x00, 0x02,                                // Coordinate system y-axis
    0x00, 0x04,                                // Coordinate system z-axis
    0x00, 0x02,                                // LOD Count
    0x00, 0x00,                                // MaxLOD: 0
    0x00, 0x00, 0x00, 0x01,                    // Complexity name length
    0x41,                                      // 'A' - Complexity name
    0x00, 0x00, 0x00, 0x06,                    // DB name length
    0x74, 0x65, 0x73, 0x74, 0x44, 0x42         // Name
};

const unsigned char RawV25::definition[] = {
    0x00, 0x00, 0x00, 0x02,  // Joint name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Joint name indices per LOD row count
    0x00, 0x00, 0x00, 0x09,  // Indices matrix row-0
    0x00, 0x00,              // Joint name index: 0
    0x00, 0x01,              // Joint name index: 1
    0x00, 0x02,              // Joint name index: 2
    0x00, 0x03,              // Joint name index: 3
    0x00, 0x04,              // Joint name index: 4
    0x00, 0x05,              // Joint name index: 5
    0x00, 0x06,              // Joint name index: 6
    0x00, 0x07,              // Joint name index: 7
    0x00, 0x08,              // Joint name index: 8
    0x00, 0x00, 0x00, 0x06,  // Indices matrix row-1
    0x00, 0x00,              // Joint name index: 0
    0x00, 0x01,              // Joint name index: 1
    0x00, 0x02,              // Joint name index: 2
    0x00, 0x03,              // Joint name index: 3
    0x00, 0x06,              // Joint name index: 6
    0x00, 0x08,              // Joint name index: 8
    0x00, 0x00, 0x00, 0x02,  // Blend shape name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Blend shape name indices per LOD row count
    0x00, 0x00, 0x00, 0x09,  // Indices matrix row-0
    0x00, 0x00,              // Blend shape name index: 0
    0x00, 0x01,              // Blend shape name index: 1
    0x00, 0x02,              // Blend shape name index: 2
    0x00, 0x03,              // Blend shape name index: 3
    0x00, 0x04,              // Blend shape name index: 4
    0x00, 0x05,              // Blend shape name index: 5
    0x00, 0x06,              // Blend shape name index: 6
    0x00, 0x07,              // Blend shape name index: 7
    0x00, 0x08,              // Blend shape name index: 8
    0x00, 0x00, 0x00, 0x04,  // Indices matrix row-1
    0x00, 0x02,              // Blend shape name index: 2
    0x00, 0x05,              // Blend shape name index: 5
    0x00, 0x07,              // Blend shape name index: 7
    0x00, 0x08,              // Blend shape name index: 8
    0x00, 0x00, 0x00, 0x02,  // Animated map name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Animated map name indices per LOD row count
    0x00, 0x00, 0x00, 0x0a,  // Indices matrix row-0
    0x00, 0x00,              // Animated map name index: 0
    0x00, 0x01,              // Animated map name index: 1
    0x00, 0x02,              // Animated map name index: 2
    0x00, 0x03,              // Animated map name index: 3
    0x00, 0x04,              // Animated map name index: 4
    0x00, 0x05,              // Animated map name index: 5
    0x00, 0x06,              // Animated map name index: 6
    0x00, 0x07,              // Animated map name index: 7
    0x00, 0x08,              // Animated map name index: 8
    0x00, 0x09,              // Animated map name index: 9
    0x00, 0x00, 0x00, 0x04,  // Indices matrix row-1
    0x00, 0x02,              // Animated map name index: 2
    0x00, 0x05,              // Animated map name index: 5
    0x00, 0x07,              // Animated map name index: 7
    0x00, 0x08,              // Animated map name index: 8
    0x00, 0x00, 0x00, 0x02,  // Mesh name indices lod to row mapping length
    0x00, 0x00,              // Map from LOD-0 to row 0 in below defined matrix
    0x00, 0x01,              // Map from LOD-1 to row 1 in below defined matrix
    0x00, 0x00, 0x00, 0x02,  // Mesh name indices per LOD row count
    0x00, 0x00, 0x00, 0x02,  // Indices matrix row-0
    0x00, 0x00,              // Mesh name index: 0
    0x00, 0x01,              // Mesh name index: 1
    0x00, 0x00, 0x00, 0x01,  // Indices matrix row-1
    0x00, 0x02,              // Mesh name index: 2
    0x00, 0x00, 0x00, 0x09,  // Gui control names length
    0x00, 0x00, 0x00, 0x02,  // Gui control name 0 length
    0x47, 0x41,              // Gui control name 0 : GA
    0x00, 0x00, 0x00, 0x02,  // Gui control name 1 length
    0x47, 0x42,              // Gui control name 1 : GB
    0x00, 0x00, 0x00, 0x02,  // Gui control name 2 length
    0x47, 0x43,              // Gui control name 2 : GC
    0x00, 0x00, 0x00, 0x02,  // Gui control name 3 length
    0x47, 0x44,              // Gui control name 3 : GD
    0x00, 0x00, 0x00, 0x02,  // Gui control name 4 length
    0x47, 0x45,              // Gui control name 4 : GE
    0x00, 0x00, 0x00, 0x02,  // Gui control name 5 length
    0x47, 0x46,              // Gui control name 5 : GF
    0x00, 0x00, 0x00, 0x02,  // Gui control name 6 length
    0x47, 0x47,              // Gui control name 6 : GG
    0x00, 0x00, 0x00, 0x02,  // Gui control name 7 length
    0x47, 0x48,              // Gui control name 7 : GH
    0x00, 0x00, 0x00, 0x02,  // Gui control name 8 length
    0x47, 0x49,              // Gui control name 8 : GI
    0x00, 0x00, 0x00, 0x09,  // Raw control names length
    0x00, 0x00, 0x00, 0x02,  // Raw control name 0 length
    0x52, 0x41,              // Raw control name 0 : RA
    0x00, 0x00, 0x00, 0x02,  // Raw control name 1 length
    0x52, 0x42,              // Raw control name 1 : RB
    0x00, 0x00, 0x00, 0x02,  // Raw control name 2 length
    0x52, 0x43,              // Raw control name 2 : RC
    0x00, 0x00, 0x00, 0x02,  // Raw control name 3 length
    0x52, 0x44,              // Raw control name 3 : RD
    0x00, 0x00, 0x00, 0x02,  // Raw control name 4 length
    0x52, 0x45,              // Raw control name 4 : RE
    0x00, 0x00, 0x00, 0x02,  // Raw control name 5 length
    0x52, 0x46,              // Raw control name 5 : RF
    0x00, 0x00, 0x00, 0x02,  // Raw control name 6 length
    0x52, 0x47,              // Raw control name 6 : RG
    0x00, 0x00, 0x00, 0x02,  // Raw control name 7 length
    0x52, 0x48,              // Raw control name 7 : RH
    0x00, 0x00, 0x00, 0x02,  // Raw control name 8 length
    0x52, 0x49,              // Raw control name 8 : RI
    0x00, 0x00, 0x00, 0x09,  // Joint names length
    0x00, 0x00, 0x00, 0x02,  // Joint name 0 length
    0x4a, 0x41,              // Joint name 0 : JA
    0x00, 0x00, 0x00, 0x02,  // Joint name 1 length
    0x4a, 0x42,              // Joint name 1 : JB
    0x00, 0x00, 0x00, 0x02,  // Joint name 2 length
    0x4a, 0x43,              // Joint name 2 : JC
    0x00, 0x00, 0x00, 0x02,  // Joint name 3 length
    0x4a, 0x44,              // Joint name 3 : JD
    0x00, 0x00, 0x00, 0x02,  // Joint name 4 length
    0x4a, 0x45,              // Joint name 4 : JE
    0x00, 0x00, 0x00, 0x02,  // Joint name 5 length
    0x4a, 0x46,              // Joint name 5 : JF
    0x00, 0x00, 0x00, 0x02,  // Joint name 6 length
    0x4a, 0x47,              // Joint name 6 : JG
    0x00, 0x00, 0x00, 0x02,  // Joint name 7 length
    0x4a, 0x48,              // Joint name 7 : JH
    0x00, 0x00, 0x00, 0x02,  // Joint name 8 length
    0x4a, 0x49,              // Joint name 8 : JI
    0x00, 0x00, 0x00, 0x09,  // BlendShape names length
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 0 length
    0x42, 0x41,              // Blendshape name 0 : BA
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 1 length
    0x42, 0x42,              // Blendshape name 1 : BB
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 2 length
    0x42, 0x43,              // Blendshape name 2 : BC
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 3 length
    0x42, 0x44,              // Blendshape name 3 : BD
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 4 length
    0x42, 0x45,              // Blendshape name 4 : BE
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 5 length
    0x42, 0x46,              // Blendshape name 5 : BF
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 6 length
    0x42, 0x47,              // Blendshape name 6 : BG
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 7 length
    0x42, 0x48,              // Blendshape name 7 : BH
    0x00, 0x00, 0x00, 0x02,  // Blendshape name 8 length
    0x42, 0x49,              // Blendshape name 8 : BI
    0x00, 0x00, 0x00, 0x0a,  // Animated Map names length
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 0 length
    0x41, 0x41,              // Animated Map name 0 : AA
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 1 length
    0x41, 0x42,              // Animated Map name 1 : AB
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 2 length
    0x41, 0x43,              // Animated Map name 2 : AC
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 3 length
    0x41, 0x44,              // Animated Map name 3 : AD
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 4 length
    0x41, 0x45,              // Animated Map name 4 : AE
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 5 length
    0x41, 0x46,              // Animated Map name 5 : AF
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 6 length
    0x41, 0x47,              // Animated Map name 6 : AG
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 7 length
    0x41, 0x48,              // Animated Map name 7 : AH
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 8 length
    0x41, 0x49,              // Animated Map name 8 : AI
    0x00, 0x00, 0x00, 0x02,  // Animated Map name 9 length
    0x41, 0x4a,              // Animated Map name 8 : AJ
    0x00, 0x00, 0x00, 0x03,  // Mesh names length
    0x00, 0x00, 0x00, 0x02,  // Mesh name 0 length
    0x4d, 0x41,              // Mesh name 0 : MA
    0x00, 0x00, 0x00, 0x02,  // Mesh name 1 length
    0x4d, 0x42,              // Mesh name 1 : MB
    0x00, 0x00, 0x00, 0x02,  // Mesh name 2 length
    0x4d, 0x43,              // Mesh name 2 : MC
    0x00, 0x00, 0x00, 0x09,  // Mesh indices length for mesh -> blendShape mapping
    0x00, 0x00,              // Mesh index 0
    0x00, 0x00,              // Mesh index 0
    0x00, 0x00,              // Mesh index 0
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x01,              // Mesh index 1
    0x00, 0x02,              // Mesh index 2
    0x00, 0x02,              // Mesh index 2
    0x00, 0x00, 0x00, 0x09,  // BlendShape indices length for mesh -> blendShape mapping
    0x00, 0x00,              // BlendShape 0
    0x00, 0x01,              // BlendShape 1
    0x00, 0x02,              // BlendShape 2
    0x00, 0x03,              // BlendShape 3
    0x00, 0x04,              // BlendShape 4
    0x00, 0x05,              // BlendShape 5
    0x00, 0x06,              // BlendShape 6
    0x00, 0x07,              // BlendShape 7
    0x00, 0x08,              // BlendShape 8
    0x00, 0x00, 0x00, 0x09,  // Joint hierarchy length
    0x00, 0x00,              // JA - root
    0x00, 0x00,              // JB
    0x00, 0x00,              // JC
    0x00, 0x01,              // JD
    0x00, 0x01,              // JE
    0x00, 0x04,              // JF
    0x00, 0x02,              // JG
    0x00, 0x04,              // JH
    0x00, 0x02,              // JI
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation X values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation Y values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint translation Z values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation X values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation Y values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00,  // 9.0f
    0x00, 0x00, 0x00, 0x09,  // Neutral joint rotation Z values length
    0x3f, 0x80, 0x00, 0x00,  // 1.0f
    0x40, 0x00, 0x00, 0x00,  // 2.0f
    0x40, 0x40, 0x00, 0x00,  // 3.0f
    0x40, 0x80, 0x00, 0x00,  // 4.0f
    0x40, 0xa0, 0x00, 0x00,  // 5.0f
    0x40, 0xc0, 0x00, 0x00,  // 6.0f
    0x40, 0xe0, 0x00, 0x00,  // 7.0f
    0x41, 0x00, 0x00, 0x00,  // 8.0f
    0x41, 0x10, 0x00, 0x00   // 9.0f
};

const unsigned char RawV25::conditionals[]{
    // Input indices
    0x00,
    0x00,
    0x00,
    0x0f,  // Input indices count
    0x00,
    0x00,  // Index: 0      C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x02,  // Index: 2  C0      L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x05,  // Index: 5      C1  L0
    0x00,
    0x06,  // Index: 6      C1  L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x08,  // Index: 8  C0  C1  L0
    0x00,
    0x08,  // Index: 8      C1  L0
    // Output indices
    0x00,
    0x00,
    0x00,
    0x0f,  // Output indices count
    0x00,
    0x00,  // Index: 0      C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x01,  // Index: 1  C0  C1  L0  L1
    0x00,
    0x02,  // Index: 2  C0      L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x03,  // Index: 3      C1  L0  L1
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x04,  // Index: 4  C0      L0
    0x00,
    0x05,  // Index: 5      C1  L0
    0x00,
    0x06,  // Index: 6      C1  L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x07,  // Index: 7  C0      L0
    0x00,
    0x08,  // Index: 8  C0  C1  L0
    0x00,
    0x08,  // Index: 8      C1  L0
    // From values
    0x00,
    0x00,
    0x00,
    0x0f,  // From values count
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0  L1
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f  C0  C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0  C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0  L1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f      C1  L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f  C0      L0
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f      C1  L0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f  C0  C1  L0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0
    // To values
    0x00,
    0x00,
    0x00,
    0x0f,  // To values count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0  C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0  C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0      L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0  C1  L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0
    // Slope values
    0x00,
    0x00,
    0x00,
    0x0f,  // Slope values count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0  C1  L0  L1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C0  C1  L0  L1
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C1  L0  L1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C0      L0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f      C1  L0
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f      C1  L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0  C1  L0
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f      C1  L0
    // Cut values
    0x00,
    0x00,
    0x00,
    0x0f,  // Cut values count
    0x00,
    0x00,
    0x00,
    0x00,  // 0.0f      C1  L0  L1
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f  C0  C1  L0  L1
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f  C0  C1  L0  L1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f  C0      L0  L1
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f      C1  L0  L1
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f      C1  L0  L1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0      L0
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f      C1  L0
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f      C1  L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C0      L0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C0  C1  L0
    0x3e,
    0x4c,
    0xcc,
    0xcd  // 0.2f       C1  L0
};

const unsigned char RawV25::psds[]{
    // Rows
    0x00,
    0x00,
    0x00,
    0x18,  // Row index count
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x08,  // Index:  8  C1
    0x00,
    0x09,  // Index:  9      C2
    0x00,
    0x09,  // Index:  9      C2
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0a,  // Index: 10  C1
    0x00,
    0x0b,  // Index: 11      C2
    0x00,
    0x0c,  // Index: 12      C2
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0d,  // Index: 13  C1
    0x00,
    0x0e,  // Index: 14  C1
    0x00,
    0x0e,  // Index: 14  C1
    0x00,
    0x0f,  // Index: 15  C1
    0x00,
    0x10,  // Index: 16      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x12,  // Index: 18      C2
    0x00,
    0x13,  // Index: 19  C1
    0x00,
    0x13,  // Index: 19  C1
    0x00,
    0x14,  // Index: 20  C1
    // Columns
    0x00,
    0x00,
    0x00,
    0x18,  // Column index count
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x05,  // Index: 5      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x07,  // Index: 7  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x01,  // Index: 1  C1  C2
    0x00,
    0x02,  // Index: 2  C1
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x04,  // Index: 4  C1
    0x00,
    0x00,  // Index: 0      C2
    0x00,
    0x03,  // Index: 3      C2
    0x00,
    0x04,  // Index: 4  C1
    0x00,
    0x05,  // Index: 5      C2
    0x00,
    0x06,  // Index: 6      C2
    0x00,
    0x07,  // Index: 7  C1
    0x00,
    0x02,  // Index: 2  C1
    // Values
    0x00,
    0x00,
    0x00,
    0x18,  // Value count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f  C1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f  C1
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f  C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f      C2
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f      C2
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f      C2
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f  C1
    0x3f,
    0x80,
    0x00,
    0x00  // 1.0f  C1
};

const unsigned char RawV25::controls[] = {
    0x00,
    0x0c  // PSD count
};

const unsigned char RawV25::joints[] = {
    0x00,
    0x51,  // Rows = 81
    0x00,
    0x0a,  // Columns = 10
    // Joint groups
    0x00,
    0x00,
    0x00,
    0x04,  // Joint group count
    // Joint group-0
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x03,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x07,  // Input indices count
    0x00,
    0x00,  // Index: 0      C1
    0x00,
    0x01,  // Index: 1  C0  C1
    0x00,
    0x02,  // Index: 2  C0
    0x00,
    0x03,  // Index: 3      C1
    0x00,
    0x06,  // Index: 6      C1
    0x00,
    0x07,  // Index: 7  C0
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x02,  // Index: 2
    0x00,
    0x03,  // Index: 3
    0x00,
    0x05,  // Index: 5
    0x00,
    0x00,
    0x00,
    0x15,  // Float value count: 21
    // Row 0
    0x00,
    0x00,
    0x00,
    0x00,  // 0.00f      C1
    0x3d,
    0x4c,
    0xcc,
    0xcd,  // 0.05f  C0  C1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.10f  C0
    0x3e,
    0x19,
    0x99,
    0x9a,  // 0.15f      C1
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.20f      C1
    0x3e,
    0x80,
    0x00,
    0x00,  // 0.25f  C0
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.30f  C0  C1
    // Row 1
    0x3e,
    0xb3,
    0x33,
    0x33,  // 0.35f      C1
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.40f  C0  C1
    0x3e,
    0xe6,
    0x66,
    0x66,  // 0.45f  C0
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.50f      C1
    0x3f,
    0x0c,
    0xcc,
    0xcd,  // 0.55f      C1
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.60f  C0
    0x3f,
    0x26,
    0x66,
    0x66,  // 0.65f  C0  C1
    // Row 2
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.70f      C1
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0  C1
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f  C0
    0x3f,
    0x59,
    0x99,
    0x9a,  // 0.85f      C1
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.90f      C1
    0x3f,
    0x73,
    0x33,
    0x33,  // 0.95f  C0
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.00f  C0  C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x01,  // Joint index count: 1
    0x00,
    0x00,  // Index: 0
    // Joint group-1
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x04,  // LOD-0 row-count
    0x00,
    0x02,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x05,  // Input indices count
    0x00,
    0x03,  // Index: 3      C1
    0x00,
    0x04,  // Index: 4  C0
    0x00,
    0x07,  // Index: 7  C0
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x09,  // Index: 9      C1
    0x00,
    0x00,
    0x00,
    0x04,  // Output indices count
    0x00,
    0x12,  // Index: 18
    0x00,
    0x14,  // Index: 20
    0x00,
    0x24,  // Index: 36
    0x00,
    0x26,  // Index: 38
    0x00,
    0x00,
    0x00,
    0x14,  // Float value count: 20
    // Row 0
    0x3c,
    0x23,
    0xd7,
    0x0a,  // 0.01f      C1
    0x3c,
    0xa3,
    0xd7,
    0x0a,  // 0.02f  C0
    0x3c,
    0xf5,
    0xc2,
    0x8f,  // 0.03f  C0
    0x3d,
    0x23,
    0xd7,
    0x0a,  // 0.04f  C0  C1
    0x3d,
    0x4c,
    0xcc,
    0xcd,  // 0.05f      C1
    // Row 1
    0x3d,
    0x75,
    0xc2,
    0x8f,  // 0.06f      C1
    0x3d,
    0x8f,
    0x5c,
    0x29,  // 0.07f  C0
    0x3d,
    0xa3,
    0xd7,
    0x0a,  // 0.08f  C0
    0x3d,
    0xb8,
    0x51,
    0xec,  // 0.09f  C0  C1
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.10f      C1
    // Row 2
    0x3d,
    0xe1,
    0x47,
    0xae,  // 0.11f      C1
    0x3d,
    0xf5,
    0xc2,
    0x8f,  // 0.12f  C0
    0x3e,
    0x05,
    0x1e,
    0xb8,  // 0.13f  C0
    0x3e,
    0x0f,
    0x5c,
    0x29,  // 0.14f  C0  C1
    0x3e,
    0x19,
    0x99,
    0x9a,  // 0.15f      C1
    // Row 3
    0x3e,
    0x23,
    0xd7,
    0x0a,  // 0.16f      C1
    0x3e,
    0x2e,
    0x14,
    0x7b,  // 0.17f  C0
    0x3e,
    0x38,
    0x51,
    0xec,  // 0.18f  C0
    0x3e,
    0x42,
    0x8f,
    0x5c,  // 0.19f  C0  C1
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.20f      C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x02,  // Index: 2
    0x00,
    0x04,  // Index: 4
    // Joint group-2
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x02,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x04,  // Input indices count
    0x00,
    0x04,  // Index: 4  C0
    0x00,
    0x05,  // Index: 5      C1
    0x00,
    0x08,  // Index: 8  C0  C1
    0x00,
    0x09,  // Index: 9      C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x37,  // Index: 55
    0x00,
    0x38,  // Index: 56
    0x00,
    0x3f,  // Index: 63
    0x00,
    0x00,
    0x00,
    0x0c,  // Float value count: 12
    // Row 0
    0x3e,
    0x9e,
    0xb8,
    0x52,  // 0.31f  C0
    0x3e,
    0xb8,
    0x51,
    0xec,  // 0.36f      C1
    0x3e,
    0xd7,
    0x0a,
    0x3d,  // 0.42f  C0  C1
    0x3e,
    0xf0,
    0xa3,
    0xd7,  // 0.47f      C1
    // Row 1
    0x3f,
    0x07,
    0xae,
    0x14,  // 0.53f  C0
    0x3f,
    0x14,
    0x7a,
    0xe1,  // 0.58f      C1
    0x3f,
    0x23,
    0xd7,
    0x0a,  // 0.64f  C0  C1
    0x3f,
    0x30,
    0xa3,
    0xd7,  // 0.69f      C1
    // Row 2
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f      C1
    0x3f,
    0x5c,
    0x28,
    0xf6,  // 0.86f  C0  C1
    0x3f,
    0x68,
    0xf5,
    0xc3,  // 0.91f       C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x06,  // Index: 6
    0x00,
    0x07,  // Index: 7
    // Joint group-3
    0x00,
    0x00,
    0x00,
    0x02,  // LOD count
    0x00,
    0x03,  // LOD-0 row-count
    0x00,
    0x00,  // LOD-1 row-count
    0x00,
    0x00,
    0x00,
    0x04,  // Input indices count
    0x00,
    0x02,  // Index: 2  C0
    0x00,
    0x05,  // Index: 5      C1
    0x00,
    0x06,  // Index: 6  C0  C1
    0x00,
    0x08,  // Index: 8      C1
    0x00,
    0x00,
    0x00,
    0x03,  // Output indices count
    0x00,
    0x2d,  // Index: 45
    0x00,
    0x2e,  // Index: 46
    0x00,
    0x47,  // Index: 71
    0x00,
    0x00,
    0x00,
    0x0c,  // Float value count: 12
    // Row 0
    0x3e,
    0x9e,
    0xb8,
    0x52,  // 0.31f  C0
    0x3e,
    0xb8,
    0x51,
    0xec,  // 0.36f      C1
    0x3e,
    0xd7,
    0x0a,
    0x3d,  // 0.42f  C0  C1
    0x3e,
    0xf0,
    0xa3,
    0xd7,  // 0.47f      C1
    // Row 1
    0x3f,
    0x07,
    0xae,
    0x14,  // 0.53f  C0
    0x3f,
    0x14,
    0x7a,
    0xe1,  // 0.58f      C1
    0x3f,
    0x23,
    0xd7,
    0x0a,  // 0.64f  C0  C1
    0x3f,
    0x30,
    0xa3,
    0xd7,  // 0.69f      C1
    // Row 2
    0x3f,
    0x40,
    0x00,
    0x00,  // 0.75f  C0
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.80f      C1
    0x3f,
    0x5c,
    0x28,
    0xf6,  // 0.86f  C0  C1
    0x3f,
    0x68,
    0xf5,
    0xc3,  // 0.91f       C1
    // Joint indices
    0x00,
    0x00,
    0x00,
    0x02,  // Joint index count: 2
    0x00,
    0x05,  // Index: 5
    0x00,
    0x07  // Index: 7
};

const unsigned char RawV25::blendshapes[] = {
    0x00, 0x00, 0x00, 0x02,  // LOD count
    0x00, 0x07,              // LOD-0 row-count
    0x00, 0x04,              // LOD-1 row-count
    0x00, 0x00, 0x00, 0x07,  // Input indices count
    0x00, 0x00,              // Index: 0      C1  L0  L1
    0x00, 0x01,              // Index: 1  C0  C1  L0  L1
    0x00, 0x02,              // Index: 2  C0      L0  L1
    0x00, 0x03,              // Index: 3      C1  L0  L1
    0x00, 0x06,              // Index: 6      C1  L0
    0x00, 0x07,              // Index: 7  C0      L0
    0x00, 0x08,              // Index: 8  C0  C1  L0
    0x00, 0x00, 0x00, 0x07,  // Output indices count
    0x00, 0x00,              // Index: 0      C1  L0  L1
    0x00, 0x01,              // Index: 1  C0  C1  L0  L1
    0x00, 0x02,              // Index: 2  C0      L0  L1
    0x00, 0x03,              // Index: 3      C1  L0  L1
    0x00, 0x06,              // Index: 6      C1  L0
    0x00, 0x07,              // Index: 7  C0      L0
    0x00, 0x08               // Index: 8  C0  C1  L0
};

const unsigned char RawV25::animatedmaps[] = {
    // LOD sizes
    0x00,
    0x00,
    0x00,
    0x02,  // Row count per LOD
    0x00,
    0x0f,  // LOD-0 row-count
    0x00,
    0x06  // LOD-1 row-count
};

const unsigned char RawV25::geometry[] = {
    0x00,
    0x00,
    0x00,
    0x03,  // Mesh count
    // Mesh-0
    0x00,
    0x00,
    0x01,
    0x52,  // Mesh-0 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x01,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x40,
    0xe0,
    0x00,
    0x00,  // 7.0f
    0x41,
    0x00,
    0x00,
    0x00,  // 8.0f
    0x41,
    0x10,
    0x00,
    0x00,  // 9.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    // Mesh-1
    0x00,
    0x00,
    0x01,
    0x52,  // Mesh-1 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3e,
    0xcc,
    0xcc,
    0xcd,  // 0.4f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3f,
    0x66,
    0x66,
    0x66,  // 0.9f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x01,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x40,
    0xc0,
    0x00,
    0x00,  // 6.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    // Mesh-2
    0x00,
    0x00,
    0x01,
    0x84,  // Mesh-2 size
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex positions Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates U values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Texture coordinates V values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex normals Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - position indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - texture coordinate indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex texture coordinate: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex texture coordinate: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex texture coordinate: 2
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex layouts - normal indices length
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex normal: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex normal: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex normal: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Face count: 1
    0x00,
    0x00,
    0x00,
    0x03,  // Face 1 layout indices length: 3
    0x00,
    0x00,
    0x00,
    0x00,  // Layout index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Layout index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Layout index: 2
    0x00,
    0x08,  // Maximum influence per vertex
    0x00,
    0x00,
    0x00,
    0x03,  // Skin weights structure count: 3 (for each vertex)
    0x00,
    0x00,
    0x00,
    0x03,  // Weights length: 3 (for each influencing joint)
    0x3d,
    0xcc,
    0xcc,
    0xcd,  // 0.1f
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x19,
    0x99,
    0x9a,  // 0.6f
    0x00,
    0x00,
    0x00,
    0x03,  // Influencing joint count: 3 (for each weight)
    0x00,
    0x00,  // Joint: 0
    0x00,
    0x01,  // Joint: 1
    0x00,
    0x02,  // Joint: 2
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0x99,
    0x99,
    0x9a,  // 0.3f
    0x3f,
    0x33,
    0x33,
    0x33,  // 0.7f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x03,  // Joint: 3
    0x00,
    0x04,  // Joint: 4
    0x00,
    0x00,
    0x00,
    0x02,  // Weights length: 2 (for each influencing joint)
    0x3e,
    0x4c,
    0xcc,
    0xcd,  // 0.2f
    0x3f,
    0x4c,
    0xcc,
    0xcd,  // 0.8f
    0x00,
    0x00,
    0x00,
    0x02,  // Influencing joint count: 2 (for each weight)
    0x00,
    0x05,  // Joint: 5
    0x00,
    0x06,  // Joint: 6
    0x00,
    0x00,
    0x00,
    0x02,  // Number of blendshapes
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas X values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Y values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Blend shape deltas Z values length
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x40,
    0x00,
    0x00,
    0x00,  // 2.0f
    0x40,
    0x40,
    0x00,
    0x00,  // 3.0f
    0x00,
    0x00,
    0x00,
    0x03,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Vertex position: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x02,  // Blend shape index in Definition
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas X values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas Y values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Blend shape deltas Z values length
    0x40,
    0x80,
    0x00,
    0x00,  // 4.0f
    0x40,
    0xa0,
    0x00,
    0x00,  // 5.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position indices length (for each delta)
    0x00,
    0x00,
    0x00,
    0x00,  // Vertex position: 0
    0x00,
    0x00,
    0x00,
    0x02,  // Vertex position: 2
    0x00,
    0x03  // Blend shape index in Definition
};

const unsigned char RawV25::machineLearnedBehavior[] = {
    0x00,
    0x00,
    0x00,
    0x09,  // Raw control names length
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 0 length
    0x4d,
    0x41,  // Raw control name 0 : MA
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 1 length
    0x4d,
    0x42,  // Raw control name 1 : MB
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 2 length
    0x4d,
    0x43,  // Raw control name 2 : MC
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 3 length
    0x4d,
    0x44,  // Raw control name 3 : MD
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 4 length
    0x4d,
    0x45,  // Raw control name 4 : ME
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 5 length
    0x4d,
    0x46,  // Raw control name 5 : MF
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 6 length
    0x4d,
    0x47,  // Raw control name 6 : MG
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 7 length
    0x4d,
    0x48,  // Raw control name 7 : MH
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control name 8 length
    0x4d,
    0x49,  // Raw control name 8 : MI
    0x00,
    0x00,
    0x00,
    0x02,  // Neural network indices lod to row mapping length
    0x00,
    0x00,  // Map from LOD-0 to row 0 in below defined matrix
    0x00,
    0x01,  // Map from LOD-1 to row 1 in below defined matrix
    0x00,
    0x00,
    0x00,
    0x02,  // Neural network indices per LOD row count
    0x00,
    0x00,
    0x00,
    0x04,  // Indices matrix row-0
    0x00,
    0x00,  // Neural network index: 0
    0x00,
    0x01,  // Neural network index: 1
    0x00,
    0x02,  // Neural network index: 2
    0x00,
    0x03,  // Neural network index: 3
    0x00,
    0x00,
    0x00,
    0x02,  // Indices matrix row-1
    0x00,
    0x04,  // Neural network index: 4
    0x00,
    0x05,  // Neural network index: 5
    0x00,
    0x00,
    0x00,
    0x03,  // Region names length
    0x00,
    0x00,
    0x00,
    0x02,  // Region names length for mesh 0
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 0 length
    0x52,
    0x41,  // Region name 0 : RA
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 1 length
    0x52,
    0x42,  // Region name 1 : RB
    0x00,
    0x00,
    0x00,
    0x02,  // Region names length for mesh 1
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 0 length
    0x52,
    0x43,  // Region name 0 : RC
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 1 length
    0x52,
    0x44,  // Region name 1 : RD
    0x00,
    0x00,
    0x00,
    0x02,  // Region names length for mesh 2
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 0 length
    0x52,
    0x45,  // Region name 0 : RE
    0x00,
    0x00,
    0x00,
    0x02,  // Region name 1 length
    0x52,
    0x46,  // Region name 1 : RF
    0x00,
    0x00,
    0x00,
    0x03,  // Mesh count
    0x00,
    0x00,
    0x00,
    0x02,  // Region count for Mesh-0
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-0 Region-0
    0x00,
    0x00,  // Neural network index: 0
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-0 Region-1
    0x00,
    0x01,  // Neural network index: 1
    0x00,
    0x00,
    0x00,
    0x02,  // Region count for Mesh-1
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-1 Region-0
    0x00,
    0x02,  // Neural network index: 2
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-1 Region-1
    0x00,
    0x03,  // Neural network index: 3
    0x00,
    0x00,
    0x00,
    0x02,  // Region count for Mesh-2
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-2 Region-0
    0x00,
    0x04,  // Neural network index: 4
    0x00,
    0x00,
    0x00,
    0x01,  // Neural network index count for Mesh-2 Region-1
    0x00,
    0x05,  // Neural network index: 5
    0x00,
    0x00,
    0x00,
    0x06,  // Neural network count
    // Mesh-0 Region-0 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-0 Region-0 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network output index count
    0x00,
    0x09,  // Mesh-0 Region-0 neural network output index-9
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network input index count
    0x00,
    0x00,  // Mesh-0 Region-0 neural network input index-0
    0x00,
    0x01,  // Mesh-0 Region-0 neural network input index-1
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-0 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-0 neural network layer-0 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-0 Region-0 neural network layer-0 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-0 neural network layer-1 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    // Mesh-0 Region-1 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-0 Region-1 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network output index count
    0x00,
    0x0a,  // Mesh-0 Region-1 neural network output index-10
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network input index count
    0x00,
    0x02,  // Mesh-0 Region-1 neural network input index-2
    0x00,
    0x03,  // Mesh-0 Region-1 neural network input index-3
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network layer-0 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-0 Region-1 neural network layer-0 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-0 Region-1 neural network layer-1 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-0 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    // Mesh-1 Region-0 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-1 Region-0 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network output index count
    0x00,
    0x0b,  // Mesh-1 Region-0 neural network output index-11
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network input index count
    0x00,
    0x04,  // Mesh-1 Region-0 neural network input index-4
    0x00,
    0x05,  // Mesh-1 Region-0 neural network input index-5
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-0 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-0 neural network layer-0 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-1 Region-0 neural network layer-0 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-0 neural network layer-1 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    // Mesh-1 Region-1 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-1 Region-1 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network output index count
    0x00,
    0x0c,  // Mesh-1 Region-1 neural network output index-12
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network input index count
    0x00,
    0x06,  // Mesh-1 Region-1 neural network input index-6
    0x00,
    0x07,  // Mesh-1 Region-1 neural network input index-7
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network layer-0 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-1 Region-1 neural network layer-0 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-1 Region-1 neural network layer-1 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-1 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    // Mesh-2 Region-0 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-2 Region-0 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network output index count
    0x00,
    0x0d,  // Mesh-2 Region-0 neural network output index-13
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network input index count
    0x00,
    0x08,  // Mesh-2 Region-0 neural network input index-8
    0x00,
    0x00,  // Mesh-2 Region-0 neural network input index-0
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-0 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-0 neural network layer-0 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-2 Region-0 neural network layer-0 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 bias count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-0 neural network layer-1 weight count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-0 neural network layer-1 activation function parameter count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    // Mesh-2 Region-1 neural network
    0x00,
    0x00,
    0x00,
    0x5a,  // Mesh-2 Region-1 neural network size
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network output index count
    0x00,
    0x0e,  // Mesh-2 Region-1 neural network output index-14
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network input index count
    0x00,
    0x04,  // Mesh-2 Region-1 neural network input index-4
    0x00,
    0x07,  // Mesh-2 Region-1 neural network input index-7
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network layer count
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network layer-0 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x04,  // Mesh-2 Region-1 neural network layer-0 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-0 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 bias count
    0x3f,
    0x00,
    0x00,
    0x00,  // 0.5f
    0x00,
    0x00,
    0x00,
    0x02,  // Mesh-2 Region-1 neural network layer-1 weight count
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x3f,
    0x80,
    0x00,
    0x00,  // 1.0f
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 activation function ID
    0x00,
    0x00,
    0x00,
    0x01,  // Mesh-2 Region-1 neural network layer-1 activation function parameter count
    0x3f,
    0x80,
    0x00,
    0x00  // 1.0f
};

const unsigned char RawV25::rbfBehavior[] = {
    0x00,
    0x00,
    0x00,
    0x02,  // Solver LOD mapping lods length (2)
    0x00,
    0x00,  // Solver LOD mapping lod 0 (0)
    0x00,
    0x01,  // Solver LOD mapping lod 1 (1)
    0x00,
    0x00,
    0x00,
    0x02,  // Solver LOD mapping indices length (2)
    0x00,
    0x00,
    0x00,
    0x02,  // Solver LODs lod 0 indices length (2)
    0x00,
    0x00,  // Solver index 0 (0)
    0x00,
    0x01,  // Solver index 1 (1)
    0x00,
    0x00,
    0x00,
    0x02,  // Solver LODs lod 1 indices length (2)
    0x00,
    0x01,  // Solver index 0 (1)
    0x00,
    0x02,  // Solver index 1 (2)
    0x00,
    0x00,
    0x00,
    0x03,  // Solvers length (3)
    // Solver 0
    0x00,
    0x00,
    0x00,
    0x49,  // Solver size (73)
    0x00,
    0x00,
    0x00,
    0x03,  // Solver name length (3)
    0x52,
    0x53,
    0x41,  // Solver name (RSA)
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control indices length (2)
    0x00,
    0x0b,  // Raw control index 0 (11)
    0x00,
    0x0c,  // Raw control index 1 (12)
    0x00,
    0x00,
    0x00,
    0x03,  // Solver pose indices length (3)
    0x00,
    0x00,  // Pose index 0 (0)
    0x00,
    0x01,  // Pose index 1 (1)
    0x00,
    0x02,  // Pose index 2 (2)
    0x00,
    0x00,
    0x00,
    0x06,  // Raw control values length (6)
    0x40,
    0x00,
    0x00,
    0x00,  // Pose 0 Raw control value 0 (2.0)
    0x00,
    0x00,
    0x00,
    0x00,  // Pose 0 Raw control value 1 (0.0)
    0x3f,
    0x80,
    0x00,
    0x00,  // Pose 1 Raw control value 0 (1.0)
    0x3f,
    0x80,
    0x00,
    0x00,  // Pose 1 Raw control value 1 (1.0)
    0x40,
    0x40,
    0x00,
    0x00,  // Pose 2 Raw control value 0 (3.0)
    0xc0,
    0x40,
    0x00,
    0x00,  // Pose 2 Raw control value 1 (-3.0)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver radius (1.0)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver weight threshold (1.0)
    0x00,
    0x00,  // Solver type (0)
    0x00,
    0x00,  // Solver automatic radius (0)
    0x00,
    0x01,  // Solver distance method (1)
    0x00,
    0x00,  // Solver normalize method (0)
    0x00,
    0x02,  // Solver function type (2)
    0x00,
    0x00,  // Solver TwistAxis method (0)
    // Solver 1
    0x00,
    0x00,
    0x00,
    0x35,  // Solver size (53)
    0x00,
    0x00,
    0x00,
    0x03,  // Solver name length (3)
    0x52,
    0x53,
    0x42,  // Solver name (RSB)
    0x00,
    0x00,
    0x00,
    0x01,  // Raw control indices length (1)
    0x00,
    0x03,  // Raw control index 0 (3)
    0x00,
    0x00,
    0x00,
    0x02,  // Solver pose indices length (2)
    0x00,
    0x03,  // Pose index 0 (3)
    0x00,
    0x04,  // Pose index 1 (4)
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control values length (2)
    0x00,
    0x00,
    0x00,
    0x00,  // Pose 2 Raw control value 0 (0.0)
    0x40,
    0x80,
    0x00,
    0x00,  // Pose 3 Raw control value 1 (4.0)
    0x40,
    0x00,
    0x00,
    0x00,  // Solver radius 0 (2.0)
    0x40,
    0x00,
    0x00,
    0x00,  // Solver weight threshold 0 (2.0)
    0x00,
    0x01,  // Solver type (1)
    0x00,
    0x00,  // Solver automatic radius (0)
    0x00,
    0x03,  // Solver distance method (3)
    0x00,
    0x01,  // Solver normalize method (1)
    0x00,
    0x02,  // Solver function type (2)
    0x00,
    0x01,  // Solver TwistAxis method (1)
    // Solver 2
    0x00,
    0x00,
    0x00,
    0x49,  // Solver size (73)
    0x00,
    0x00,
    0x00,
    0x03,  // Solver name length (3)
    0x52,
    0x53,
    0x43,  // Solver name (RSC)
    0x00,
    0x00,
    0x00,
    0x02,  // Raw control indices length (2)
    0x00,
    0x16,  // Raw control index 0 (22)
    0x00,
    0x17,  // Raw control index 0 (23)
    0x00,
    0x00,
    0x00,
    0x03,  // Solver pose indices length (3)
    0x00,
    0x05,  // Pose index 0 (5)
    0x00,
    0x06,  // Pose index 1 (6)
    0x00,
    0x07,  // Pose index 2 (7)
    0x00,
    0x00,
    0x00,
    0x06,  // Raw control values length (6)
    0x40,
    0x00,
    0x00,
    0x00,  // Pose 5 Raw control value 0(2.0)
    0x00,
    0x00,
    0x00,
    0x00,  // Pose 5 Raw control value 1(0.0)
    0x3f,
    0x80,
    0x00,
    0x00,  // Pose 6 Raw control value 0(1.0)
    0x3f,
    0x80,
    0x00,
    0x00,  // Pose 6 Raw control value 1(1.0)
    0x40,
    0x40,
    0x00,
    0x00,  // Pose 7 Raw control value 0(3.0)
    0xc0,
    0x40,
    0x00,
    0x00,  // Pose 7 Raw control value 1(-3.0f)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver radius (1.0)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver weight threshold (0.5)
    0x00,
    0x00,  // Solver type (0)
    0x00,
    0x00,  // Solver automatic radius (0)
    0x00,
    0x01,  // Solver distance method (1)
    0x00,
    0x00,  // Solver normalize method (0)
    0x00,
    0x00,  // Solver function type (0)
    0x00,
    0x00,  // Solver TwistAxis method (0)
    0x00,
    0x00,
    0x00,
    0x08,  // Pose length (8)
    // Pose 0
    0x00,
    0x00,
    0x00,
    0x02,  // pose name length (2)
    0x52,
    0x41,  // pose name (RA)
    0x00,
    0x00,
    0x00,
    0x00,  // Solver pose scale 0 (0.0)
    // Pose 1
    0x00,
    0x00,
    0x00,
    0x02,  // pose name length (2)
    0x52,
    0x42,  // pose name (RB)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver pose scale (1.0)
    // Pose 2
    0x00,
    0x00,
    0x00,
    0x02,  // pose name length (2)
    0x52,
    0x43,  // pose name (RC)
    0x40,
    0x00,
    0x00,
    0x00,  // Solver pose scale (2.0)
    // Pose 3
    0x00,
    0x00,
    0x00,
    0x02,  // Pose name length (2)
    0x52,
    0x44,  // pose name (RD)
    0x40,
    0x00,
    0x00,
    0x00,  // Solver pose scale 0 (2.0)
    // Pose 4
    0x00,
    0x00,
    0x00,
    0x02,  // pose name length (2)
    0x52,
    0x45,  // pose name(RE)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver pose scale (1.0)
    // Pose 5
    0x00,
    0x00,
    0x00,
    0x02,  // pose name length (2)
    0x52,
    0x46,  // pose name(RF)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver scale 0 (1.0)
    // Pose 6
    0x00,
    0x00,
    0x00,
    0x02,  // pose name ,length (2)
    0x52,
    0x47,  // pose name(RG)
    0x3f,
    0x80,
    0x00,
    0x00,  // Solver pose scale (1.0)
    // Pose 7
    0x00,
    0x00,
    0x00,
    0x02,  // pose name length (2)
    0x52,
    0x48,  // pose name(RH)
    0x3f,
    0x00,
    0x00,
    0x00,  // Solver pose scale (0.5)
};

const unsigned char RawV25::rbfBehaviorExt[] = {
    0x00, 0x00, 0x00, 0x09,  // Pose control name count
    0x00, 0x00, 0x00, 0x02,  // Pose control name 0 length
    0x50, 0x41,              // Pose control name 0 : PA
    0x00, 0x00, 0x00, 0x02,  // Pose control name 1 length
    0x50, 0x42,              // Pose control name 1 : PB
    0x00, 0x00, 0x00, 0x02,  // Pose control name 2 length
    0x50, 0x43,              // Pose control name 2 : PC
    0x00, 0x00, 0x00, 0x02,  // Pose control name 3 length
    0x50, 0x44,              // Pose control name 3 : PD
    0x00, 0x00, 0x00, 0x02,  // Pose control name 4 length
    0x50, 0x45,              // Pose control name 4 : PE
    0x00, 0x00, 0x00, 0x02,  // Pose control name 5 length
    0x50, 0x46,              // Pose control name 5 : PF
    0x00, 0x00, 0x00, 0x02,  // Pose control name 6 length
    0x50, 0x47,              // Pose control name 6 : PG
    0x00, 0x00, 0x00, 0x02,  // Pose control name 7 length
    0x50, 0x48,              // Pose control name 7 : PH
    0x00, 0x00, 0x00, 0x02,  // Pose control name 8 length
    0x50, 0x49,              // Pose control name 8 : PI
    0x00, 0x00, 0x00, 0x08,  // Pose count
    0x00, 0x00, 0x00, 0x01,  // Pose-0 input control index count
    0x00, 0x00,              // Pose-0 input control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-0 output control index count
    0x00, 0x08,              // Pose-0 output control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-0 output control weight count
    0x3f, 0x80, 0x00, 0x00,  // Pose-0 output control weight-0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // Pose-1 input control index count
    0x00, 0x01,              // Pose-1 input control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-1 output control index count
    0x00, 0x09,              // Pose-1 output control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-1 output control weight count
    0x3f, 0x80, 0x00, 0x00,  // Pose-1 output control weight-0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // Pose-2 input control index count
    0x00, 0x02,              // Pose-2 input control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-2 output control index count
    0x00, 0x0a,              // Pose-2 output control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-2 output control weight count
    0x3f, 0x80, 0x00, 0x00,  // Pose-2 output control weight-0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // Pose-3 input control index count
    0x00, 0x03,              // Pose-3 input control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-3 output control index count
    0x00, 0x0b,              // Pose-3 output control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-3 output control weight count
    0x3f, 0x80, 0x00, 0x00,  // Pose-3 output control weight-0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // Pose-4 input control index count
    0x00, 0x04,              // Pose-4 input control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-4 output control index count
    0x00, 0x0c,              // Pose-4 output control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-4 output control weight count
    0x3f, 0x80, 0x00, 0x00,  // Pose-4 output control weight-0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // Pose-5 input control index count
    0x00, 0x05,              // Pose-5 input control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-5 output control index count
    0x00, 0x0d,              // Pose-5 output control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-5 output control weight count
    0x3f, 0x80, 0x00, 0x00,  // Pose-5 output control weight-0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // Pose-6 input control index count
    0x00, 0x06,              // Pose-6 input control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-6 output control index count
    0x00, 0x0e,              // Pose-6 output control index-0
    0x00, 0x00, 0x00, 0x01,  // Pose-6 output control weight count
    0x3f, 0x80, 0x00, 0x00,  // Pose-6 output control weight-0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // Pose-7 input control index count
    0x00, 0x07,              // Pose-7 input control index-0
    0x00, 0x00, 0x00, 0x02,  // Pose-7 output control index count
    0x00, 0x0f,              // Pose-7 output control index-0
    0x00, 0x10,              // Pose-7 output control index-1
    0x00, 0x00, 0x00, 0x02,  // Pose-7 output control weight count
    0x3f, 0x00, 0x00, 0x00,  // Pose-7 output control weight-0 (0.5)
    0x3f, 0x00, 0x00, 0x00   // Pose-7 output control weight-1 (0.5)
};

const unsigned char RawV25::jointBehaviorMetadata[] = {
    0x00, 0x00, 0x00, 0x09,  // joint representations length (9)
    0x00, 0x00,              // joint 0 translation (Vector)
    0x00, 0x00,              // joint 0 rotation (EulerAngles)
    0x00, 0x00,              // joint 0 scale (Vector)
    0x00, 0x00,              // joint 1 translation (Vector)
    0x00, 0x00,              // joint 1 rotation (EulerAngles)
    0x00, 0x00,              // joint 1 scale (Vector)
    0x00, 0x00,              // joint 2 translation (Vector)
    0x00, 0x01,              // joint 2 rotation (Quaternion)
    0x00, 0x00,              // joint 2 scale (Vector)
    0x00, 0x00,              // joint 3 translation (Vector)
    0x00, 0x01,              // joint 3 rotation (Quaternion)
    0x00, 0x00,              // joint 3 scale (Vector)
    0x00, 0x00,              // joint 4 translation (Vector)
    0x00, 0x00,              // joint 4 rotation (EulerAngles)
    0x00, 0x00,              // joint 4 scale (Vector)
    0x00, 0x00,              // joint 5 translation (Vector)
    0x00, 0x00,              // joint 5 rotation (EulerAngles)
    0x00, 0x00,              // joint 5 scale (Vector)
    0x00, 0x00,              // joint 6 translation (Vector)
    0x00, 0x00,              // joint 6 rotation (EulerAngles)
    0x00, 0x00,              // joint 6 scale (Vector)
    0x00, 0x00,              // joint 7 translation (Vector)
    0x00, 0x01,              // joint 7 rotation (Quaternion)
    0x00, 0x00,              // joint 7 translation (Vector)
    0x00, 0x00,              // joint 8 translation (Vector)
    0x00, 0x00,              // joint 8 rotation (EulerAngles)
    0x00, 0x00,              // joint 8 translation (Vector)
};

const unsigned char RawV25::twistSwingBehavior[] = {
    0x00, 0x00, 0x00, 0x03,  // twist setups length (3)
    0x00, 0x00, 0x00, 0x02,  // setup 0 twist blend weights length (2)
    0x3f, 0x80, 0x00, 0x00,  // setup 0 blend weight 0 (1.0)
    0x40, 0x00, 0x00, 0x00,  // setup 0 blend weight 1 (2.0)
    0x00, 0x00, 0x00, 0x02,  // setup 0 twist output joint indices length (2)
    0x00, 0x00,              // setup 0 twist output index 0 (0)
    0x00, 0x01,              // setup 0 twist output index 1 (1)
    0x00, 0x00, 0x00, 0x04,  // setup 0 twist input control indices length (4)
    0x00, 0x05,              // setup 0 twist input index 0 (5)
    0x00, 0x06,              // setup 0 twist input index 1 (6)
    0x00, 0x07,              // setup 0 twist input index 2 (7)
    0x00, 0x08,              // setup 0 twist input index 3 (8)
    0x00, 0x00,              // setup 0 twist axis (X)
    0x00, 0x00, 0x00, 0x02,  // setup 1 twist blend weights length (2)
    0xc0, 0x00, 0x00, 0x00,  // setup 1 blend weight 0 (-2.0)
    0xbf, 0x80, 0x00, 0x00,  // setup 1 blend weight 1 (-1.0)
    0x00, 0x00, 0x00, 0x02,  // setup 1 twist output joint indices length (2)
    0x00, 0x04,              // setup 1 twist output index 0 (4)
    0x00, 0x06,              // setup 1 twist output index 1 (6)
    0x00, 0x00, 0x00, 0x04,  // setup 1 twist input control indices length (4)
    0x00, 0x0b,              // setup 1 twist input index 0 (11)
    0x00, 0x0c,              // setup 1 twist input index 1 (12)
    0x00, 0x0d,              // setup 1 twist input index 2 (13)
    0x00, 0x0e,              // setup 1 twist input index 3 (14)
    0x00, 0x01,              // setup 1 twist axis (Y)
    0x00, 0x00, 0x00, 0x01,  // setup 2 twist blend weights length (1)
    0x3f, 0x80, 0x00, 0x00,  // setup 2 blend weight 0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // setup 2 twist output joint indices length (1)
    0x00, 0x05,              // setup 2 twist output index 5 (5)
    0x00, 0x00, 0x00, 0x04,  // setup 2 twist input control indices length (4)
    0x00, 0x1b,              // setup 2 twist input index 0 (27)
    0x00, 0x1c,              // setup 2 twist input index 1 (28)
    0x00, 0x1d,              // setup 2 twist input index 2 (29)
    0x00, 0x1e,              // setup 2 twist input index 3 (30)
    0x00, 0x02,              // setup 2 twist axis (Z)
    0x00, 0x00, 0x00, 0x03,  // swing setups length (3)
    0x00, 0x00, 0x00, 0x02,  // setup 0 swing blend weights length (2)
    0x3f, 0x80, 0x00, 0x00,  // setup 0 blend weight 0 (1.0)
    0x40, 0x00, 0x00, 0x00,  // setup 0 blend weight 1 (2.0)
    0x00, 0x00, 0x00, 0x02,  // setup 0 swing output joint indices length (2)
    0x00, 0x00,              // setup 0 swing output index 0 (0)
    0x00, 0x01,              // setup 0 swing output index 1 (1)
    0x00, 0x00, 0x00, 0x04,  // setup 0 swing input control indices length (4)
    0x00, 0x05,              // setup 0 swing input index 0 (5)
    0x00, 0x06,              // setup 0 swing input index 1 (6)
    0x00, 0x07,              // setup 0 swing input index 2 (7)
    0x00, 0x08,              // setup 0 swing input index 3 (8)
    0x00, 0x00,              // setup 0 twist axis (X)
    0x00, 0x00, 0x00, 0x02,  // setup 1 swing blend weights length (2)
    0xc0, 0x00, 0x00, 0x00,  // setup 1 blend weight 0 (-2.0)
    0xbf, 0x80, 0x00, 0x00,  // setup 1 blend weight 1 (-1.0)
    0x00, 0x00, 0x00, 0x02,  // setup 1 swing output joint indices length (2)
    0x00, 0x04,              // setup 1 swing output index 0 (4)
    0x00, 0x06,              // setup 1 swing output index 1 (6)
    0x00, 0x00, 0x00, 0x04,  // setup 1 swing input control indices length (4)
    0x00, 0x0b,              // setup 1 swing input index 0 (11)
    0x00, 0x0c,              // setup 1 swing input index 1 (12)
    0x00, 0x0d,              // setup 1 swing input index 2 (13)
    0x00, 0x0e,              // setup 1 swing input index 3 (14)
    0x00, 0x01,              // setup 1 twist axis (Y)
    0x00, 0x00, 0x00, 0x01,  // setup 2 swing blend weights length (1)
    0x3f, 0x80, 0x00, 0x00,  // setup 2 blend weight 0 (1.0)
    0x00, 0x00, 0x00, 0x01,  // setup 2 swing output joint indices length (1)
    0x00, 0x05,              // setup 2 swing output index 5 (5)
    0x00, 0x00, 0x00, 0x04,  // setup 2 swing input control indices length (4)
    0x00, 0x1b,              // setup 2 swing input index 0 (27)
    0x00, 0x1c,              // setup 2 swing input index 1 (28)
    0x00, 0x1d,              // setup 2 swing input index 2 (29)
    0x00, 0x1e,              // setup 2 swing input index 3 (30)
    0x00, 0x02               // setup 2 twist axis (Z)
};

std::vector<char> RawV25::getBytes() {
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wstringop-overflow"
#endif
    std::vector<char> bytes;
    // Header
    bytes.insert(bytes.end(), header, header + sizeof(header));
    // Descriptor
    bytes.insert(bytes.end(), descriptor, descriptor + sizeof(descriptor));
    // Definition
    bytes.insert(bytes.end(), definition, definition + sizeof(definition));
    // Behavior
    // > Controls
    bytes.insert(bytes.end(), controls, controls + sizeof(controls));
    bytes.insert(bytes.end(), conditionals, conditionals + sizeof(conditionals));
    bytes.insert(bytes.end(), psds, psds + sizeof(psds));
    // > Joints
    bytes.insert(bytes.end(), joints, joints + sizeof(joints));
    // > BlendShapes
    bytes.insert(bytes.end(), blendshapes, blendshapes + sizeof(blendshapes));
    // > AnimatedMaps
    bytes.insert(bytes.end(), animatedmaps, animatedmaps + sizeof(animatedmaps));
    bytes.insert(bytes.end(), conditionals, conditionals + sizeof(conditionals));
    // Geometry
    bytes.insert(bytes.end(), geometry, geometry + sizeof(geometry));
    // Machine learned behavior
    bytes.insert(bytes.end(), machineLearnedBehavior, machineLearnedBehavior + sizeof(machineLearnedBehavior));
    // RBF behavior
    bytes.insert(bytes.end(), rbfBehavior, rbfBehavior + sizeof(rbfBehavior));
    // RBF behavior ext
    bytes.insert(bytes.end(), rbfBehaviorExt, rbfBehaviorExt + sizeof(rbfBehaviorExt));
    // JointBehavior meta data
    bytes.insert(bytes.end(), jointBehaviorMetadata, jointBehaviorMetadata + sizeof(jointBehaviorMetadata));
    // Twist swing behavior
    bytes.insert(bytes.end(), twistSwingBehavior, twistSwingBehavior + sizeof(twistSwingBehavior));
    return bytes;
#if !defined(__clang__) && defined(__GNUC__) && (__GNUC__ >= 12)
    #pragma GCC diagnostic pop
#endif
}

std::vector<char> RawV24DowngradedFromV25::getBytes() {
    auto bytes = RawV25::getBytes();
    bytes[5] = 0x00;
    bytes[6] = 0x04;
    return bytes;
}

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif
// Descriptor
const pma::String<char> DecodedV25::name = "test";
const Archetype DecodedV25::archetype = Archetype::other;
const Gender DecodedV25::gender = Gender::other;
const std::uint16_t DecodedV25::age = 42u;
const pma::Vector<DecodedV25::StringPair> DecodedV25::metadata = {{"key-A", "value-A"}, {"key-B", "value-B"}};
const TranslationUnit DecodedV25::translationUnit = TranslationUnit::m;
const RotationUnit DecodedV25::rotationUnit = RotationUnit::radians;
const CoordinateSystem DecodedV25::coordinateSystem = {Direction::right, Direction::up, Direction::front};
const std::uint16_t DecodedV25::lodCount[] = {
    2u,  // MaxLOD-0 - MinLOD-1
    1u,  // MaxLOD-1 - MinLOD-1
    1u   // MaxLOD-0 - MinLOD-0
};
const std::uint16_t DecodedV25::maxLODs[] = {
    0u,  // MaxLOD-0 - MinLOD-1
    1u,  // MaxLOD-1 - MinLOD-0
    0u   // MaxLOD-0 - MinLOD-0
};
const pma::String<char> DecodedV25::complexity = "A";
const pma::String<char> DecodedV25::dbName = "testDB";

// Definition
const pma::Vector<pma::String<char>> DecodedV25::guiControlNames = {"GA", "GB", "GC", "GD", "GE", "GF", "GG", "GH", "GI"};
const pma::Vector<pma::String<char>> DecodedV25::rawControlNames = {"RA", "RB", "RC", "RD", "RE", "RF", "RG", "RH", "RI"};
const DecodedV25::VectorOfCharStringMatrix DecodedV25::jointNames = {{// MaxLOD-0 - MinLOD-1
                                                                      {"JA", "JB", "JC", "JD", "JE", "JF", "JG", "JH", "JI"},
                                                                      {"JA", "JB", "JC", "JD", "JG", "JI"}},
                                                                     {// MaxLOD-1 - MinLOD-0
                                                                      {"JA", "JB", "JC", "JD", "JG", "JI"}},
                                                                     {
                                                                         // MaxLOD-0 - MinLOD-0
                                                                         {"JA", "JB", "JC", "JD", "JE", "JF", "JG", "JH", "JI"},
                                                                     }};
const DecodedV25::VectorOfCharStringMatrix DecodedV25::blendShapeNames = {
    {// MaxLOD-0 - MinLOD-1
     {"BA", "BB", "BC", "BD", "BE", "BF", "BG", "BH", "BI"},
     {"BC", "BF", "BH", "BI"}},
    {// MaxLOD-1 - MinLOD-1
     {"BC", "BF", "BH", "BI"}},
    {
        // MaxLOD-0 - MinLOD-0
        {"BA", "BB", "BC", "BD", "BE", "BF", "BG", "BH", "BI"},
    }};
const DecodedV25::VectorOfCharStringMatrix DecodedV25::animatedMapNames = {
    {// MaxLOD-0 - MinLOD-1
     {"AA", "AB", "AC", "AD", "AE", "AF", "AG", "AH", "AI", "AJ"},
     {"AC", "AF", "AH", "AI"}},
    {// MaxLOD-1 - MinLOD-1
     {"AC", "AF", "AH", "AI"}},
    {
        // MaxLOD-0 - MinLOD-0
        {"AA", "AB", "AC", "AD", "AE", "AF", "AG", "AH", "AI", "AJ"},
    }};
const DecodedV25::VectorOfCharStringMatrix DecodedV25::meshNames = {{// MaxLOD-0 - MinLOD-1
                                                                     {"MA", "MB"},
                                                                     {"MC"}},
                                                                    {// MaxLOD-1 - MinLOD-1
                                                                     {"MC"}},
                                                                    {// MaxLOD-0 - MinLOD-0
                                                                     {"MA", "MB"}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::meshBlendShapeIndices = {{// MaxLOD-0 - MinLOD-1
                                                                                    {0, 1, 2, 3, 4, 5, 6},
                                                                                    {7, 8}},
                                                                                   {// MaxLOD-1 - MinLOD-1
                                                                                    {0, 1}},
                                                                                   {
                                                                                       // MaxLOD-0 - MinLOD-0
                                                                                       {0, 1, 2, 3, 4, 5, 6},
                                                                                   }};
const pma::Matrix<std::uint16_t> DecodedV25::jointHierarchy = {{// MaxLOD-0 - MinLOD-1
                                                                {0, 0, 0, 1, 1, 4, 2, 4, 2}},
                                                               {// MaxLOD-1 - MinLOD-1
                                                                {0, 0, 0, 1, 2, 2}},
                                                               {// MaxLOD-0 - MinLOD-0
                                                                {0, 0, 0, 1, 1, 4, 2, 4, 2}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV25::neutralJointTranslations = {
    {// MaxLOD-0 - MinLOD-1
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}},
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-1 - MinLOD-1
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-0 - MinLOD-0
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV25::neutralJointRotations = {
    {// MaxLOD-0 - MinLOD-1
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}},
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-1 - MinLOD-1
     {{1.0f, 1.0f, 1.0f}, {2.0f, 2.0f, 2.0f}, {3.0f, 3.0f, 3.0f}, {4.0f, 4.0f, 4.0f}, {7.0f, 7.0f, 7.0f}, {9.0f, 9.0f, 9.0f}}},
    {// MaxLOD-0 - MinLOD-0
     {{1.0f, 1.0f, 1.0f},
      {2.0f, 2.0f, 2.0f},
      {3.0f, 3.0f, 3.0f},
      {4.0f, 4.0f, 4.0f},
      {5.0f, 5.0f, 5.0f},
      {6.0f, 6.0f, 6.0f},
      {7.0f, 7.0f, 7.0f},
      {8.0f, 8.0f, 8.0f},
      {9.0f, 9.0f, 9.0f}}}};

// Behavior
const std::uint16_t DecodedV25::guiControlCount = 9u;
const std::uint16_t DecodedV25::rawControlCount = 9u;
const std::uint16_t DecodedV25::psdCount = 12u;
// Behavior->Conditionals
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::conditionalInputIndices = {
    {
        // MaxLOD-0 - MinLOD-1
        {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8},
        {0, 1, 1, 2, 3, 3},
    },
    {
        // MaxLOD-1 - MinLOD-1
        {0, 1, 1, 2, 3, 3},
    },
    {// MaxLOD-0 - MinLOD-0
     {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::conditionalOutputIndices = {
    {
        // MaxLOD-0 - MinLOD-1
        {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8},
        {0, 1, 1, 2, 3, 3},
    },
    {
        // MaxLOD-1 - MinLOD-1
        {0, 1, 1, 2, 3, 3},
    },
    {// MaxLOD-0 - MinLOD-0
     {0, 1, 1, 2, 3, 3, 4, 4, 4, 5, 6, 7, 7, 8, 8}}};
const pma::Vector<pma::Matrix<float>> DecodedV25::conditionalFromValues = {
    {// MaxLOD-0 - MinLOD-1
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f, 0.0f, 0.4f, 0.7f, 0.5f, 0.0f, 0.1f, 0.6f, 0.2f, 0.0f},
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f}},
    {// MaxLOD-1 - MinLOD-1
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f}},
    {// MaxLOD-0 - MinLOD-0
     {0.0f, 0.0f, 0.6f, 0.4f, 0.1f, 0.7f, 0.0f, 0.4f, 0.7f, 0.5f, 0.0f, 0.1f, 0.6f, 0.2f, 0.0f}}};
const pma::Vector<pma::Matrix<float>> DecodedV25::conditionalToValues = {
    {// MaxLOD-0 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f, 0.4f, 0.7f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f, 0.8f, 1.0f},
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.6f, 1.0f, 0.9f, 0.7f, 1.0f, 0.4f, 0.7f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f, 0.8f, 1.0f}}};
const pma::Vector<pma::Matrix<float>> DecodedV25::conditionalSlopeValues = {
    {// MaxLOD-0 - MinLOD-1
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f, 0.6f, 0.6f, 0.6f, 0.5f, 0.6f, 0.7f, 0.7f, 0.8f, 0.9f},
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f}},
    {// MaxLOD-1 - MinLOD-1
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f}},
    {// MaxLOD-0 - MinLOD-0
     {1.0f, 0.9f, 0.9f, 0.8f, 0.7f, 0.7f, 0.6f, 0.6f, 0.6f, 0.5f, 0.6f, 0.7f, 0.7f, 0.8f, 0.9f}}};
const pma::Vector<pma::Matrix<float>> DecodedV25::conditionalCutValues = {
    {// MaxLOD-0 - MinLOD-1
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f, 1.0f, 1.0f, 1.0f, 0.2f, 0.4f, 0.8f, 0.8f, 1.0f, 0.2f},
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f}},
    {// MaxLOD-1 - MinLOD-1
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f}},
    {// MaxLOD-0 - MinLOD-0
     {0.0f, 0.5f, 0.5f, 0.4f, 0.3f, 0.3f, 1.0f, 1.0f, 1.0f, 0.2f, 0.4f, 0.8f, 0.8f, 1.0f, 0.2f}}};
// Behavior->PSDs
const pma::Vector<std::uint16_t> DecodedV25::psdRowIndices = {8,  8,  8,  9,  9,  10, 10, 10, 11, 12, 13, 13,
                                                              13, 14, 14, 15, 16, 18, 18, 18, 18, 19, 19, 20};
const pma::Vector<std::uint16_t> DecodedV25::psdColumnIndices = {0, 3, 6, 2, 5, 2, 3, 7, 3, 2, 0, 1,
                                                                 2, 3, 6, 0, 4, 0, 3, 4, 5, 6, 7, 2};
const pma::Vector<float> DecodedV25::psdValues = {1.0f, 0.9f, 0.9f, 0.6f, 1.0f, 0.8f, 0.9f, 0.8f, 1.0f, 0.3f, 1.0f, 0.9f,
                                                  1.0f, 0.9f, 0.5f, 0.5f, 0.9f, 0.7f, 0.6f, 1.0f, 1.0f, 1.0f, 0.6f, 1.0f};
// Behavior->Joints
const pma::Vector<std::uint16_t> DecodedV25::jointRowCount = {
    81u,  // MaxLOD-0 - MinLOD-1
    54u,  // MaxLOD-1 - MinLOD-1
    81u   // MaxLOD-0 - MinLOD-0
};
const std::uint16_t DecodedV25::jointColumnCount = 10u;
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::jointVariableIndices = {
    {// MaxLOD-0 - MinLOD-1
     {2, 3, 4, 5, 12, 13, 14, 18, 20, 36, 38, 39, 40, 41, 45, 46, 48, 49, 50, 55, 56, 57, 58, 59, 63, 71},
     {2, 3, 4, 5, 12, 13, 14, 18, 20, 39, 40, 41, 48, 49, 50, 55, 56, 57, 58, 59}},
    {// MaxLOD-1 - MinLOD-1
     {2, 3, 4, 5, 12, 13, 14, 18, 20, 37, 38, 39, 40, 41}},
    {// MaxLOD-0 - MinLOD-0
     {2, 3, 4, 5, 12, 13, 14, 18, 20, 36, 38, 39, 40, 41, 45, 46, 48, 49, 50, 55, 56, 57, 58, 59, 63, 71}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::jointGroupLODs = {{
                                                                                // Joint Group 0
                                                                                {3, 3},  // MaxLOD-0 - MaxLOD-1
                                                                                {3},     // MaxLOD-1 - MaxLOD-1
                                                                                {3},     // MaxLOD-0 - MaxLOD-1
                                                                            },
                                                                            {
                                                                                // Joint group 1
                                                                                {4, 2},  // MaxLOD-0 - MaxLOD-1
                                                                                {2},     // MaxLOD-1 - MaxLOD-1
                                                                                {4}      // MaxLOD-0 - MaxLOD-0
                                                                            },
                                                                            {
                                                                                // Joint group 2
                                                                                {3, 2},  // MaxLOD-0 - MinLOD-1
                                                                                {2},     // MaxLOD-1 - MinLOD-1
                                                                                {3}      // MaxLOD-0 - MinLOD-0
                                                                            },
                                                                            {
                                                                                // Joint group 3
                                                                                {3, 0},  // MaxLOD-0 - MinLOD-1
                                                                                {0},     // MaxLOD-1 - MinLOD-1
                                                                                {3}      // MaxLOD-0 - MinLOD-0
                                                                            }};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV25::jointGroupInputIndices = {{ // Joint Group 0
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {0, 1, 2, 3, 6, 7, 8},
                                                                                                   {0, 1, 2, 3, 6, 7, 8}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {0, 1, 2, 3, 6, 7, 8}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {0, 1, 2, 3, 6, 7, 8}}},
                                                                                                 { // Joint Group 1
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {3, 4, 7, 8, 9},
                                                                                                   {3, 4, 7, 8, 9}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {3, 4, 7, 8, 9}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {3, 4, 7, 8, 9}}},
                                                                                                 { // Joint Group 2
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {4, 5, 8, 9},
                                                                                                   {4, 5, 8, 9}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {4, 5, 8, 9}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {4, 5, 8, 9}}},
                                                                                                 { // Joint Group 3
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {2, 5, 6, 8},
                                                                                                   {2, 5, 6, 8}},
                                                                                                  {// MaxLOD-1 - MaxLOD-1
                                                                                                   {}},
                                                                                                  {// MaxLOD-0 - MaxLOD-0
                                                                                                   {2, 5, 6, 8}}}};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV25::jointGroupOutputIndices = {{ // Joint Group 0
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {2, 3, 5},
                                                                                                    {2, 3, 5}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {2, 3, 5}},
                                                                                                   {
                                                                                                       // MaxLOD-0 - MaxLOD-0
                                                                                                       {2, 3, 5},
                                                                                                   }},
                                                                                                  { // Joint Group 1
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {18, 20, 36, 38},
                                                                                                    {18, 20}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {18, 20}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {18, 20, 36, 38}}},
                                                                                                  { // Joint Group 2
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {55, 56, 63},
                                                                                                    {55, 56}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {37, 38}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {55, 56, 63}}},
                                                                                                  { // Joint Group 3
                                                                                                   {// MaxLOD-0 - MaxLOD-1
                                                                                                    {45, 46, 71},
                                                                                                    {}},
                                                                                                   {// MaxLOD-1 - MaxLOD-1
                                                                                                    {}},
                                                                                                   {// MaxLOD-0 - MaxLOD-0
                                                                                                    {45, 46, 71}}}};
const pma::Vector<pma::Vector<pma::Matrix<float>>> DecodedV25::jointGroupValues = {
    { // Joint Group 0
     {// MaxLOD-0 - MaxLOD-1
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f},
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}},
     {// MaxLOD-1 - MinLOD-1
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}},
     {// MaxLOD-0 - MinLOD-0
      {0.00f, 0.05f, 0.10f, 0.15f, 0.20f, 0.25f, 0.30f, 0.35f, 0.40f, 0.45f, 0.50f,
       0.55f, 0.60f, 0.65f, 0.70f, 0.75f, 0.80f, 0.85f, 0.90f, 0.95f, 1.00f}}},
    { // Joint group 1
     {// MaxLOD-0 - MaxLOD-1
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f,
       0.11f, 0.12f, 0.13f, 0.14f, 0.15f, 0.16f, 0.17f, 0.18f, 0.19f, 0.20f},
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f}},
     {// MaxLOD-1 - MinLOD-1
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f}},
     {// MaxLOD-0 - MinLOD-0
      {0.01f, 0.02f, 0.03f, 0.04f, 0.05f, 0.06f, 0.07f, 0.08f, 0.09f, 0.10f,
       0.11f, 0.12f, 0.13f, 0.14f, 0.15f, 0.16f, 0.17f, 0.18f, 0.19f, 0.20f}}},
    { // Joint group 2
     {// MaxLOD-0 - MaxLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f},
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f}},
     {// MaxLOD-1 - MinLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f}},
     {// MaxLOD-0 - MinLOD-0
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f}}},
    { // Joint group 3
     {// MaxLOD-0 - MaxLOD-1
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f},
      {}},
     {// MaxLOD-1 - MinLOD-1
      {}},
     {// MaxLOD-0 - MinLOD-0
      {0.31f, 0.36f, 0.42f, 0.47f, 0.53f, 0.58f, 0.64f, 0.69f, 0.75f, 0.80f, 0.86f, 0.91f}}}};
const pma::Vector<pma::Vector<pma::Matrix<std::uint16_t>>> DecodedV25::jointGroupJointIndices = {{ // Joint Group 0
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {0},
                                                                                                   {0}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {0}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {0}}},
                                                                                                 { // Joint Group 1
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {2, 4},
                                                                                                   {2}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {2}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {2, 4}}},
                                                                                                 { // Joint Group 2
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {6, 7},
                                                                                                   {6}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {4}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {6, 7}}},
                                                                                                 { // Joint Group 3
                                                                                                  {// MaxLOD-0 - MaxLOD-1
                                                                                                   {5, 7},
                                                                                                   {}},
                                                                                                  {// MaxLOD-1 - MinLOD-1
                                                                                                   {}},
                                                                                                  {// MaxLOD-0 - MinLOD-0
                                                                                                   {5, 7}}}};
// Behavior->BlendShapes
const pma::Matrix<std::uint16_t> DecodedV25::blendShapeLODs = {{
    {7, 4},  // MaxLOD-0 - MaxLOD-1
    {4},     // MaxLOD-1 - MinLOD-1
    {7}      // MaxLOD-0 - MinLOD-0
}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::blendShapeInputIndices = {{// MaxLOD-0 - MaxLOD-1
                                                                                     {0, 1, 2, 3, 6, 7, 8},
                                                                                     {0, 1, 2, 3}},
                                                                                    {// MaxLOD-1 - MinLOD-1
                                                                                     {0, 1, 2, 3}},
                                                                                    {// MaxLOD-0 - MinLOD-0
                                                                                     {0, 1, 2, 3, 6, 7, 8}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::blendShapeOutputIndices = {{// MaxLOD-0 - MaxLOD-1
                                                                                      {0, 1, 2, 3, 6, 7, 8},
                                                                                      {0, 1, 2, 3}},
                                                                                     {// MaxLOD-1 - MinLOD-1
                                                                                      {0, 1, 2, 3}},
                                                                                     {// MaxLOD-0 - MinLOD-0
                                                                                      {0, 1, 2, 3, 6, 7, 8}}};
// Behavior->AnimatedMaps
const pma::Vector<std::uint16_t> DecodedV25::animatedMapCount = {
    10,  // MaxLOD-0 - MaxLOD-1
    4,   // MaxLOD-1 - MinLOD-1
    10   // MaxLOD-0 - MinLOD-0
};
const pma::Matrix<std::uint16_t> DecodedV25::animatedMapLODs = {{
    {15, 6},  // MaxLOD-0 - MaxLOD-1
    {6},      // MaxLOD-1 - MinLOD-1
    {15}      // MaxLOD-0 - MinLOD-0
}};
// Geometry
const pma::Vector<std::uint32_t> DecodedV25::meshCount = {
    3u,  // MaxLOD-0 - MaxLOD-1
    1u,  // MaxLOD-1 - MinLOD-1
    2u   // MaxLOD-0 - MinLOD-0
};
const pma::Vector<pma::Matrix<Vector3>> DecodedV25::vertexPositions = {{ // MaxLOD-0 - MaxLOD-1
                                                                        {// Mesh-0
                                                                         {7.0f, 7.0f, 7.0f},
                                                                         {8.0f, 8.0f, 8.0f},
                                                                         {9.0f, 9.0f, 9.0f}},
                                                                        {// Mesh-1
                                                                         {4.0f, 4.0f, 4.0f},
                                                                         {5.0f, 5.0f, 5.0f},
                                                                         {6.0f, 6.0f, 6.0f}},
                                                                        {// Mesh-2
                                                                         {1.0f, 1.0f, 1.0f},
                                                                         {2.0f, 2.0f, 2.0f},
                                                                         {3.0f, 3.0f, 3.0f}}},
                                                                       { // MaxLOD-1 - MinLOD-1
                                                                        {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                         {1.0f, 1.0f, 1.0f},
                                                                         {2.0f, 2.0f, 2.0f},
                                                                         {3.0f, 3.0f, 3.0f}}},
                                                                       { // MaxLOD-0 - MinLOD-0
                                                                        {// Mesh-0
                                                                         {7.0f, 7.0f, 7.0f},
                                                                         {8.0f, 8.0f, 8.0f},
                                                                         {9.0f, 9.0f, 9.0f}},
                                                                        {// Mesh-1
                                                                         {4.0f, 4.0f, 4.0f},
                                                                         {5.0f, 5.0f, 5.0f},
                                                                         {6.0f, 6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<TextureCoordinate>> DecodedV25::vertexTextureCoordinates = {{ // MaxLOD-0 - MaxLOD-1
                                                                                           {// Mesh-0
                                                                                            {7.0f, 7.0f},
                                                                                            {8.0f, 8.0f},
                                                                                            {9.0f, 9.0f}},
                                                                                           {// Mesh-1
                                                                                            {4.0f, 4.0f},
                                                                                            {5.0f, 5.0f},
                                                                                            {6.0f, 6.0f}},
                                                                                           {// Mesh-2
                                                                                            {1.0f, 1.0f},
                                                                                            {2.0f, 2.0f},
                                                                                            {3.0f, 3.0f}}},
                                                                                          { // MaxLOD-1 - MinLOD-1
                                                                                           {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                            {1.0f, 1.0f},
                                                                                            {2.0f, 2.0f},
                                                                                            {3.0f, 3.0f}}},
                                                                                          { // MaxLOD-0 - MinLOD-0
                                                                                           {// Mesh-0
                                                                                            {7.0f, 7.0f},
                                                                                            {8.0f, 8.0f},
                                                                                            {9.0f, 9.0f}},
                                                                                           {// Mesh-1
                                                                                            {4.0f, 4.0f},
                                                                                            {5.0f, 5.0f},
                                                                                            {6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<Vector3>> DecodedV25::vertexNormals = {{ // MaxLOD-0 - MaxLOD-1
                                                                      {// Mesh-0
                                                                       {7.0f, 7.0f, 7.0f},
                                                                       {8.0f, 8.0f, 8.0f},
                                                                       {9.0f, 9.0f, 9.0f}},
                                                                      {// Mesh-1
                                                                       {4.0f, 4.0f, 4.0f},
                                                                       {5.0f, 5.0f, 5.0f},
                                                                       {6.0f, 6.0f, 6.0f}},
                                                                      {// Mesh-2
                                                                       {1.0f, 1.0f, 1.0f},
                                                                       {2.0f, 2.0f, 2.0f},
                                                                       {3.0f, 3.0f, 3.0f}}},
                                                                     { // MaxLOD-1 - MinLOD-1
                                                                      {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                       {1.0f, 1.0f, 1.0f},
                                                                       {2.0f, 2.0f, 2.0f},
                                                                       {3.0f, 3.0f, 3.0f}}},
                                                                     { // MaxLOD-0 - MinLOD-0
                                                                      {// Mesh-0
                                                                       {7.0f, 7.0f, 7.0f},
                                                                       {8.0f, 8.0f, 8.0f},
                                                                       {9.0f, 9.0f, 9.0f}},
                                                                      {// Mesh-1
                                                                       {4.0f, 4.0f, 4.0f},
                                                                       {5.0f, 5.0f, 5.0f},
                                                                       {6.0f, 6.0f, 6.0f}}}};
const pma::Vector<pma::Matrix<VertexLayout>> DecodedV25::vertexLayouts = {{ // MaxLOD-0 - MaxLOD-1
                                                                           {// Mesh-0
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-1
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-2
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}},
                                                                          { // MaxLOD-1 - MinLOD-1
                                                                           {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}},
                                                                          { // MaxLOD-0 - MinLOD-0
                                                                           {// Mesh-0
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}},
                                                                           {// Mesh-1
                                                                            {0, 0, 0},
                                                                            {1, 1, 1},
                                                                            {2, 2, 2}}}};
const pma::Matrix<pma::Matrix<std::uint32_t>> DecodedV25::faces = {{ // MaxLOD-0 - MaxLOD-1
                                                                    {// Mesh-0
                                                                     {0, 1, 2}},
                                                                    {// Mesh-1
                                                                     {0, 1, 2}},
                                                                    {// Mesh-2
                                                                     {0, 1, 2}}},
                                                                   { // MaxLOD-1 - MinLOD-1
                                                                    {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                     {0, 1, 2}}},
                                                                   { // MaxLOD-0 - MinLOD-0
                                                                    {// Mesh-0
                                                                     {0, 1, 2}},
                                                                    {// Mesh-1
                                                                     {0, 1, 2}}}};
const pma::Matrix<std::uint16_t> DecodedV25::maxInfluencePerVertex = {{
                                                                          // MaxLOD-0 - MaxLOD-1
                                                                          8u,  // Mesh-0
                                                                          8u,  // Mesh-1
                                                                          8u   // Mesh-2
                                                                      },
                                                                      {
                                                                          // MaxLOD-1 - MinLOD-1
                                                                          8u  // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                      },
                                                                      {
                                                                          // MaxLOD-0 - MinLOD-0
                                                                          8u,  // Mesh-0
                                                                          8u   // Mesh-1
                                                                      }};
const pma::Matrix<pma::Matrix<float>> DecodedV25::skinWeightsValues = {{ // MaxLOD-0 - MinLOD-1
                                                                        {// Mesh-0
                                                                         {0.7f, 0.1f, 0.2f},
                                                                         {0.5f, 0.5f},
                                                                         {0.4f, 0.6f}},
                                                                        {// Mesh-1
                                                                         {0.4f, 0.3f, 0.3f},
                                                                         {0.8f, 0.2f},
                                                                         {0.1f, 0.9f}},
                                                                        {// Mesh-2
                                                                         {0.1f, 0.3f, 0.6f},
                                                                         {0.3f, 0.7f},
                                                                         {0.2f, 0.8f}}},
                                                                       {// MaxLOD-1 - MinLOD-1
                                                                        {
                                                                            // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                            {0.1f, 0.3f, 0.6f},
                                                                            {1.0f},  // 0.3f normalized to 1.0f
                                                                            {1.0f}   // 0.8f normalized to 1.0f
                                                                        }},
                                                                       { // MaxLOD-0 - MinLOD-0
                                                                        {// Mesh-0
                                                                         {0.7f, 0.1f, 0.2f},
                                                                         {0.5f, 0.5f},
                                                                         {0.4f, 0.6f}},
                                                                        {// Mesh-1
                                                                         {0.4f, 0.3f, 0.3f},
                                                                         {0.8f, 0.2f},
                                                                         {0.1f, 0.9f}}}};
const pma::Matrix<pma::Matrix<std::uint16_t>> DecodedV25::skinWeightsJointIndices = {{ // MaxLOD-0 - MaxLOD-1
                                                                                      {// Mesh-0
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-1
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-2
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}}},
                                                                                     { // MaxLOD-1 - MinLOD-1
                                                                                      {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                       {0, 1, 2},
                                                                                       {3},
                                                                                       {4}}},
                                                                                     { // MaxLOD-0 - MinLOD-0
                                                                                      {// Mesh-0
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}},
                                                                                      {// Mesh-1
                                                                                       {0, 1, 2},
                                                                                       {3, 4},
                                                                                       {5, 6}}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::correctiveBlendShapeIndices = {{ // MaxLOD-0 - MaxLOD-1
                                                                                          {// Mesh-0
                                                                                           2},
                                                                                          {// Mesh-1
                                                                                           2},
                                                                                          {// Mesh-2
                                                                                           2,
                                                                                           3}},
                                                                                         { // MaxLOD-1 - MinLOD-1
                                                                                          {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                           2}},
                                                                                         { // MaxLOD-0 - MinLOD-0
                                                                                          {// Mesh-0
                                                                                           2},
                                                                                          {// Mesh-1
                                                                                           2}}};
const pma::Matrix<pma::Matrix<Vector3>> DecodedV25::correctiveBlendShapeDeltas = {{  // MaxLOD-0 - MaxLOD-1
                                                                                   { // Mesh-0
                                                                                    {// Blendshape-0
                                                                                     {7.0f, 7.0f, 7.0f},
                                                                                     {8.0f, 8.0f, 8.0f},
                                                                                     {9.0f, 9.0f, 9.0f}}},
                                                                                   { // Mesh-1
                                                                                    {// Blendshape-0
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f},
                                                                                     {6.0f, 6.0f, 6.0f}}},
                                                                                   { // Mesh-2
                                                                                    {// Blendshape-0
                                                                                     {1.0f, 1.0f, 1.0f},
                                                                                     {2.0f, 2.0f, 2.0f},
                                                                                     {3.0f, 3.0f, 3.0f}},
                                                                                    {// Blendshape-1
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f}}}},
                                                                                  {  // MaxLOD-1 - MinLOD-1
                                                                                   { // Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                    {// Blendshape-0
                                                                                     {1.0f, 1.0f, 1.0f},
                                                                                     {2.0f, 2.0f, 2.0f},
                                                                                     {3.0f, 3.0f, 3.0f}}}},
                                                                                  {  // MaxLOD-0 - MinLOD-0
                                                                                   { // Mesh-0
                                                                                    {// Blendshape-0
                                                                                     {7.0f, 7.0f, 7.0f},
                                                                                     {8.0f, 8.0f, 8.0f},
                                                                                     {9.0f, 9.0f, 9.0f}}},
                                                                                   { // Mesh-1
                                                                                    {// Blendshape-0
                                                                                     {4.0f, 4.0f, 4.0f},
                                                                                     {5.0f, 5.0f, 5.0f},
                                                                                     {6.0f, 6.0f, 6.0f}}}}};
const pma::Matrix<pma::Matrix<std::uint32_t>> DecodedV25::correctiveBlendShapeVertexIndices = {
    {// MaxLOD-0 - MaxLOD-1
     {
         // Mesh-0
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-1
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-2
         {0, 1, 2},  // Blendshape-0
         {0, 2}      // Blendshape-1
     }},
    {// MaxLOD-1 - MinLOD-1
     {
         // Mesh-0 (Mesh-2 under MaxLOD-0)
         {0, 1, 2}  // Blendshape-0
     }},
    {// MaxLOD-0 - MinLOD-0
     {
         // Mesh-0
         {0, 1, 2},  // Blendshape-0
     },
     {
         // Mesh-1
         {0, 1, 2},  // Blendshape-0
     }}};
// Machine learned behavior
const pma::Vector<pma::String<char>> DecodedV25::mlControlNames = {"MA", "MB", "MC", "MD", "ME", "MF", "MG", "MH", "MI"};
const pma::Matrix<std::uint16_t> DecodedV25::neuralNetworkIndicesPerLOD = {{
                                                                               // MaxLOD-0 - MaxLOD-1
                                                                               0,  // Mesh-0 Region-0
                                                                               1,  // Mesh-0 Region-1
                                                                               2,  // Mesh-1 Region-0
                                                                               3,  // Mesh-1 Region-1
                                                                               4,  // Mesh-2 Region-0
                                                                               5   // Mesh-2 Region-1
                                                                           },
                                                                           {
                                                                               // MaxLOD-1 - MinLOD-1
                                                                               0,  // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                               1   // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
                                                                           },
                                                                           {
                                                                               // MaxLOD-0 - MinLOD-0
                                                                               0,  // Mesh-0 Region-0
                                                                               1,  // Mesh-0 Region-1
                                                                               2,  // Mesh-1 Region-0
                                                                               3   // Mesh-1 Region-1
                                                                           }};
const DecodedV25::VectorOfCharStringMatrix DecodedV25::regionNames = {{ // MaxLOD-0 - MinLOD-1
                                                                       {// Mesh-0
                                                                        "RA",
                                                                        "RB"},
                                                                       {// Mesh-1
                                                                        "RC",
                                                                        "RD"},
                                                                       {// Mesh-2
                                                                        "RE",
                                                                        "RF"}},
                                                                      { // MaxLOD-1 - MinLOD-1
                                                                       {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                        "RE",
                                                                        "RF"}},
                                                                      { // MaxLOD-0 - MinLOD-0
                                                                       {// Mesh-0
                                                                        "RA",
                                                                        "RB"},
                                                                       {// Mesh-1
                                                                        "RC",
                                                                        "RD"}}};
const pma::Matrix<pma::Matrix<std::uint16_t>> DecodedV25::neuralNetworkIndicesPerMeshRegion = {{  // MaxLOD-0 - MaxLOD-1
                                                                                                { // Mesh-0
                                                                                                 {// Region-0
                                                                                                  0},
                                                                                                 {// Region-1
                                                                                                  1}},
                                                                                                { // Mesh-1
                                                                                                 {// Region-0
                                                                                                  2},
                                                                                                 {// Region-1
                                                                                                  3}},
                                                                                                { // Mesh-2
                                                                                                 {// Region-0
                                                                                                  4},
                                                                                                 {// Region-1
                                                                                                  5}}},
                                                                                               { // MaxLOD-1 - MinLOD-1
                                                                                                {// Mesh-0 (Mesh-2 under MaxLOD-0)
                                                                                                 {
                                                                                                     // Region-0
                                                                                                     0  // (4 under MaxLOD-0)
                                                                                                 },
                                                                                                 {
                                                                                                     // Region-1
                                                                                                     1  // (5 under MaxLOD-0)
                                                                                                 }}},
                                                                                               {  // MaxLOD-0 - MinLOD-0
                                                                                                { // Mesh-0
                                                                                                 {// Region-0
                                                                                                  0},
                                                                                                 {// Region-1
                                                                                                  1}},
                                                                                                { // Mesh-1
                                                                                                 {// Region-0
                                                                                                  2},
                                                                                                 {// Region-1
                                                                                                  3}}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::neuralNetworkInputIndices = {
    { // MaxLOD-0 - MaxLOD-1
     {// Mesh-0 Region-0
      0,
      1},
     {// Mesh-0 Region-1
      2,
      3},
     {// Mesh-1 Region-0
      4,
      5},
     {// Mesh-1 Region-1
      6,
      7},
     {// Mesh-2 Region-0
      8,
      0},
     {// Mesh-2 Region-1
      4,
      7}},
    { // MaxLOD-1 - MinLOD-1
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
      8,
      0},
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
      4,
      7}},
    { // MaxLOD-0 - MinLOD-0
     {// Mesh-0 Region-0
      0,
      1},
     {// Mesh-0 Region-1
      2,
      3},
     {// Mesh-1 Region-0
      4,
      5},
     {// Mesh-1 Region-1
      6,
      7}}};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::neuralNetworkOutputIndices = {
    { // MaxLOD-0 - MaxLOD-1
     {// Mesh-0 Region-0
      9},
     {// Mesh-0 Region-1
      10},
     {// Mesh-1 Region-0
      11},
     {// Mesh-1 Region-1
      12},
     {// Mesh-2 Region-0
      13},
     {// Mesh-2 Region-1
      14}},
    { // MaxLOD-1 - MinLOD-1
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
      13},
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
      14}},
    { // MaxLOD-0 - MinLOD-0
     {// Mesh-0 Region-0
      9},
     {// Mesh-0 Region-1
      10},
     {// Mesh-1 Region-0
      11},
     {// Mesh-1 Region-1
      12}}};
const pma::Matrix<std::uint16_t> DecodedV25::neuralNetworkLayerCount = {{
                                                                            // MaxLOD-0 - MaxLOD-1
                                                                            2,  // Mesh-0 Region-0
                                                                            2,  // Mesh-0 Region-1
                                                                            2,  // Mesh-1 Region-0
                                                                            2,  // Mesh-1 Region-1
                                                                            2,  // Mesh-2 Region-0
                                                                            2   // Mesh-2 Region-1
                                                                        },
                                                                        {
                                                                            // MaxLOD-1 - MinLOD-1
                                                                            2,  // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                            2   // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
                                                                        },
                                                                        {
                                                                            // MaxLOD-0 - MinLOD-0
                                                                            2,  // Mesh-0 Region-0
                                                                            2,  // Mesh-0 Region-1
                                                                            2,  // Mesh-1 Region-0
                                                                            2   // Mesh-1 Region-1
                                                                        }};
const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::neuralNetworkActivationFunction = {
    { // MaxLOD-0 - MaxLOD-1
     {// Mesh-0 Region-0
      1,
      1},
     {// Mesh-0 Region-1
      1,
      1},
     {// Mesh-1  Region-0
      1,
      1},
     {// Mesh-1 Region-1
      1,
      1},
     {// Mesh-2 Region-0
      1,
      1},
     {// Mesh-2 Region-1
      1,
      1}},
    { // MaxLOD-1 - MinLOD-1
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
      1,
      1},
     {// Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
      1,
      1}},
    { // MaxLOD-0 - MinLOD-0
     {// Mesh-0  Region-0
      1,
      1},
     {// Mesh-0 Region-1
      1,
      1},
     {// Mesh-1 Region-0
      1,
      1},
     {// Mesh-1 Region-1
      1,
      1}}};
const pma::Matrix<pma::Matrix<float>> DecodedV25::neuralNetworkActivationFunctionParameters = {
    {// MaxLOD-0 - MaxLOD-1
     {
         // Mesh-0  Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-0 Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-0
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-1
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-2 Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-2 Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     }},
    {// MaxLOD-1 - MinLOD-1
     {
         // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     }},
    {// MaxLOD-0 - MinLOD-0
     {
         // Mesh-0 Region-0
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     },
     {
         // Mesh-0 Region-1
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-0
         {1.0f},  // Layer-0
         {1.0f}   // Layer-1
     },
     {
         // Mesh-1 Region-1
         {0.5f},  // Layer-0
         {0.5f}   // Layer-1
     }}};
const pma::Matrix<pma::Matrix<float>> DecodedV25::neuralNetworkBiases = {{// MaxLOD-0 - MaxLOD-1
                                                                          {
                                                                              // Mesh-0 Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-0 Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-0
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-1
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-2 Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-2 Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          }},
                                                                         {// MaxLOD-1 - MinLOD-1
                                                                          {
                                                                              // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-0 (Mesh-2 under MaxLOD-0) Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          }},
                                                                         {// MaxLOD-0 - MinLOD-0
                                                                          {
                                                                              // Mesh-0 Region-0
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-0 Region-1
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-0
                                                                              {0.5f, 0.5f},  // Layer-0
                                                                              {0.5f}         // Layer-1
                                                                          },
                                                                          {
                                                                              // Mesh-1 Region-1
                                                                              {1.0f, 1.0f},  // Layer-0
                                                                              {1.0f}         // Layer-1
                                                                          }}};
const pma::Matrix<pma::Matrix<float>> DecodedV25::neuralNetworkWeights = {{// MaxLOD-0 - MaxLOD-1
                                                                           {
                                                                               // Mesh-0 Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-0 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-0
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-1
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-2 Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-2 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           }},
                                                                          {// MaxLOD-1 - MinLOD-1
                                                                           {
                                                                               // Mesh-0 (Mesh-2 under MaxLOD-0) Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-0 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           }},
                                                                          {// MaxLOD-0 - MinLOD-0
                                                                           {
                                                                               // Mesh-0 Region-0
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-0 Region-1
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-0
                                                                               {1.0f, 1.0f, 1.0f, 1.0f},  // Layer-0
                                                                               {1.0f, 1.0f}               // Layer-1
                                                                           },
                                                                           {
                                                                               // Mesh-1 Region-1
                                                                               {0.5f, 0.5f, 0.5f, 0.5f},  // Layer-0
                                                                               {0.5f, 0.5f}               // Layer-1
                                                                           }}};

const pma::Matrix<std::uint16_t> DecodedV25::solverIndicesPerLOD{{// MaxLOD-0 - MinLOD-1
                                                                  0,
                                                                  1,
                                                                  2},
                                                                 {// MaxLOD-1 - MinLOD-1
                                                                  1,
                                                                  2},
                                                                 {// MaxLOD-0 - MinLOD-0
                                                                  0,
                                                                  1}};

const pma::Vector<pma::String<char>> DecodedV25::solverNames{"RSA", "RSB", "RSC"};

const pma::Matrix<std::uint16_t> DecodedV25::solverRawControlIndices{// Solver 0
                                                                     {11, 12},
                                                                     // Solver 1
                                                                     {3},
                                                                     // Solver 2
                                                                     {22, 23}};

const pma::Matrix<std::uint16_t> DecodedV25::solverPoseIndices{
    // Solver 0
    {0, 1, 2},
    // Solver 1
    {3, 4},
    // Solver 2
    {5, 6, 7},
};

const pma::Vector<float> DecodedV25::solverRadius{1.0f, 2.0f, 1.0f};

const pma::Vector<float> DecodedV25::solverWeightThreshold{1.0f, 2.0f, 1.0f};

const pma::Vector<std::uint16_t> DecodedV25::solverType{0, 1, 0};

const pma::Vector<std::uint16_t> DecodedV25::solverAutomaticRadius{0, 0, 0};

const pma::Vector<std::uint16_t> DecodedV25::solverDistanceMethod{1, 3, 1};

const pma::Vector<std::uint16_t> DecodedV25::solverNormalizeMethod{0, 1, 0};

const pma::Vector<std::uint16_t> DecodedV25::solverFunctionType{2, 2, 0};

const pma::Vector<std::uint16_t> DecodedV25::solverTwistAxis{0, 1, 0};

const pma::Vector<pma::String<char>> DecodedV25::poseNames{"RA", "RB", "RC", "RD", "RE", "RF", "RG", "RH"};

const Vector<float> DecodedV25::poseScale{
    0.0f,  // Pose 0 (RA)
    1.0f,  // Pose 1 (RB)
    2.0f,  // Pose 2 (RC)
    2.0f,  // Pose 3 (RD)
    1.0f,  // Pose 4 (RE)
    1.0f,  // Pose 5 (RF)
    1.0f,  // Pose 6 (RG)
    0.5f   // Pose 7 (RH)
};

const pma::Vector<std::uint16_t> DecodedV25::poseDistanceMethod{
    0,  // Pose 0 (RA)
    1,  // Pose 1 (RB)
    2,  // Pose 2 (RC)
    3,  // Pose 0 (RD)
    4,  // Pose 1 (RE)
    1,  // Pose 0 (RF)
    2,  // Pose 1 (RG)
    2   // Pose 2 (RH)

};

const pma::Vector<std::uint16_t> DecodedV25::poseFunctionType{
    5,  // Pose 0 (RA)
    4,  // Pose 1 (RB)
    3,  // Pose 2 (RC)
    2,  // Pose 3 (RD)
    1,  // Pose 4 (RE)
    0,  // Pose 5 (RF)
    1,  // Pose 6 (RG)
    2   // Pose 7 (RH)
};

const Matrix<float> DecodedV25::solverRawControlValues{
    {
        // Solver 0
        // Pose 0 (RA)
        2.0f,   // Raw control index 11
        0.0f,   // Raw control index 12
                // Pose 1 (RB)
        1.0f,   // Raw control index 11
        1.0f,   // Raw control index 12
                // Pose 2 (RC)
        3.0f,   // Raw control index 11
        -3.0f,  // Raw control index 12
    },
    {
        // Solver 1
        // Pose 0 (RD)
        0.0f,  // Raw control index 3
               // Pose 1 (RE)
        4.0f,  // Raw control index 3
    },
    {
        // Solver 2
        // Pose 0 (RF)
        2.0f,   // Raw control index 22
        0.0f,   // Raw control index 23
                // Pose 1 (RG)
        1.0f,   // Raw control index 22
        1.0f,   // Raw control index 23
                // Pose 2 (RH)
        3.0f,   // Raw control index 22
        -3.0f,  // Raw control index 23
    },
};

const pma::Vector<pma::String<char>> DecodedV25::poseControlNames = {"PA", "PB", "PC", "PD", "PE", "PF", "PG", "PH", "PI"};

const Matrix<std::uint16_t> DecodedV25::poseInputControlIndices = {
    {0},
    {1},
    {2},
    {3},
    {4},
    {5},
    {6},
    {7},
};

const Matrix<std::uint16_t> DecodedV25::poseOutputControlIndices = {
    {8},
    {9},
    {10},
    {11},
    {12},
    {13},
    {14},
    {15, 16},
};

const Matrix<float> DecodedV25::poseOutputControlWeights = {{1.0f}, {1.0f}, {1.0f}, {1.0f}, {1.0f}, {1.0f}, {1.0f}, {0.5f, 0.5f}};

const pma::Matrix<TranslationRepresentation> DecodedV25::jointTranslationRepresentation = {{// MaxLOD-0 - MinLOD-1
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector},
                                                                                           {
                                                                                               // MaxLOD-1 - MinLOD-1
                                                                                               TranslationRepresentation::Vector,
                                                                                               TranslationRepresentation::Vector,
                                                                                               TranslationRepresentation::Vector,
                                                                                               TranslationRepresentation::Vector,
                                                                                               TranslationRepresentation::Vector,
                                                                                               TranslationRepresentation::Vector,
                                                                                           },
                                                                                           {// MaxLOD-0 - MinLOD-0
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector,
                                                                                            TranslationRepresentation::Vector}};

const pma::Matrix<RotationRepresentation> DecodedV25::jointRotationRepresentation = {
    {
        // MaxLOD-0 - MinLOD-1
        RotationRepresentation::EulerAngles,  // JA
        RotationRepresentation::EulerAngles,  // JB
        RotationRepresentation::Quaternion,   // JC
        RotationRepresentation::Quaternion,   // JD
        RotationRepresentation::EulerAngles,  // JE
        RotationRepresentation::EulerAngles,  // JF
        RotationRepresentation::EulerAngles,  // JG
        RotationRepresentation::Quaternion,   // JH
        RotationRepresentation::EulerAngles   // JI
    },
    {
        // MaxLOD-1 - MinLOD-1
        RotationRepresentation::EulerAngles,  // JA
        RotationRepresentation::EulerAngles,  // JB
        RotationRepresentation::Quaternion,   // JC
        RotationRepresentation::Quaternion,   // JD
        RotationRepresentation::EulerAngles,  // JG
        RotationRepresentation::EulerAngles   // JI
    },
    {
        // MaxLOD-0 - MinLOD-0
        RotationRepresentation::EulerAngles,  // JA
        RotationRepresentation::EulerAngles,  // JB
        RotationRepresentation::Quaternion,   // JC
        RotationRepresentation::Quaternion,   // JD
        RotationRepresentation::EulerAngles,  // JE
        RotationRepresentation::EulerAngles,  // JF
        RotationRepresentation::EulerAngles,  // JG
        RotationRepresentation::Quaternion,   // JH
        RotationRepresentation::EulerAngles   // JI
    }};

const pma::Matrix<ScaleRepresentation> DecodedV25::jointScaleRepresentation = {{// MaxLOD-0 - MinLOD-1
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector},
                                                                               {
                                                                                   // MaxLOD-1 - MinLOD-1
                                                                                   ScaleRepresentation::Vector,
                                                                                   ScaleRepresentation::Vector,
                                                                                   ScaleRepresentation::Vector,
                                                                                   ScaleRepresentation::Vector,
                                                                                   ScaleRepresentation::Vector,
                                                                                   ScaleRepresentation::Vector,
                                                                               },
                                                                               {// MaxLOD-0 - MinLOD-0
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector,
                                                                                ScaleRepresentation::Vector}};

const pma::Vector<pma::Matrix<float>> DecodedV25::swingBlendWeights = {{// MaxLOD-0 - MinLOD-1
                                                                        {1.0f, 2.0f},
                                                                        {-2.0f, -1.0f},
                                                                        {1.0f}},
                                                                       {
                                                                           // MaxLOD-1 - MinLOD-0
                                                                           {1.0f, 2.0f},
                                                                           {-1.0f},
                                                                       },
                                                                       {// MaxLOD-0 - MinLOD-0
                                                                        {1.0f, 2.0f},
                                                                        {-2.0f, -1.0f},
                                                                        {1.0f}}};

const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::swingOutputJointIndices = {{// MaxLOD-0 - MinLOD-1
                                                                                      {0u, 1u},
                                                                                      {4u, 6u},
                                                                                      {5u}},
                                                                                     {// MaxLOD-1 - MinLOD-0
                                                                                      {0u, 1u},
                                                                                      {4u}},
                                                                                     {// MaxLOD-0 - MinLOD-0
                                                                                      {0u, 1u},
                                                                                      {4u, 6u},
                                                                                      {5u}}};

const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::swingInputControlIndices = {{// MaxLOD-0 - MinLOD-1
                                                                                       {5u, 6u, 7u, 8u},
                                                                                       {11u, 12u, 13u, 14u},
                                                                                       {27u, 28u, 29u, 30u}},
                                                                                      {// MaxLOD-1 - MinLOD-0
                                                                                       {5u, 6u, 7u, 8u},
                                                                                       {11u, 12u, 13u, 14u},
                                                                                       {27u, 28u, 29u, 30u}},
                                                                                      {// MaxLOD-0 - MinLOD-0
                                                                                       {5u, 6u, 7u, 8u},
                                                                                       {11u, 12u, 13u, 14u},
                                                                                       {27u, 28u, 29u, 30u}}};

const pma::Matrix<TwistAxis> DecodedV25::swingTwistAxes = {{// MaxLOD-0 - MinLOD-1
                                                            TwistAxis::X,
                                                            TwistAxis::Y,
                                                            TwistAxis::Z},
                                                           {// MaxLOD-1 - MinLOD-0
                                                            TwistAxis::X,
                                                            TwistAxis::Y,
                                                            TwistAxis::Z},
                                                           {// MaxLOD-0 - MinLOD-0
                                                            TwistAxis::X,
                                                            TwistAxis::Y,
                                                            TwistAxis::Z}};

const pma::Vector<pma::Matrix<float>> DecodedV25::twistBlendWeights = {{// MaxLOD-0 - MinLOD-1
                                                                        {1.0f, 2.0f},
                                                                        {-2.0f, -1.0f},
                                                                        {1.0f}},
                                                                       {
                                                                           // MaxLOD-1 - MinLOD-0
                                                                           {1.0f, 2.0f},
                                                                           {-1.0f},
                                                                       },
                                                                       {// MaxLOD-0 - MinLOD-0
                                                                        {1.0f, 2.0f},
                                                                        {-2.0f, -1.0f},
                                                                        {1.0f}}};

const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::twistOutputJointIndices = {{// MaxLOD-0 - MinLOD-1
                                                                                      {0u, 1u},
                                                                                      {4u, 6u},
                                                                                      {5u}},
                                                                                     {// MaxLOD-1 - MinLOD-0
                                                                                      {0u, 1u},
                                                                                      {4u}},
                                                                                     {// MaxLOD-0 - MinLOD-0
                                                                                      {0u, 1u},
                                                                                      {4u, 6u},
                                                                                      {5u}}};

const pma::Vector<pma::Matrix<std::uint16_t>> DecodedV25::twistInputControlIndices = {{// MaxLOD-0 - MinLOD-1
                                                                                       {5u, 6u, 7u, 8u},
                                                                                       {11u, 12u, 13u, 14u},
                                                                                       {27u, 28u, 29u, 30u}},
                                                                                      {// MaxLOD-1 - MinLOD-0
                                                                                       {5u, 6u, 7u, 8u},
                                                                                       {11u, 12u, 13u, 14u},
                                                                                       {27u, 28u, 29u, 30u}},
                                                                                      {// MaxLOD-0 - MinLOD-0
                                                                                       {5u, 6u, 7u, 8u},
                                                                                       {11u, 12u, 13u, 14u},
                                                                                       {27u, 28u, 29u, 30u}}};

const pma::Matrix<TwistAxis> DecodedV25::twistTwistAxes = {{// MaxLOD-0 - MinLOD-1
                                                            TwistAxis::X,
                                                            TwistAxis::Y,
                                                            TwistAxis::Z},
                                                           {// MaxLOD-1 - MinLOD-0
                                                            TwistAxis::X,
                                                            TwistAxis::Y,
                                                            TwistAxis::Z},
                                                           {// MaxLOD-0 - MinLOD-0
                                                            TwistAxis::X,
                                                            TwistAxis::Y,
                                                            TwistAxis::Z}};

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

std::size_t DecodedV25::lodConstraintToIndex(std::uint16_t maxLOD, std::uint16_t minLOD) {
    // Relies on having only TWO available LODs (0, 1)
    return (minLOD == 1u ? maxLOD : 2ul);
}

RawJoints DecodedV25::getJoints(std::uint16_t currentMaxLOD, std::uint16_t currentMinLOD, pma::MemoryResource* memRes) {
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    RawJoints joints{memRes};
    joints.rowCount = jointRowCount[srcIndex];
    joints.colCount = jointColumnCount;
    for (std::size_t i = 0ul; i < jointGroupLODs.size(); ++i) {
        RawJointGroup jntGrp{memRes};
        jntGrp.lods.assign(jointGroupLODs[i][srcIndex].begin(), jointGroupLODs[i][srcIndex].end());
        jntGrp.inputIndices.assign(jointGroupInputIndices[i][srcIndex][0ul].begin(),
                                   jointGroupInputIndices[i][srcIndex][0ul].end());
        jntGrp.outputIndices.assign(jointGroupOutputIndices[i][srcIndex][0ul].begin(),
                                    jointGroupOutputIndices[i][srcIndex][0ul].end());
        jntGrp.values.assign(jointGroupValues[i][srcIndex][0ul].begin(), jointGroupValues[i][srcIndex][0ul].end());
        jntGrp.jointIndices.assign(jointGroupJointIndices[i][srcIndex][0ul].begin(),
                                   jointGroupJointIndices[i][srcIndex][0ul].end());
        joints.jointGroups.push_back(std::move(jntGrp));
    }

    return joints;
}

RawBlendShapeChannels DecodedV25::getBlendShapes(std::uint16_t currentMaxLOD,
                                                 std::uint16_t currentMinLOD,
                                                 pma::MemoryResource* memRes) {
    RawBlendShapeChannels blendShapes{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    blendShapes.lods.assign(blendShapeLODs[srcIndex].begin(), blendShapeLODs[srcIndex].end());
    blendShapes.inputIndices.assign(blendShapeInputIndices[srcIndex][0ul].begin(), blendShapeInputIndices[srcIndex][0ul].end());
    blendShapes.outputIndices.assign(blendShapeOutputIndices[srcIndex][0ul].begin(),
                                     blendShapeOutputIndices[srcIndex][0ul].end());
    return blendShapes;
}

RawConditionalTable DecodedV25::getConditionals(std::uint16_t currentMaxLOD,
                                                std::uint16_t currentMinLOD,
                                                pma::MemoryResource* memRes) {
    RawConditionalTable conditionals{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    conditionals.inputIndices.assign(conditionalInputIndices[srcIndex][0ul].begin(),
                                     conditionalInputIndices[srcIndex][0ul].end());
    conditionals.outputIndices.assign(conditionalOutputIndices[srcIndex][0ul].begin(),
                                      conditionalOutputIndices[srcIndex][0ul].end());
    conditionals.fromValues.assign(conditionalFromValues[srcIndex][0ul].begin(), conditionalFromValues[srcIndex][0ul].end());
    conditionals.toValues.assign(conditionalToValues[srcIndex][0ul].begin(), conditionalToValues[srcIndex][0ul].end());
    conditionals.slopeValues.assign(conditionalSlopeValues[srcIndex][0ul].begin(), conditionalSlopeValues[srcIndex][0ul].end());
    conditionals.cutValues.assign(conditionalCutValues[srcIndex][0ul].begin(), conditionalCutValues[srcIndex][0ul].end());
    return conditionals;
}

RawAnimatedMaps DecodedV25::getAnimatedMaps(std::uint16_t currentMaxLOD,
                                            std::uint16_t currentMinLOD,
                                            pma::MemoryResource* memRes) {
    RawAnimatedMaps animatedMaps{memRes};
    const auto srcIndex = lodConstraintToIndex(currentMaxLOD, currentMinLOD);
    animatedMaps.lods.assign(animatedMapLODs[srcIndex].begin(), animatedMapLODs[srcIndex].end());
    animatedMaps.conditionals = getConditionals(currentMaxLOD, currentMinLOD, memRes);
    return animatedMaps;
}

}  // namespace dna
