// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnatests/Defs.h"

#include "dna/DNA.h"
#include "dna/Reader.h"

#include <pma/TypeDefs.h>

namespace dna {

struct RawV28 {
    static const unsigned char header[187];
    static const unsigned char descriptor[87];
    static const unsigned char definition[794];
    static const unsigned char controls[2];
    static const unsigned char conditionals[324];
    static const unsigned char psds[204];
    static const unsigned char joints[444];
    static const unsigned char blendshapes[44];
    static const unsigned char animatedmaps[8];
    static const unsigned char geometry[1080];
    static const unsigned char machineLearnedBehavior[762];
    static const unsigned char rbfBehavior[327];
    static const unsigned char rbfBehaviorExt[228];
    static const unsigned char jointBehaviorMetadata[58];
    static const unsigned char twistSwingBehavior[200];
    static const unsigned char machineLearnedBehaviorExt[448];
    static const unsigned char descriptorExt[17];

    static std::vector<char> getBytes();
};

struct RawV27DowngradedFromV28 {
    static std::vector<char> getBytes();
};

struct DecodedV28 {
    using VectorOfCharStringMatrix = pma::Vector<pma::Matrix<pma::String<char>>>;
    using StringPair = std::pair<pma::String<char>, pma::String<char>>;

    // Descriptor
    static const pma::String<char> name;
    static const Archetype archetype;
    static const Gender gender;
    static const std::uint16_t age;

    static const pma::Vector<StringPair> metadata;
    static const TranslationUnit translationUnit;
    static const RotationUnit rotationUnit;
    static const CoordinateSystem coordinateSystem;
    static const std::uint16_t lodCount[3ul];
    static const std::uint16_t maxLODs[3ul];
    static const pma::String<char> complexity;
    static const pma::String<char> dbName;

    // Definition
    static const pma::Vector<pma::String<char>> guiControlNames;
    static const pma::Vector<pma::String<char>> rawControlNames;
    static const VectorOfCharStringMatrix jointNames;
    static const VectorOfCharStringMatrix blendShapeNames;
    static const VectorOfCharStringMatrix animatedMapNames;
    static const VectorOfCharStringMatrix meshNames;
    static const pma::Vector<pma::Matrix<std::uint16_t>> meshBlendShapeIndices;
    static const pma::Matrix<std::uint16_t> jointHierarchy;
    static const pma::Vector<pma::Matrix<Vector3>> neutralJointTranslations;
    static const pma::Vector<pma::Matrix<Vector3>> neutralJointRotations;

    // Behavior
    static const std::uint16_t guiControlCount;
    static const std::uint16_t rawControlCount;
    static const std::uint16_t psdCount;
    static const pma::Vector<pma::Matrix<std::uint16_t>> conditionalInputIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> conditionalOutputIndices;
    static const pma::Vector<pma::Matrix<float>> conditionalFromValues;
    static const pma::Vector<pma::Matrix<float>> conditionalToValues;
    static const pma::Vector<pma::Matrix<float>> conditionalSlopeValues;
    static const pma::Vector<pma::Matrix<float>> conditionalCutValues;
    static const pma::Vector<std::uint16_t> psdRowIndices;
    static const pma::Vector<std::uint16_t> psdColumnIndices;
    static const pma::Vector<float> psdValues;
    static const pma::Vector<std::uint16_t> jointRowCount;
    static const std::uint16_t jointColumnCount;
    static const pma::Vector<pma::Matrix<std::uint16_t>> jointVariableIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> jointGroupLODs;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> jointGroupInputIndices;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> jointGroupOutputIndices;
    static const pma::Matrix<pma::Matrix<float>> jointGroupValues;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> jointGroupJointIndices;
    static const pma::Matrix<std::uint16_t> blendShapeLODs;
    static const pma::Vector<pma::Matrix<std::uint16_t>> blendShapeInputIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> blendShapeOutputIndices;
    static const pma::Vector<std::uint16_t> animatedMapCount;
    static const pma::Matrix<std::uint16_t> animatedMapLODs;

    // Geometry
    static const pma::Vector<std::uint32_t> meshCount;
    static const pma::Vector<pma::Matrix<Vector3>> vertexPositions;
    static const pma::Vector<pma::Matrix<TextureCoordinate>> vertexTextureCoordinates;
    static const pma::Vector<pma::Matrix<Vector3>> vertexNormals;
    static const pma::Vector<pma::Matrix<VertexLayout>> vertexLayouts;
    static const pma::Matrix<pma::Matrix<std::uint32_t>> faces;
    static const pma::Matrix<std::uint16_t> maxInfluencePerVertex;
    static const pma::Matrix<pma::Matrix<float>> skinWeightsValues;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> skinWeightsJointIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> correctiveBlendShapeIndices;
    static const pma::Matrix<pma::Matrix<Vector3>> correctiveBlendShapeDeltas;
    static const pma::Matrix<pma::Matrix<std::uint32_t>> correctiveBlendShapeVertexIndices;

    // Machine learned behavior
    static const pma::Vector<pma::String<char>> mlControlNames;
    static const VectorOfCharStringMatrix regionNames;
    static const pma::Matrix<std::uint16_t> neuralNetworkIndicesPerLOD;
    static const pma::Matrix<pma::Matrix<std::uint16_t>> neuralNetworkIndicesPerMeshRegion;
    static const pma::Vector<pma::Matrix<std::uint16_t>> neuralNetworkInputIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> neuralNetworkOutputIndices;
    static const pma::Matrix<std::uint16_t> neuralNetworkLayerCount;
    static const pma::Vector<pma::Matrix<std::uint16_t>> neuralNetworkActivationFunction;
    static const pma::Matrix<pma::Matrix<float>> neuralNetworkActivationFunctionParameters;
    static const pma::Matrix<pma::Matrix<float>> neuralNetworkBiases;
    static const pma::Matrix<pma::Matrix<float>> neuralNetworkWeights;

    // RBF behavior
    static const pma::Vector<pma::String<char>> poseNames;
    static const pma::Vector<std::uint16_t> poseDistanceMethod;
    static const pma::Vector<std::uint16_t> poseFunctionType;
    static const pma::Vector<float> poseScale;

    static const pma::Matrix<std::uint16_t> solverIndicesPerLOD;
    static const pma::Vector<pma::String<char>> solverNames;
    static const pma::Vector<float> solverRadius;
    static const pma::Vector<float> solverWeightThreshold;
    static const pma::Vector<std::uint16_t> solverType;
    static const pma::Vector<std::uint16_t> solverAutomaticRadius;
    static const pma::Vector<std::uint16_t> solverDistanceMethod;
    static const pma::Vector<std::uint16_t> solverNormalizeMethod;
    static const pma::Vector<std::uint16_t> solverFunctionType;
    static const pma::Vector<std::uint16_t> solverTwistAxis;
    static const pma::Matrix<std::uint16_t> solverRawControlIndices;
    static const pma::Matrix<std::uint16_t> solverPoseIndices;
    static const pma::Matrix<float> solverRawControlValues;

    // RBF behavior ext
    static const pma::Vector<pma::String<char>> poseControlNames;
    static const pma::Matrix<std::uint16_t> poseInputControlIndices;
    static const pma::Matrix<std::uint16_t> poseOutputControlIndices;
    static const pma::Matrix<float> poseOutputControlWeights;

    // Joint behavior metadata
    static const pma::Matrix<TranslationRepresentation> jointTranslationRepresentation;
    static const pma::Matrix<RotationRepresentation> jointRotationRepresentation;
    static const pma::Matrix<ScaleRepresentation> jointScaleRepresentation;

    // Twist swing setups
    static const pma::Vector<pma::Matrix<std::uint16_t>> swingInputControlIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> swingOutputJointIndices;
    static const pma::Vector<pma::Matrix<float>> swingBlendWeights;
    static const pma::Matrix<TwistAxis> swingTwistAxes;
    static const pma::Vector<pma::Matrix<std::uint16_t>> twistInputControlIndices;
    static const pma::Vector<pma::Matrix<std::uint16_t>> twistOutputJointIndices;
    static const pma::Vector<pma::Matrix<float>> twistBlendWeights;
    static const pma::Matrix<TwistAxis> twistTwistAxes;

    // Machine learned behavior ext
    static const pma::Matrix<pma::Matrix<MachineLearnedBehaviorOperationType>> mlOperationTypes;
    static const pma::Vector<pma::Matrix<pma::Matrix<std::uint32_t>>> mlOperationParameters;
    static const pma::Vector<pma::Matrix<pma::Matrix<std::uint16_t>>> mlDependencyOperationSetIndices;
    static const pma::Vector<pma::Matrix<pma::Matrix<std::uint16_t>>> mlDependencyOperationIndices;
    static const pma::Vector<pma::Matrix<pma::Matrix<std::uint16_t>>> mlOperationIndicesPerLOD;

    // Descriptor ext
    static const RotationSequence rotationSequence;
    static const RotationSign rotationSign;
    static const FaceWindingOrder faceWindingOrder;

    static std::size_t lodConstraintToIndex(std::uint16_t maxLOD, std::uint16_t minLOD);
    static RawJoints getJoints(std::uint16_t currentMaxLOD, std::uint16_t currentMinLOD, pma::MemoryResource* memRes);
    static RawBlendShapeChannels getBlendShapes(std::uint16_t currentMaxLOD,
                                                std::uint16_t currentMinLOD,
                                                pma::MemoryResource* memRes);
    static RawConditionalTable getConditionals(std::uint16_t currentMaxLOD,
                                               std::uint16_t currentMinLOD,
                                               pma::MemoryResource* memRes);
    static RawAnimatedMaps getAnimatedMaps(std::uint16_t currentMaxLOD, std::uint16_t currentMinLOD, pma::MemoryResource* memRes);
};

}  // namespace dna
