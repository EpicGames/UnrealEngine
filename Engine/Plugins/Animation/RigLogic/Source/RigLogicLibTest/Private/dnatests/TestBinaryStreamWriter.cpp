// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/TestBinaryStreamWriter.h"

BinaryStreamWriterTest::~BinaryStreamWriterTest() = default;

TEST_F(BinaryStreamWriterTest, SetName) {
    writer->setLODCount(1);
    writer->setName("test");
    writer->write();

    reader->read();
    auto gotName = reader->getName();
    EXPECT_STREQ(gotName, "test");
}

TEST_F(BinaryStreamWriterTest, SetControlName) {
    writer->setLODCount(1);
    writer->setRawControlName(0ul, "test");
    writer->write();

    reader->read();
    EXPECT_EQ(reader->getRawControlCount(), 1ul);
    EXPECT_STREQ(reader->getRawControlName(0ul), "test");
}

TEST_F(BinaryStreamWriterTest, SetJointRowCount) {
    writer->setLODCount(1);
    writer->setJointRowCount(1ul);
    writer->write();

    reader->read();
    EXPECT_EQ(reader->getJointRowCount(), 1ul);
}
