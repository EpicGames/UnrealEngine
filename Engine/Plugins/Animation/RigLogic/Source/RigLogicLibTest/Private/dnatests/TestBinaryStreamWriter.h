// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnatests/Defs.h"

#include "dna/BinaryStreamReader.h"
#include "dna/BinaryStreamWriter.h"

#include <pma/resources/DefaultMemoryResource.h>

class BinaryStreamWriterTest : public ::testing::Test {
public:
    ~BinaryStreamWriterTest();

protected:
    void SetUp() override {
        stream = pma::makeScoped<trio::MemoryStream>();
        writer = dna::BinaryStreamWriter::create(stream.get(), &memRes);
        reader = dna::BinaryStreamReader::create(stream.get(), {}, &memRes);
    }

    void TearDown() override {
        dna::BinaryStreamReader::destroy(reader);
        dna::BinaryStreamWriter::destroy(writer);
    }

protected:
    pma::ScopedPtr<trio::MemoryStream, pma::FactoryDestroy<trio::MemoryStream>> stream;
    pma::DefaultMemoryResource memRes;
    dna::BinaryStreamWriter* writer;
    dna::BinaryStreamReader* reader;
};
