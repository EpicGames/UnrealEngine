// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/Defs.h"
#include "dnatests/FakeDNAReader.h"
#include "dnatests/Fixturesv21.h"
#include "dnatests/Fixturesv22.h"

#include "dna/BinaryStreamReader.h"
#include "dna/BinaryStreamWriter.h"
#include "dna/Configuration.h"
#include "dna/DNA.h"
#include "dna/stream/FilteredBinaryInputArchive.h"

#include <pma/resources/AlignedMemoryResource.h>

#include <memory>

namespace {

struct DNAParameters {
    dna::FileVersion fileVersion;
    std::uint16_t maxLOD;
    std::uint16_t minLOD;
};

class FilteredDNAInputArchiveTest : public ::testing::TestWithParam<DNAParameters> {
protected:
    void SetUp() override {
        dnaInstance.reset(new dna::DNA{dna::UnknownLayerPolicy::Preserve, dna::UpgradeFormatPolicy::Allowed, &amr});
        dnaParams = GetParam();

        std::vector<char> bytes;
        if (dnaParams.fileVersion == dna::FileVersion::v21) {
            bytes = dna::RawV21::getBytes();
        } else {
            bytes = dna::RawV22::getBytes();
        }

        auto stream = pma::makeScoped<trio::MemoryStream>();
        stream->write(bytes.data(), bytes.size());
        stream->seek(0);

        dna::LODConstraint lod{dnaParams.maxLOD, dnaParams.minLOD, &amr};
        dna::FilteredBinaryInputArchive archive{stream.get(), dna::DataLayer::All, std::move(lod), &amr};
        archive >> *dnaInstance;
    }

protected:
    pma::AlignedMemoryResource amr;
    std::unique_ptr<dna::DNA> dnaInstance;
    DNAParameters dnaParams;
};

}  // namespace

template<class TDecodedDNA>
static void verifyDefinition(const dna::DNA& result, const DNAParameters& dnaParams) {
    const auto index = TDecodedDNA::lodConstraintToIndex(dnaParams.maxLOD, dnaParams.minLOD);

    ASSERT_EQ(result.descriptor.lodCount, TDecodedDNA::lodCount[index]);
    ASSERT_ELEMENTS_EQ(result.definition.jointHierarchy,
                       TDecodedDNA::jointHierarchy[index],
                       TDecodedDNA::jointHierarchy[index].size());

    for (std::uint16_t lod = {}; lod < result.descriptor.lodCount; ++lod) {
        auto jointIndices = result.definition.lodJointMapping.getIndices(lod);
        ASSERT_EQ(jointIndices.size(), TDecodedDNA::jointNames[index][lod].size());
        for (std::size_t i = 0ul; i < jointIndices.size(); ++i) {
            ASSERT_EQ(result.definition.jointNames[jointIndices[i]], TDecodedDNA::jointNames[index][lod][i]);
        }

        auto blendShapeIndices = result.definition.lodBlendShapeMapping.getIndices(lod);
        ASSERT_EQ(blendShapeIndices.size(), TDecodedDNA::blendShapeNames[index][lod].size());
        for (std::size_t i = 0ul; i < blendShapeIndices.size(); ++i) {
            ASSERT_EQ(result.definition.blendShapeChannelNames[blendShapeIndices[i]],
                      TDecodedDNA::blendShapeNames[index][lod][i]);
        }

        auto animatedMapIndices = result.definition.lodAnimatedMapMapping.getIndices(lod);
        ASSERT_EQ(animatedMapIndices.size(), TDecodedDNA::animatedMapNames[index][lod].size());
        for (std::size_t i = 0ul; i < animatedMapIndices.size(); ++i) {
            ASSERT_EQ(result.definition.animatedMapNames[animatedMapIndices[i]], TDecodedDNA::animatedMapNames[index][lod][i]);
        }

        auto meshIndices = result.definition.lodMeshMapping.getIndices(lod);
        ASSERT_EQ(meshIndices.size(), TDecodedDNA::meshNames[index][lod].size());
        for (std::size_t i = 0ul; i < meshIndices.size(); ++i) {
            ASSERT_EQ(result.definition.meshNames[meshIndices[i]], TDecodedDNA::meshNames[index][lod][i]);
        }
    }

    ASSERT_EQ(result.definition.meshNames.size(), TDecodedDNA::meshCount[index]);
}

template<class TDecodedDNA>
static void verifyJoints(const dna::RawJoints& result, const DNAParameters& dnaParams, pma::MemoryResource* memRes) {
    const auto expected = TDecodedDNA::getJoints(dnaParams.maxLOD, dnaParams.minLOD, memRes);

    ASSERT_EQ(result.rowCount, expected.rowCount);
    ASSERT_EQ(result.colCount, expected.colCount);
    ASSERT_EQ(result.jointGroups.size(), expected.jointGroups.size());
    for (std::size_t jointGroupIdx = 0ul; jointGroupIdx < expected.jointGroups.size(); ++jointGroupIdx) {
        const auto& jntGrp = result.jointGroups[jointGroupIdx];
        const auto& expectedJntGrp = expected.jointGroups[jointGroupIdx];
        ASSERT_ELEMENTS_EQ(jntGrp.lods, expectedJntGrp.lods, expectedJntGrp.lods.size());
        ASSERT_ELEMENTS_EQ(jntGrp.inputIndices, expectedJntGrp.inputIndices, expectedJntGrp.inputIndices.size());
        ASSERT_ELEMENTS_EQ(jntGrp.outputIndices, expectedJntGrp.outputIndices, expectedJntGrp.outputIndices.size());
        ASSERT_ELEMENTS_NEAR(jntGrp.values, expectedJntGrp.values, expectedJntGrp.values.size(), 0.005f);
        ASSERT_ELEMENTS_EQ(jntGrp.jointIndices, expectedJntGrp.jointIndices, expectedJntGrp.jointIndices.size());
    }
}

template<class TDecodedDNA>
static void verifyBlendShapes(const dna::RawBlendShapeChannels& result,
                              const DNAParameters& dnaParams,
                              pma::MemoryResource* memRes) {
    const auto expected = TDecodedDNA::getBlendShapes(dnaParams.maxLOD, dnaParams.minLOD, memRes);
    ASSERT_ELEMENTS_EQ(result.inputIndices, expected.inputIndices, expected.inputIndices.size());
    ASSERT_ELEMENTS_EQ(result.outputIndices, expected.outputIndices, expected.outputIndices.size());
    ASSERT_ELEMENTS_EQ(result.lods, expected.lods, expected.lods.size());
}

template<class TDecodedDNA>
static void verifyAnimatedMaps(const dna::RawAnimatedMaps& result, const DNAParameters& dnaParams, pma::MemoryResource* memRes) {
    const auto expected = TDecodedDNA::getAnimatedMaps(dnaParams.maxLOD, dnaParams.minLOD, memRes);

    ASSERT_ELEMENTS_EQ(result.lods, expected.lods, expected.lods.size());
    ASSERT_ELEMENTS_EQ(result.conditionals.inputIndices,
                       expected.conditionals.inputIndices,
                       expected.conditionals.inputIndices.size());
    ASSERT_ELEMENTS_EQ(result.conditionals.outputIndices,
                       expected.conditionals.outputIndices,
                       expected.conditionals.outputIndices.size());
    ASSERT_ELEMENTS_NEAR(result.conditionals.fromValues,
                         expected.conditionals.fromValues,
                         expected.conditionals.fromValues.size(),
                         0.005f);
    ASSERT_ELEMENTS_NEAR(result.conditionals.toValues,
                         expected.conditionals.toValues,
                         expected.conditionals.toValues.size(),
                         0.005f);
    ASSERT_ELEMENTS_NEAR(result.conditionals.slopeValues,
                         expected.conditionals.slopeValues,
                         expected.conditionals.slopeValues.size(),
                         0.005f);
    ASSERT_ELEMENTS_NEAR(result.conditionals.cutValues,
                         expected.conditionals.cutValues,
                         expected.conditionals.cutValues.size(),
                         0.005f);
}

TEST_P(FilteredDNAInputArchiveTest, FilterJoints) {
    const auto& result = dnaInstance->behavior.joints;
    if (dnaParams.fileVersion == dna::FileVersion::v21) {
        verifyJoints<dna::DecodedV21>(result, dnaParams, &amr);
    } else {
        verifyJoints<dna::DecodedV22>(result, dnaParams, &amr);
    }
}

TEST_P(FilteredDNAInputArchiveTest, FilterBlendShapes) {
    const auto& result = dnaInstance->behavior.blendShapeChannels;
    if (dnaParams.fileVersion == dna::FileVersion::v21) {
        verifyBlendShapes<dna::DecodedV21>(result, dnaParams, &amr);
    } else {
        verifyBlendShapes<dna::DecodedV22>(result, dnaParams, &amr);
    }
}

TEST_P(FilteredDNAInputArchiveTest, FilterAnimatedMaps) {
    const auto& result = dnaInstance->behavior.animatedMaps;
    if (dnaParams.fileVersion == dna::FileVersion::v21) {
        verifyAnimatedMaps<dna::DecodedV21>(result, dnaParams, &amr);
    } else {
        verifyAnimatedMaps<dna::DecodedV22>(result, dnaParams, &amr);
    }
}

TEST_P(FilteredDNAInputArchiveTest, FilterDefinition) {
    if (dnaParams.fileVersion == dna::FileVersion::v21) {
        verifyDefinition<dna::DecodedV21>(*dnaInstance, dnaParams);
    } else {
        verifyDefinition<dna::DecodedV22>(*dnaInstance, dnaParams);
    }
}

INSTANTIATE_TEST_SUITE_P(FilteredDNAInputArchiveTest,
                         FilteredDNAInputArchiveTest,
                         ::testing::Values(DNAParameters{dna::FileVersion::v21, 0u, 1u},
                                           DNAParameters{dna::FileVersion::v21, 1u, 1u},
                                           DNAParameters{dna::FileVersion::v21, 0u, 0u},
                                           DNAParameters{dna::FileVersion::v22, 0u, 1u},
                                           DNAParameters{dna::FileVersion::v22, 1u, 1u},
                                           DNAParameters{dna::FileVersion::v22, 0u, 0u}));

namespace {

class GeometryFilteringTest : public ::testing::TestWithParam<dna::FileVersion> {
protected:
    void SetUp() override {
        dnaFileVersion = GetParam();
        dnaInstance = pma::makeScoped<dna::DNA>(dna::UnknownLayerPolicy::Preserve, dna::UpgradeFormatPolicy::Allowed, &amr);

        std::vector<char> bytes;
        if (dnaFileVersion == dna::FileVersion::v21) {
            bytes = dna::RawV21::getBytes();
        } else {
            bytes = dna::RawV22::getBytes();
        }

        stream = pma::makeScoped<trio::MemoryStream>();
        stream->write(bytes.data(), bytes.size());
        stream->seek(0);
    }

protected:
    dna::FileVersion dnaFileVersion;
    pma::AlignedMemoryResource amr;
    pma::ScopedPtr<trio::MemoryStream, pma::FactoryDestroy<trio::MemoryStream>> stream;
    pma::ScopedPtr<dna::DNA> dnaInstance;
};

}  // namespace

TEST_P(GeometryFilteringTest, IncludeBlendShapeTargets) {
    dna::LODConstraint lod{0u, std::numeric_limits<std::uint16_t>::max(), &amr};
    dna::FilteredBinaryInputArchive archive{stream.get(), dna::DataLayer::Geometry, std::move(lod), &amr};
    archive >> *dnaInstance;

    ASSERT_FALSE(dnaInstance->geometry.meshes.size() == 0ul);
    for (const auto& mesh : dnaInstance->geometry.meshes) {
        ASSERT_FALSE(mesh.blendShapeTargets.size() == 0ul);
    }
}

TEST_P(GeometryFilteringTest, IgnoreBlendShapeTargets) {
    dna::LODConstraint lod{0u, std::numeric_limits<std::uint16_t>::max(), &amr};
    dna::FilteredBinaryInputArchive archive{stream.get(), dna::DataLayer::GeometryWithoutBlendShapes, std::move(lod), &amr};
    archive >> *dnaInstance;

    ASSERT_FALSE(dnaInstance->geometry.meshes.size() == 0ul);
    for (const auto& mesh : dnaInstance->geometry.meshes) {
        ASSERT_TRUE(mesh.blendShapeTargets.size() == 0ul);
    }
}

INSTANTIATE_TEST_SUITE_P(GeometryFilteringTest,
                         GeometryFilteringTest,
                         ::testing::Values(dna::FileVersion::v21, dna::FileVersion::v22));

namespace {

class FilterLODsDNAReader : public dna::FakeDNAReader {
public:
    FilterLODsDNAReader(dna::MemoryResource* memRes = nullptr) :
        jointNames{memRes},
        jointIndicesPerLOD{memRes} {
        lodCount = 6u;
        jointNames = {{"body_joint1", memRes},
                      {"body_joint2", memRes},
                      {"body_joint3", memRes},
                      {"body_joint4", memRes},
                      {"body_joint5", memRes},
                      {"head_joint1", memRes},
                      {"head_joint2", memRes},
                      {"head_joint3", memRes},
                      {"head_joint4", memRes},
                      {"head_joint5", memRes},
                      {"head_joint6", memRes},
                      {"head_joint7", memRes},
                      {"head_joint8", memRes},
                      {"head_joint9", memRes},
                      {"head_joint10", memRes}};

        jointIndicesPerLOD.resize(lodCount);
        jointIndicesPerLOD[0] = {5u, 6u, 7u, 8u, 9u, 10u, 11u, 12u, 13u, 14u};
        jointIndicesPerLOD[1] = {5u, 6u, 7u, 8u, 9u, 11u, 12u, 13u};
        jointIndicesPerLOD[2] = {5u, 6u, 7u, 8u, 9u, 12u, 13u};
        jointIndicesPerLOD[3] = {5u, 6u, 7u, 8u, 9u};
        jointIndicesPerLOD[4] = {5u, 6u, 9u};
        jointIndicesPerLOD[5] = {5u, 6u};
    }

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    dna::ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const override {
        return dna::ConstArrayView<std::uint16_t>(jointIndicesPerLOD[lod]);
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointNames.size());
    }

    dna::StringView getJointName(std::uint16_t i) const override {
        return dna::StringView{jointNames[i]};
    }

private:
    std::uint16_t lodCount;
    dna::Vector<dna::String<char>> jointNames;
    dna::Matrix<std::uint16_t> jointIndicesPerLOD;
};

class FilterLODsTest : public ::testing::TestWithParam<std::vector<std::uint16_t>> {
protected:
    void SetUp() override {
        FilterLODsDNAReader dnaReader;
        stream = dna::makeScoped<dna::MemoryStream>();
        writer = dna::makeScoped<dna::BinaryStreamWriter>(stream.get());
        writer->setFrom(&dnaReader);
        writer->write();
        stream->seek(0);
        lods = GetParam();
        reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
        reader->read();
        dna::Configuration config;
        config.lods = lods;
        readerSpecificLODs = dna::makeScoped<dna::BinaryStreamReader>(stream.get(), config);
        readerSpecificLODs->read();
    }

protected:
    dna::ScopedPtr<dna::BinaryStreamReader, dna::FactoryDestroy<dna::BinaryStreamReader>> reader;
    dna::ScopedPtr<dna::BinaryStreamReader, dna::FactoryDestroy<dna::BinaryStreamReader>> readerSpecificLODs;
    dna::ScopedPtr<dna::BinaryStreamWriter, dna::FactoryDestroy<dna::BinaryStreamWriter>> writer;
    dna::ScopedPtr<dna::MemoryStream> stream;
    std::vector<std::uint16_t> lods;
};

}  // namespace

TEST_P(FilterLODsTest, KeepJointsNotInLODs) {
    std::uint16_t allJoints = reader->getJointCount();
    std::uint16_t jointsInLOD0 = static_cast<std::uint16_t>(reader->getJointIndicesForLOD(0).size());
    std::uint16_t jointsNotInLODs = static_cast<std::uint16_t>(allJoints - jointsInLOD0);
    ASSERT_EQ(allJoints, 15);
    ASSERT_EQ(jointsInLOD0, 10);
    ASSERT_EQ(jointsNotInLODs, 5);

    ASSERT_TRUE(lods.size() > 0);
    std::sort(lods.begin(), lods.end());
    std::uint16_t maxLOD = lods[0];
    std::uint16_t jointsInMaxLOD = static_cast<std::uint16_t>(reader->getJointIndicesForLOD(maxLOD).size());

    std::uint16_t expectedJoints = static_cast<std::uint16_t>(jointsInMaxLOD + jointsNotInLODs);
    std::uint16_t actualJoints = readerSpecificLODs->getJointCount();
    ASSERT_EQ(actualJoints, expectedJoints);
}

INSTANTIATE_TEST_SUITE_P(FilteredDNAInputArchiveTest,
                         FilterLODsTest,
                         ::testing::Values(std::vector<std::uint16_t>{0},
                                           std::vector<std::uint16_t>{2},
                                           std::vector<std::uint16_t>{5},
                                           std::vector<std::uint16_t>{3, 1},
                                           std::vector<std::uint16_t>{4, 3, 5},
                                           std::vector<std::uint16_t>{0, 1, 2, 3, 4}));

namespace {

class FilterSkinWeightsDNAReader : public dna::FakeDNAReader {
public:
    FilterSkinWeightsDNAReader(dna::MemoryResource* memRes = nullptr) :
        jointNames{memRes},
        meshNames{memRes},
        jointIndicesPerLOD{memRes},
        meshesPerLOD{memRes},
        skinWeightsValues{memRes},
        skinWeightsJointIndices{memRes} {
        lodCount = 3u;
        jointNames = {{"head_joint1", memRes},
                      {"head_joint2", memRes},
                      {"head_joint3", memRes},
                      {"head_joint4", memRes},
                      {"head_joint5", memRes},
                      {"head_joint6", memRes},
                      {"head_joint7", memRes},
                      {"head_joint8", memRes},
                      {"head_joint9", memRes},
                      {"head_joint10", memRes}};
        meshNames = {"mesh1", "mesh2"};
        jointIndicesPerLOD.resize(lodCount);
        jointIndicesPerLOD[0] = {0u, 1u, 2u, 3u, 4u, 5u, 6u, 7u, 8u, 9u};
        jointIndicesPerLOD[1] = {4u, 5u, 7u};
        jointIndicesPerLOD[2] = {};
        meshesPerLOD.resize(lodCount);
        meshesPerLOD[0] = {0u, 1u};
        meshesPerLOD[1] = {0u, 1u};
        meshesPerLOD[2] = {0u, 1u};
        skinWeightsValues = {{// Mesh-0
                              {0.7f, 0.1f, 0.2f},
                              {0.5f, 0.5f},
                              {0.4f, 0.6f}},
                             {// Mesh-1
                              {0.4f, 0.3f, 0.3f},
                              {0.8f, 0.2f},
                              {0.1f, 0.9f}}};
        skinWeightsJointIndices = {{// Mesh-0
                                    {0, 1, 2},
                                    {3, 4},
                                    {6, 9}},
                                   {// Mesh-1
                                    {0, 1, 2},
                                    {7, 8},
                                    {5, 6}}};
    }

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    dna::ConstArrayView<std::uint16_t> getJointIndicesForLOD(std::uint16_t lod) const override {
        return dna::ConstArrayView<std::uint16_t>(jointIndicesPerLOD[lod]);
    }

    std::uint16_t getJointCount() const override {
        return static_cast<std::uint16_t>(jointNames.size());
    }

    dna::StringView getJointName(std::uint16_t i) const override {
        return dna::StringView{jointNames[i]};
    }

    std::uint16_t getMeshCount() const override {
        return static_cast<std::uint16_t>(meshNames.size());
    }

    dna::StringView getMeshName(std::uint16_t i) const override {
        return dna::StringView{meshNames[i]};
    }

    dna::ConstArrayView<std::uint16_t> getMeshIndicesForLOD(std::uint16_t lod) const override {
        return dna::ConstArrayView<std::uint16_t>(meshesPerLOD[lod]);
    }

    std::uint32_t getSkinWeightsCount(std::uint16_t meshIndex) const override {
        if (meshIndex < skinWeightsJointIndices.size()) {
            return static_cast<std::uint16_t>(skinWeightsJointIndices[meshIndex].size());
        }
        return {};
    }

    dna::ConstArrayView<float> getSkinWeightsValues(std::uint16_t meshIndex, std::uint32_t vertexIndex) const override {
        if (vertexIndex < getSkinWeightsCount(meshIndex)) {
            return dna::ConstArrayView<float>(skinWeightsValues[meshIndex][vertexIndex]);
        }
        return {};
    }

    dna::ConstArrayView<std::uint16_t> getSkinWeightsJointIndices(std::uint16_t meshIndex,
                                                                  std::uint32_t vertexIndex) const override {
        if (vertexIndex < getSkinWeightsCount(meshIndex)) {
            return dna::ConstArrayView<std::uint16_t>(skinWeightsJointIndices[meshIndex][vertexIndex]);
        }
        return {};
    }

private:
    std::uint16_t lodCount;
    dna::Vector<dna::String<char>> jointNames;
    dna::Vector<dna::String<char>> meshNames;
    dna::Matrix<std::uint16_t> jointIndicesPerLOD;
    dna::Matrix<std::uint16_t> meshesPerLOD;
    dna::Vector<dna::Matrix<float>> skinWeightsValues;
    dna::Vector<dna::Matrix<std::uint16_t>> skinWeightsJointIndices;
};

class FilterSkinWeightsTest : public ::testing::Test {
protected:
    void SetUp() override {
        FilterSkinWeightsDNAReader dnaReader;
        stream = dna::makeScoped<dna::MemoryStream>();
        writer = dna::makeScoped<dna::BinaryStreamWriter>(stream.get());
        writer->setFrom(&dnaReader);
        writer->write();
        reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
        reader->read();

        lods = {2u};
        dna::Configuration config;
        config.lods = lods;
        readerLOD2 = dna::makeScoped<dna::BinaryStreamReader>(stream.get(), config);
        readerLOD2->read();
    }

protected:
    dna::ScopedPtr<dna::BinaryStreamReader, dna::FactoryDestroy<dna::BinaryStreamReader>> reader;
    dna::ScopedPtr<dna::BinaryStreamReader, dna::FactoryDestroy<dna::BinaryStreamReader>> readerLOD2;
    dna::ScopedPtr<dna::BinaryStreamWriter, dna::FactoryDestroy<dna::BinaryStreamWriter>> writer;
    dna::ScopedPtr<dna::MemoryStream> stream;
    std::vector<std::uint16_t> lods;
};
}  // namespace

TEST_F(FilterSkinWeightsTest, RemoveAllJoints) {
    std::uint16_t jointCount = reader->getJointCount();
    ASSERT_EQ(jointCount, 10u);

    ASSERT_EQ(reader->getSkinWeightsCount(0u), 3u);
    ASSERT_EQ(reader->getSkinWeightsCount(1u), 3u);

    ASSERT_EQ(reader->getSkinWeightsJointIndices(0u, 0u).size(), 3u);
    ASSERT_EQ(reader->getSkinWeightsJointIndices(0u, 1u).size(), 2u);
    ASSERT_EQ(reader->getSkinWeightsJointIndices(0u, 2u).size(), 2u);
    ASSERT_EQ(reader->getSkinWeightsJointIndices(1u, 0u).size(), 3u);
    ASSERT_EQ(reader->getSkinWeightsJointIndices(1u, 1u).size(), 2u);
    ASSERT_EQ(reader->getSkinWeightsJointIndices(1u, 2u).size(), 2u);

    ASSERT_EQ(reader->getSkinWeightsValues(0u, 0u).size(), 3u);
    ASSERT_EQ(reader->getSkinWeightsValues(0u, 1u).size(), 2u);
    ASSERT_EQ(reader->getSkinWeightsValues(0u, 2u).size(), 2u);
    ASSERT_EQ(reader->getSkinWeightsValues(1u, 0u).size(), 3u);
    ASSERT_EQ(reader->getSkinWeightsValues(1u, 1u).size(), 2u);
    ASSERT_EQ(reader->getSkinWeightsValues(1u, 2u).size(), 2u);

    ASSERT_EQ(readerLOD2->getSkinWeightsJointIndices(0u, 0u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsJointIndices(0u, 1u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsJointIndices(0u, 2u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsJointIndices(1u, 0u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsJointIndices(1u, 1u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsJointIndices(1u, 2u).size(), 0u);

    ASSERT_EQ(readerLOD2->getSkinWeightsValues(0u, 0u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsValues(0u, 1u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsValues(0u, 2u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsValues(1u, 0u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsValues(1u, 1u).size(), 0u);
    ASSERT_EQ(readerLOD2->getSkinWeightsValues(1u, 2u).size(), 0u);
}
