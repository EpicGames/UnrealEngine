// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/Defs.h"

#include "dna/LODConstraint.h"

TEST(LODConstraint, GetMaxLOD) {
    dna::LODConstraint lc1{0, 7, nullptr};
    ASSERT_EQ(lc1.getMaxLOD(), 0);

    dna::LODConstraint lc2{2, 5, nullptr};
    ASSERT_EQ(lc2.getMaxLOD(), 2);

    std::uint16_t lods[] = {1, 3, 4};
    dna::LODConstraint lc3{{lods, 3}, nullptr};
    ASSERT_EQ(lc3.getMaxLOD(), 1);
}

TEST(LODConstraint, GetMinLOD) {
    dna::LODConstraint lc1{0, 7, nullptr};
    ASSERT_EQ(lc1.getMinLOD(), 7);

    dna::LODConstraint lc2{2, 5, nullptr};
    ASSERT_EQ(lc2.getMinLOD(), 5);

    std::uint16_t lods[] = {1, 3, 4};
    dna::LODConstraint lc3{{lods, 3}, nullptr};
    ASSERT_EQ(lc3.getMinLOD(), 4);
}

TEST(LODConstraint, GetLODCount) {
    dna::LODConstraint lc1{0, 7, nullptr};
    ASSERT_EQ(lc1.getLODCount(), 8);

    dna::LODConstraint lc2{2, 5, nullptr};
    ASSERT_EQ(lc2.getLODCount(), 4);

    std::uint16_t lods[] = {1, 3, 4};
    dna::LODConstraint lc3{{lods, 3}, nullptr};
    ASSERT_EQ(lc3.getLODCount(), 3);
}

TEST(LODConstraint, HasImpactOn) {
    dna::LODConstraint lc1{0, 7, nullptr};
    ASSERT_FALSE(lc1.hasImpactOn(0));
    ASSERT_FALSE(lc1.hasImpactOn(1));
    ASSERT_FALSE(lc1.hasImpactOn(7));
    ASSERT_FALSE(lc1.hasImpactOn(8));
    ASSERT_TRUE(lc1.hasImpactOn(9));

    dna::LODConstraint lc2{2, 5, nullptr};
    ASSERT_FALSE(lc2.hasImpactOn(0));
    ASSERT_TRUE(lc2.hasImpactOn(1));
    ASSERT_TRUE(lc2.hasImpactOn(4));
    ASSERT_TRUE(lc2.hasImpactOn(5));
    ASSERT_TRUE(lc2.hasImpactOn(6));

    std::uint16_t lods3[] = {0, 7};
    dna::LODConstraint lc3{{lods3, 2}, nullptr};
    ASSERT_FALSE(lc3.hasImpactOn(0));
    ASSERT_FALSE(lc3.hasImpactOn(1));
    ASSERT_TRUE(lc3.hasImpactOn(2));
    ASSERT_TRUE(lc3.hasImpactOn(3));
    ASSERT_TRUE(lc3.hasImpactOn(4));

    std::uint16_t lods4[] = {1, 3, 5};
    dna::LODConstraint lc4{{lods4, 3}, nullptr};
    ASSERT_FALSE(lc4.hasImpactOn(0));
    ASSERT_TRUE(lc4.hasImpactOn(1));
    ASSERT_TRUE(lc4.hasImpactOn(2));
    ASSERT_TRUE(lc4.hasImpactOn(3));
    ASSERT_TRUE(lc4.hasImpactOn(4));
}

TEST(LODConstraint, ClampTo) {
    dna::LODConstraint lc1{0, 7, nullptr};
    ASSERT_EQ(lc1.getLODCount(), 8);
    lc1.clampTo(2);
    ASSERT_EQ(lc1.getLODCount(), 2);
}

TEST(LODConstraint, ApplyTo) {
    // Some values for each LOD (the position indicates which LOD)
    dna::Vector<std::uint16_t> unconstrainedLODs{10, 20, 30, 40, 50, 60, 70};
    dna::Vector<std::uint16_t> expected{20, 40, 60};

    std::uint16_t lods[] = {1, 3, 5};
    dna::LODConstraint lc{{lods, 3}, nullptr};

    lc.applyTo(unconstrainedLODs);
    ASSERT_EQ(unconstrainedLODs, expected);
}
