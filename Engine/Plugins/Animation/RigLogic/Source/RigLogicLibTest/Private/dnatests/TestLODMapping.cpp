// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/Defs.h"

#include "dna/LODConstraint.h"
#include "dna/LODMapping.h"

#include <pma/resources/DefaultMemoryResource.h>

namespace {

class LODMappingTest : public ::testing::Test {
protected:
    void SetUp() override {
        pma::Matrix<std::uint16_t> indices = {
            {1, 2, 3, 4},                                             // LOD-3
            {1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16},  // LOD-0
            {1, 2, 3, 4, 5, 6, 7, 8},                                 // LOD-2
            {1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12}                   // LOD-1
        };
        pma::Vector<std::uint16_t> lods = {1, 3, 2, 0};

        mapping.setLODCount(4);
        for (std::size_t i = 0ul; i < indices.size(); ++i) {
            mapping.addIndices(static_cast<std::uint16_t>(i), indices[i].data(), static_cast<std::uint16_t>(indices[i].size()));
            mapping.associateLODWithIndices(static_cast<std::uint16_t>(i), lods[i]);
        }
    }

protected:
    pma::DefaultMemoryResource memRes;
    dna::LODMapping mapping{&memRes};
};

}  // namespace

#define ASSERT_LODMAPPING_EQ(result, expected)                                                                                   \
    for (std::uint16_t lod = {}; lod < result.getLODCount(); ++lod) {                                                            \
        dna::ConstArrayView<std::uint16_t> resultIndices = result.getIndices(lod);                                               \
        dna::ConstArrayView<std::uint16_t> expectedIndices{expected[lod]};                                                       \
        ASSERT_EQ(resultIndices, expectedIndices);                                                                               \
    }

TEST_F(LODMappingTest, SetLODRangeMax) {
    ASSERT_EQ(mapping.getLODCount(), 4u);
    dna::LODConstraint lodConstraint{2u, 3u, &memRes};
    mapping.discardLODs(lodConstraint);
    ASSERT_EQ(mapping.getLODCount(), 2u);

    // These are sorted now by LOD to match the order in which they will be returned
    pma::Matrix<std::uint16_t> expected{{1, 2, 3, 4, 5, 6, 7, 8}, {1, 2, 3, 4}};

    ASSERT_LODMAPPING_EQ(mapping, expected);
}

TEST_F(LODMappingTest, SetLODRangeMin) {
    ASSERT_EQ(mapping.getLODCount(), 4u);
    dna::LODConstraint lodConstraint{0u, 1u, &memRes};
    mapping.discardLODs(lodConstraint);
    ASSERT_EQ(mapping.getLODCount(), 2u);

    // These are sorted now by LOD to match the order in which they will be returned
    pma::Matrix<std::uint16_t> expected{{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16},
                                        {1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12}};

    ASSERT_LODMAPPING_EQ(mapping, expected);
}

TEST_F(LODMappingTest, SetLODRangeMaxMin) {
    ASSERT_EQ(mapping.getLODCount(), 4u);
    dna::LODConstraint lodConstraint{1u, 2u, &memRes};
    mapping.discardLODs(lodConstraint);
    ASSERT_EQ(mapping.getLODCount(), 2u);

    // These are sorted now by LOD to match the order in which they will be returned
    pma::Matrix<std::uint16_t> expected{{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12}, {1, 2, 3, 4, 5, 6, 7, 8}};

    ASSERT_LODMAPPING_EQ(mapping, expected);
}

TEST_F(LODMappingTest, SetLODRangeExact) {
    ASSERT_EQ(mapping.getLODCount(), 4u);
    const std::uint16_t lods[] = {1u, 3u};
    dna::LODConstraint lodConstraint{dna::ConstArrayView<std::uint16_t>{lods, 2ul}, &memRes};
    mapping.discardLODs(lodConstraint);
    ASSERT_EQ(mapping.getLODCount(), 2u);

    // These are sorted now by LOD to match the order in which they will be returned
    pma::Matrix<std::uint16_t> expected{{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12}, {1, 2, 3, 4}};

    ASSERT_LODMAPPING_EQ(mapping, expected);
}
