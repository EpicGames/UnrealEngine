// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/Defs.h"
#include "dnatests/Fixturesv21.h"

#include "dna/BinaryStreamReader.h"
#include "dna/StreamReader.h"
#include "dna/types/Aliases.h"

#include <pma/resources/AlignedMemoryResource.h>
#include <status/Provider.h>

namespace {

class MalformedInputTest : public ::testing::Test {
protected:
    void SetUp() override {
        sc::StatusProvider::reset();
    }

    void TearDown() override {
        sc::StatusProvider::reset();
    }
};

}  // namespace

TEST_F(MalformedInputTest, SignatureMismatchSetsError) {
    auto bytes = dna::RawV21::getBytes();
    // Corrupt the 3-byte "DNA" signature (bytes 0-2)
    bytes[0] = 0x00;
    bytes[1] = 0x00;
    bytes[2] = 0x00;

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::SignatureMismatchError.code);
}

TEST_F(MalformedInputTest, UnsupportedGenerationOnV21FormatSetsInvalidDataError) {
    // Mutate a v21 fixture to have generation=99. Dispatcher falls through to v27 serializer
    // (IndexTable format), which misreads the SectionLookupTable as an IndexTable with
    // 0x27=39 entries, then seeks 624 bytes past EOF -> SeekError.
    // archive.isOk() == false catches this and maps it to InvalidDataError.
    auto bytes = dna::RawV21::getBytes();
    bytes[3] = 0x00;
    bytes[4] = 0x63;  // generation = 99

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::InvalidDataError.code);
}

TEST_F(MalformedInputTest, VersionMismatchSetsError) {
    // Minimal v22+ format DNA: signature + generation=99 (unsupported) + IndexTable(0 entries).
    // v22+ uses IndexTable; 0 entries means no section seeks, so the archive reads cleanly
    // and the version check is reached: supported() == (generation == 2) -> false.
    const std::vector<char> bytes = {
        '\x44',
        '\x4E',
        '\x41',  // "DNA" signature
        '\x00',
        '\x63',  // generation = 99 (unsupported)
        '\x00',
        '\x01',  // version = 1
        '\x00',
        '\x00',
        '\x00',
        '\x00'  // IndexTable: 0 entries
    };

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::VersionMismatchError.code);
}

TEST_F(MalformedInputTest, OversizedIndexTableCountSetsInvalidDataError) {
    // v22+ format, IndexTable entry count = 0xFFFFFFFF.
    // boundSize() fires before any allocation attempt: malformed=true -> InvalidDataError.
    const std::vector<char> bytes = {
        '\x44',
        '\x4E',
        '\x41',  // "DNA" signature
        '\x00',
        '\x02',  // generation = 2
        '\x00',
        '\x02',  // version = 2 (v22)
        '\xFF',
        '\xFF',
        '\xFF',
        '\xFF'  // IndexTable: 0xFFFFFFFF entries
    };

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::InvalidDataError.code);
}

TEST_F(MalformedInputTest, IndexTableEntryWithOffsetPastEOFSetsInvalidDataError) {
    // v22+ format, one IndexTable entry whose offset field = 0xDEADBEEF.
    // After reading the IndexTable, Layer::serialize calls proxy(index->offset) which
    // seeks to 0xDEADBEEF on a 27-byte stream -> SeekError -> archive.isOk()=false -> InvalidDataError.
    const std::vector<char> bytes = {
        '\x44',
        '\x4E',
        '\x41',  // "DNA" signature
        '\x00',
        '\x02',  // generation = 2
        '\x00',
        '\x02',  // version = 2 (v22)
        '\x00',
        '\x00',
        '\x00',
        '\x01',  // IndexTable: 1 entry
        // Index entry (id + version + offset + size, each uint32 big-endian):
        '\xDE',
        '\xAD',
        '\xBE',
        '\xEF',  // id
        '\x00',
        '\x00',
        '\x00',
        '\x00',  // entry version
        '\xDE',
        '\xAD',
        '\xBE',
        '\xEF',  // offset = 0xDEADBEEF (past EOF)
        '\x00',
        '\x00',
        '\x00',
        '\x04'  // size = 4
    };

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::InvalidDataError.code);
}

TEST_F(MalformedInputTest, OversizedContainerSetsInvalidDataError) {
    auto bytes = dna::RawV21::getBytes();
    // Replace the name string length field at the start of the descriptor (offset 0x27)
    // with 0xFFFFFFFF. Exceeds stream->size() -> boundSize() sets malformed=true.
    const std::size_t descriptorOffset = 0x27;
    bytes[descriptorOffset + 0] = '\xFF';
    bytes[descriptorOffset + 1] = '\xFF';
    bytes[descriptorOffset + 2] = '\xFF';
    bytes[descriptorOffset + 3] = '\xFF';

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::InvalidDataError.code);
}

TEST_F(MalformedInputTest, ExcessiveLODCountSetsInvalidDataError) {
    auto bytes = dna::RawV21::getBytes();
    // LOD Count field layout (from descriptor byte layout):
    //   header(39) + name_len(4) + name(4) + archetype(2) + gender(2) + age(2)
    //   + metadata_count(4) + 2x(key_len(4)+key(5)+val_len(4)+val(7))
    //   + translation_unit(2) + rotation_unit(2) + coord_x(2) + coord_y(2) + coord_z(2)
    //   = offset 107 (0x6B)
    // Setting LOD Count to 256 (0x0100) exceeds LODLimits::count() (33).
    const std::size_t lodCountOffset = 0x27 + 68;
    bytes[lodCountOffset + 0] = 0x01;
    bytes[lodCountOffset + 1] = 0x00;

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::InvalidDataError.code);
}

TEST_F(MalformedInputTest, DescriptorPayloadWithOversizedMetadataCountSetsInvalidDataError) {
    // mutated = dna[:39] + payload + dna[39:]
    // where payload[0:4] = name_length=4, payload[4:8] = "AAAA",
    // and payload[8:256] = 0xDEADBEEF repeating.

    // The valid name passes the first string read. The metadata count field
    // (descriptor bytes 14-17 = 0xBEEFDEAD ~= 3.2B entries) triggers
    // boundSize() -> malformed=true -> InvalidDataError, with zero allocation.
    std::vector<char> payload(256, 0);
    payload[0] = 0x00;
    payload[1] = 0x00;
    payload[2] = 0x00;
    payload[3] = 0x04;
    payload[4] = 'A';
    payload[5] = 'A';
    payload[6] = 'A';
    payload[7] = 'A';
    for (std::size_t off = 8; off < 256; off += 4) {
        payload[off + 0] = '\xDE';
        payload[off + 1] = '\xAD';
        payload[off + 2] = '\xBE';
        payload[off + 3] = '\xEF';
    }

    auto original = dna::RawV21::getBytes();
    std::vector<char> mutated;
    mutated.insert(mutated.end(), original.begin(), original.begin() + 39);
    mutated.insert(mutated.end(), payload.begin(), payload.end());
    mutated.insert(mutated.end(), original.begin() + 39, original.end());

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(mutated.data(), mutated.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::InvalidDataError.code);
}

TEST_F(MalformedInputTest, MidFilePayloadInDefinitionSectionSetsInvalidDataError) {
    // Same as DescriptorPayloadWithOversizedMetadataCountSetsInvalidDataError but targeting the definition section
    // (offset 0x7E = 126), which is deeper into the file after the descriptor has been read cleanly.
    // payload[0:4] = 0xDEADBEEF is inserted as the first uint32 of the definition,
    // which is the joint-name LOD mapping count (~3.7B). boundSize() fires.
    std::vector<char> payload(256, 0);
    for (std::size_t off = 0; off < 256; off += 4) {
        payload[off + 0] = '\xDE';
        payload[off + 1] = '\xAD';
        payload[off + 2] = '\xBE';
        payload[off + 3] = '\xEF';
    }

    const std::size_t injectionOffset = 0x7E;  // start of definition section
    auto original = dna::RawV21::getBytes();
    std::vector<char> mutated;
    mutated.insert(mutated.end(), original.begin(), original.begin() + injectionOffset);
    mutated.insert(mutated.end(), payload.begin(), payload.end());
    mutated.insert(mutated.end(), original.begin() + injectionOffset, original.end());

    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(mutated.data(), mutated.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
    ASSERT_EQ(dna::Status::get().code, dna::StreamReader::InvalidDataError.code);
}

TEST_F(MalformedInputTest, EmptyStreamSetsError) {
    std::vector<char> bytes;
    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
}

TEST_F(MalformedInputTest, TruncatedHeaderSetsError) {
    auto bytes = dna::RawV21::getBytes();
    // Keep only the 7-byte signature+version prefix, omitting the section lookup table
    bytes.resize(7);
    auto stream = pma::makeScoped<trio::MemoryStream>();
    stream->write(bytes.data(), bytes.size());
    stream->seek(0);
    auto reader = dna::makeScoped<dna::BinaryStreamReader>(stream.get());
    reader->read();

    ASSERT_FALSE(dna::Status::isOk());
}
