// Copyright Epic Games, Inc. All Rights Reserved.

#include "dnatests/TestStreamReadWriteIntegration.h"

#include "dnatests/Defs.h"
#ifdef DNA_BUILD_WITH_JSON_SUPPORT
    #include "dnatests/FixturesJSON.h"
#endif  // DNA_BUILD_WITH_JSON_SUPPORT
#include "dnatests/Fixturesv21.h"
#include "dnatests/Fixturesv22.h"
#include "dnatests/Fixturesv23.h"
#include "dnatests/Fixturesv24.h"
#include "dnatests/Fixturesv25.h"
#include "dnatests/Fixturesv26.h"
#include "dnatests/Fixturesv27.h"
#include "dnatests/Fixturesv28.h"

#include "dna/BinaryStreamReader.h"
#include "dna/BinaryStreamWriter.h"
#include "dna/Configuration.h"
#ifdef DNA_BUILD_WITH_JSON_SUPPORT
    #include "dna/JSONStreamReader.h"
    #include "dna/JSONStreamWriter.h"
#endif  // DNA_BUILD_WITH_JSON_SUPPORT

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

namespace dna {

template<class TAPICopyParameters>
static void verifyDescriptor(DescriptorReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    ASSERT_EQ(reader->getName(), StringView{DecodedDNA::name});
    ASSERT_EQ(reader->getArchetype(), DecodedDNA::archetype);
    ASSERT_EQ(reader->getGender(), DecodedDNA::gender);
    ASSERT_EQ(reader->getAge(), DecodedDNA::age);

    const auto metaDataCount = reader->getMetaDataCount();
    ASSERT_EQ(metaDataCount, 2u);
    for (std::uint32_t i = {}; i < metaDataCount; ++i) {
        const auto key = reader->getMetaDataKey(i);
        const auto value = reader->getMetaDataValue(key);
        ASSERT_EQ(key, StringView{DecodedDNA::metadata[i].first});
        ASSERT_EQ(value, StringView{DecodedDNA::metadata[i].second});
    }

    ASSERT_EQ(reader->getTranslationUnit(), DecodedDNA::translationUnit);
    ASSERT_EQ(reader->getRotationUnit(), DecodedDNA::rotationUnit);

    const auto coordinateSystem = reader->getCoordinateSystem();
    ASSERT_EQ(coordinateSystem.x, DecodedDNA::coordinateSystem.x);
    ASSERT_EQ(coordinateSystem.y, DecodedDNA::coordinateSystem.y);
    ASSERT_EQ(coordinateSystem.z, DecodedDNA::coordinateSystem.z);

    ASSERT_EQ(reader->getLODCount(), DecodedDNA::lodCount[index]);
    ASSERT_EQ(reader->getDBMaxLOD(), DecodedDNA::maxLODs[index]);
    ASSERT_EQ(reader->getDBComplexity(), StringView{DecodedDNA::complexity});
    ASSERT_EQ(reader->getDBName(), StringView{DecodedDNA::dbName});
}

template<class TAPICopyParameters>
static void verifyDefinition(DefinitionReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    const auto guiControlCount = reader->getGUIControlCount();
    ASSERT_EQ(guiControlCount, DecodedDNA::guiControlNames.size());
    for (std::uint16_t i = {}; i < guiControlCount; ++i) {
        ASSERT_EQ(reader->getGUIControlName(i), StringView{DecodedDNA::guiControlNames[i]});
    }

    const auto rawControlCount = reader->getRawControlCount();
    ASSERT_EQ(rawControlCount, DecodedDNA::rawControlNames.size());
    for (std::uint16_t i = {}; i < rawControlCount; ++i) {
        ASSERT_EQ(reader->getRawControlName(i), StringView{DecodedDNA::rawControlNames[i]});
    }

    ASSERT_EQ(reader->getJointCount(), DecodedDNA::jointNames[index][0ul].size());
    const auto& expectedJointNames = DecodedDNA::jointNames[index][TAPICopyParameters::currentLOD()];
    const auto jointIndices = reader->getJointIndicesForLOD(TAPICopyParameters::currentLOD());
    ASSERT_EQ(jointIndices.size(), expectedJointNames.size());
    for (std::size_t i = 0ul; i < jointIndices.size(); ++i) {
        ASSERT_EQ(reader->getJointName(jointIndices[i]), StringView{expectedJointNames[i]});
    }

    for (std::uint16_t i = {}; i < reader->getJointCount(); ++i) {
        ASSERT_EQ(reader->getJointParentIndex(i), DecodedDNA::jointHierarchy[index][i]);
    }

    ASSERT_EQ(reader->getBlendShapeChannelCount(), DecodedDNA::blendShapeNames[index][0ul].size());
    const auto& expectedBlendShapeNames = DecodedDNA::blendShapeNames[index][TAPICopyParameters::currentLOD()];
    const auto blendShapeIndices = reader->getBlendShapeChannelIndicesForLOD(TAPICopyParameters::currentLOD());
    ASSERT_EQ(blendShapeIndices.size(), expectedBlendShapeNames.size());
    for (std::size_t i = 0ul; i < blendShapeIndices.size(); ++i) {
        ASSERT_EQ(reader->getBlendShapeChannelName(blendShapeIndices[i]), StringView{expectedBlendShapeNames[i]});
    }

    ASSERT_EQ(reader->getAnimatedMapCount(), DecodedDNA::animatedMapNames[index][0ul].size());
    const auto& expectedAnimatedMapNames = DecodedDNA::animatedMapNames[index][TAPICopyParameters::currentLOD()];
    const auto animatedMapIndices = reader->getAnimatedMapIndicesForLOD(TAPICopyParameters::currentLOD());
    ASSERT_EQ(animatedMapIndices.size(), expectedAnimatedMapNames.size());
    for (std::size_t i = 0ul; i < animatedMapIndices.size(); ++i) {
        ASSERT_EQ(reader->getAnimatedMapName(animatedMapIndices[i]), StringView{expectedAnimatedMapNames[i]});
    }

    std::uint16_t expectedMeshCount = {};
    for (std::uint16_t i = 0ul; i < DecodedDNA::meshNames[index].size(); ++i) {
        expectedMeshCount = static_cast<std::uint16_t>(expectedMeshCount + DecodedDNA::meshNames[index][i].size());
    }
    ASSERT_EQ(reader->getMeshCount(), expectedMeshCount);
    const auto& expectedMeshNames = DecodedDNA::meshNames[index][TAPICopyParameters::currentLOD()];
    const auto meshIndices = reader->getMeshIndicesForLOD(TAPICopyParameters::currentLOD());
    ASSERT_EQ(meshIndices.size(), expectedMeshNames.size());
    for (std::size_t i = 0ul; i < meshIndices.size(); ++i) {
        ASSERT_EQ(reader->getMeshName(meshIndices[i]), StringView{expectedMeshNames[i]});
    }

    std::uint16_t expectedMeshBlendShapeMappingCount = {};
    for (std::uint16_t i = 0ul; i < DecodedDNA::meshBlendShapeIndices[index].size(); ++i) {
        expectedMeshBlendShapeMappingCount =
            static_cast<std::uint16_t>(expectedMeshBlendShapeMappingCount + DecodedDNA::meshBlendShapeIndices[index][i].size());
    }
    ASSERT_EQ(reader->getMeshBlendShapeChannelMappingCount(), expectedMeshBlendShapeMappingCount);
    const auto meshBlendShapeIndices = reader->getMeshBlendShapeChannelMappingIndicesForLOD(TAPICopyParameters::currentLOD());
    const auto& expectedMeshBlendShapeIndices = DecodedDNA::meshBlendShapeIndices[index][TAPICopyParameters::currentLOD()];
    ASSERT_EQ(meshBlendShapeIndices, ConstArrayView<std::uint16_t>{expectedMeshBlendShapeIndices});

    const auto& expectedNeutralJointTranslations = DecodedDNA::neutralJointTranslations[index][TAPICopyParameters::currentLOD()];
    ASSERT_EQ(jointIndices.size(), expectedNeutralJointTranslations.size());
    for (std::size_t i = 0ul; i < jointIndices.size(); ++i) {
        ASSERT_EQ(reader->getNeutralJointTranslation(jointIndices[i]), expectedNeutralJointTranslations[i]);
    }

    const auto& expectedNeutralJointRotations = DecodedDNA::neutralJointRotations[index][TAPICopyParameters::currentLOD()];
    ASSERT_EQ(jointIndices.size(), expectedNeutralJointRotations.size());
    for (std::size_t i = 0ul; i < jointIndices.size(); ++i) {
        ASSERT_EQ(reader->getNeutralJointRotation(jointIndices[i]), expectedNeutralJointRotations[i]);
    }
}

template<class TAPICopyParameters>
static void verifyBehavior(BehaviorReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    const auto guiToRawInputIndices = reader->getGUIToRawInputIndices();
    const auto& expectedG2RInputIndices = DecodedDNA::conditionalInputIndices[0ul][0ul];
    ASSERT_EQ(guiToRawInputIndices, ConstArrayView<std::uint16_t>{expectedG2RInputIndices});

    const auto guiToRawOutputIndices = reader->getGUIToRawOutputIndices();
    const auto& expectedG2ROutputIndices = DecodedDNA::conditionalOutputIndices[0ul][0ul];
    ASSERT_EQ(guiToRawOutputIndices, ConstArrayView<std::uint16_t>{expectedG2ROutputIndices});

    const auto guiToRawFromValues = reader->getGUIToRawFromValues();
    const auto& expectedG2RFromValues = DecodedDNA::conditionalFromValues[0ul][0ul];
    ASSERT_EQ(guiToRawFromValues, ConstArrayView<float>{expectedG2RFromValues});

    const auto guiToRawToValues = reader->getGUIToRawToValues();
    const auto& expectedG2RToValues = DecodedDNA::conditionalToValues[0ul][0ul];
    ASSERT_EQ(guiToRawToValues, ConstArrayView<float>{expectedG2RToValues});

    const auto guiToRawSlopeValues = reader->getGUIToRawSlopeValues();
    const auto& expectedG2RSlopeValues = DecodedDNA::conditionalSlopeValues[0ul][0ul];
    ASSERT_EQ(guiToRawSlopeValues, ConstArrayView<float>{expectedG2RSlopeValues});

    const auto guiToRawCutValues = reader->getGUIToRawCutValues();
    const auto& expectedG2RCutValues = DecodedDNA::conditionalCutValues[0ul][0ul];
    ASSERT_EQ(guiToRawCutValues, ConstArrayView<float>{expectedG2RCutValues});

    const auto psdRowIndices = reader->getPSDRowIndices();
    ASSERT_EQ(psdRowIndices, ConstArrayView<std::uint16_t>{DecodedDNA::psdRowIndices});

    const auto psdColumnIndices = reader->getPSDColumnIndices();
    ASSERT_EQ(psdColumnIndices, ConstArrayView<std::uint16_t>{DecodedDNA::psdColumnIndices});

    const auto psdValues = reader->getPSDValues();
    ASSERT_EQ(psdValues, ConstArrayView<float>{DecodedDNA::psdValues});

    ASSERT_EQ(reader->getPSDCount(), DecodedDNA::psdCount);
    ASSERT_EQ(reader->getJointRowCount(), DecodedDNA::jointRowCount[index]);
    ASSERT_EQ(reader->getJointColumnCount(), DecodedDNA::jointColumnCount);

    const auto jointVariableAttrIndices = reader->getJointVariableAttributeIndices(TAPICopyParameters::currentLOD());
    const auto& expectedJointVariableAttrIndices = DecodedDNA::jointVariableIndices[index][TAPICopyParameters::currentLOD()];
    ASSERT_EQ(jointVariableAttrIndices, ConstArrayView<std::uint16_t>{expectedJointVariableAttrIndices});

    const auto jointGroupCount = reader->getJointGroupCount();
    ASSERT_EQ(jointGroupCount, DecodedDNA::jointGroupLODs.size());

    for (std::uint16_t i = {}; i < jointGroupCount; ++i) {
        const auto& expectedLODs = DecodedDNA::jointGroupLODs[i][index];
        ASSERT_EQ(reader->getJointGroupLODs(i), ConstArrayView<std::uint16_t>{expectedLODs});

        const auto& expectedInputIndices = DecodedDNA::jointGroupInputIndices[i][index][0ul];
        ASSERT_EQ(reader->getJointGroupInputIndices(i), ConstArrayView<std::uint16_t>{expectedInputIndices});

        const auto outputIndices = reader->getJointGroupOutputIndices(i);
        ASSERT_EQ(outputIndices.size(), expectedLODs[0ul]);

        ConstArrayView<std::uint16_t> outputIndicesForLOD{outputIndices.data(), expectedLODs[TAPICopyParameters::currentLOD()]};
        const auto& expectedOutputIndices = DecodedDNA::jointGroupOutputIndices[i][index][TAPICopyParameters::currentLOD()];
        ASSERT_EQ(outputIndicesForLOD, ConstArrayView<std::uint16_t>{expectedOutputIndices});

        const auto values = reader->getJointGroupValues(i);
        ASSERT_EQ(values.size(), expectedLODs[0ul] * expectedInputIndices.size());

        ConstArrayView<float> valuesForLOD{values.data(),
                                           expectedLODs[TAPICopyParameters::currentLOD()] * expectedInputIndices.size()};
        const auto& expectedValues = DecodedDNA::jointGroupValues[i][index][TAPICopyParameters::currentLOD()];
        ASSERT_EQ(valuesForLOD, ConstArrayView<float>{expectedValues});

        const auto& expectedJointIndices = DecodedDNA::jointGroupJointIndices[i][index][0ul];
        ASSERT_EQ(reader->getJointGroupJointIndices(i), ConstArrayView<std::uint16_t>{expectedJointIndices});
    }

    ASSERT_EQ(reader->getBlendShapeChannelLODs(), ConstArrayView<std::uint16_t>{DecodedDNA::blendShapeLODs[index]});

    const auto blendShapeChannelInputIndices = reader->getBlendShapeChannelInputIndices();
    ASSERT_EQ(blendShapeChannelInputIndices.size(), DecodedDNA::blendShapeLODs[index][0ul]);
    ConstArrayView<std::uint16_t> blendShapeInputIndicesForLOD{
        blendShapeChannelInputIndices.data(),
        DecodedDNA::blendShapeLODs[index][TAPICopyParameters::currentLOD()]};
    ASSERT_EQ(blendShapeInputIndicesForLOD,
              ConstArrayView<std::uint16_t>{DecodedDNA::blendShapeInputIndices[index][TAPICopyParameters::currentLOD()]});

    const auto blendShapeChannelOutputIndices = reader->getBlendShapeChannelOutputIndices();
    ASSERT_EQ(blendShapeChannelOutputIndices.size(), DecodedDNA::blendShapeLODs[index][0ul]);
    ConstArrayView<std::uint16_t> blendShapeOutputIndicesForLOD{
        blendShapeChannelOutputIndices.data(),
        DecodedDNA::blendShapeLODs[index][TAPICopyParameters::currentLOD()]};
    ASSERT_EQ(blendShapeOutputIndicesForLOD,
              ConstArrayView<std::uint16_t>{DecodedDNA::blendShapeOutputIndices[index][TAPICopyParameters::currentLOD()]});

    ASSERT_EQ(reader->getAnimatedMapLODs(), ConstArrayView<std::uint16_t>{DecodedDNA::animatedMapLODs[index]});

    ASSERT_EQ(reader->getAnimatedMapCount(), DecodedDNA::animatedMapCount[index]);

    const auto animatedMapLOD = DecodedDNA::animatedMapLODs[index][TAPICopyParameters::currentLOD()];

    const auto animatedMapInputIndices = reader->getAnimatedMapInputIndices();
    ASSERT_EQ(animatedMapInputIndices.size(), DecodedDNA::animatedMapLODs[index][0ul]);
    ConstArrayView<std::uint16_t> animatedMapInputIndicesForLOD{animatedMapInputIndices.data(), animatedMapLOD};
    ASSERT_EQ(animatedMapInputIndicesForLOD,
              ConstArrayView<std::uint16_t>{DecodedDNA::conditionalInputIndices[index][TAPICopyParameters::currentLOD()]});

    const auto animatedMapOutputIndices = reader->getAnimatedMapOutputIndices();
    ASSERT_EQ(animatedMapOutputIndices.size(), DecodedDNA::animatedMapLODs[index][0ul]);
    ConstArrayView<std::uint16_t> animatedMapOutputIndicesForLOD{animatedMapOutputIndices.data(), animatedMapLOD};
    ASSERT_EQ(animatedMapOutputIndicesForLOD,
              ConstArrayView<std::uint16_t>{DecodedDNA::conditionalOutputIndices[index][TAPICopyParameters::currentLOD()]});

    const auto animatedMapFromValues = reader->getAnimatedMapFromValues();
    ASSERT_EQ(animatedMapFromValues.size(), DecodedDNA::animatedMapLODs[index][0ul]);
    ConstArrayView<float> animatedMapFromValuesForLOD{animatedMapFromValues.data(), animatedMapLOD};
    ASSERT_EQ(animatedMapFromValuesForLOD,
              ConstArrayView<float>{DecodedDNA::conditionalFromValues[index][TAPICopyParameters::currentLOD()]});

    const auto animatedMapToValues = reader->getAnimatedMapToValues();
    ASSERT_EQ(animatedMapToValues.size(), DecodedDNA::animatedMapLODs[index][0ul]);
    ConstArrayView<float> animatedMapToValuesForLOD{animatedMapToValues.data(), animatedMapLOD};
    ASSERT_EQ(animatedMapToValuesForLOD,
              ConstArrayView<float>{DecodedDNA::conditionalToValues[index][TAPICopyParameters::currentLOD()]});

    const auto animatedMapSlopeValues = reader->getAnimatedMapSlopeValues();
    ASSERT_EQ(animatedMapSlopeValues.size(), DecodedDNA::animatedMapLODs[index][0ul]);
    ConstArrayView<float> animatedMapSlopeValuesForLOD{animatedMapSlopeValues.data(), animatedMapLOD};
    ASSERT_EQ(animatedMapSlopeValuesForLOD,
              ConstArrayView<float>{DecodedDNA::conditionalSlopeValues[index][TAPICopyParameters::currentLOD()]});

    const auto animatedMapCutValues = reader->getAnimatedMapCutValues();
    ASSERT_EQ(animatedMapCutValues.size(), DecodedDNA::animatedMapLODs[index][0ul]);
    ConstArrayView<float> animatedMapCutValuesForLOD{animatedMapCutValues.data(), animatedMapLOD};
    ASSERT_EQ(animatedMapCutValuesForLOD,
              ConstArrayView<float>{DecodedDNA::conditionalCutValues[index][TAPICopyParameters::currentLOD()]});
}

template<class TAPICopyParameters>
static void verifyGeometry(GeometryReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    const auto meshCount = reader->getMeshCount();
    ASSERT_EQ(meshCount, DecodedDNA::meshCount[index]);
    for (std::uint16_t meshIndex = {}; meshIndex < meshCount; ++meshIndex) {
        const auto vertexPositionCount = reader->getVertexPositionCount(meshIndex);
        ASSERT_EQ(vertexPositionCount, DecodedDNA::vertexPositions[index][meshIndex].size());
        for (std::uint32_t vertexIndex = {}; vertexIndex < vertexPositionCount; ++vertexIndex) {
            ASSERT_EQ(reader->getVertexPosition(meshIndex, vertexIndex),
                      DecodedDNA::vertexPositions[index][meshIndex][vertexIndex]);
        }

        const auto vertexTextureCoordinateCount = reader->getVertexTextureCoordinateCount(meshIndex);
        ASSERT_EQ(vertexTextureCoordinateCount, DecodedDNA::vertexTextureCoordinates[index][meshIndex].size());
        for (std::uint32_t texCoordIndex = {}; texCoordIndex < vertexTextureCoordinateCount; ++texCoordIndex) {
            const auto& textureCoordinate = reader->getVertexTextureCoordinate(meshIndex, texCoordIndex);
            const auto& expectedTextureCoordinate = DecodedDNA::vertexTextureCoordinates[index][meshIndex][texCoordIndex];
            ASSERT_EQ(textureCoordinate.u, expectedTextureCoordinate.u);
            ASSERT_EQ(textureCoordinate.v, expectedTextureCoordinate.v);
        }

        const auto vertexNormalCount = reader->getVertexNormalCount(meshIndex);
        ASSERT_EQ(vertexNormalCount, DecodedDNA::vertexNormals[index][meshIndex].size());
        for (std::uint32_t normalIndex = {}; normalIndex < vertexNormalCount; ++normalIndex) {
            ASSERT_EQ(reader->getVertexNormal(meshIndex, normalIndex), DecodedDNA::vertexNormals[index][meshIndex][normalIndex]);
        }

        const auto vertexLayoutCount = reader->getVertexLayoutCount(meshIndex);
        ASSERT_EQ(vertexLayoutCount, DecodedDNA::vertexLayouts[index][meshIndex].size());
        for (std::uint32_t layoutIndex = {}; layoutIndex < vertexLayoutCount; ++layoutIndex) {
            const auto& layout = reader->getVertexLayout(meshIndex, layoutIndex);
            const auto& expectedLayout = DecodedDNA::vertexLayouts[index][meshIndex][layoutIndex];
            ASSERT_EQ(layout.position, expectedLayout.position);
            ASSERT_EQ(layout.textureCoordinate, expectedLayout.textureCoordinate);
            ASSERT_EQ(layout.normal, expectedLayout.normal);
        }

        const auto faceCount = reader->getFaceCount(meshIndex);
        ASSERT_EQ(faceCount, DecodedDNA::faces[index][meshIndex].size());
        for (std::uint32_t faceIndex = {}; faceIndex < faceCount; ++faceIndex) {
            ASSERT_EQ(reader->getFaceVertexLayoutIndices(meshIndex, faceIndex),
                      ConstArrayView<std::uint32_t>{DecodedDNA::faces[index][meshIndex][faceIndex]});
        }

        ASSERT_EQ(reader->getMaximumInfluencePerVertex(meshIndex), DecodedDNA::maxInfluencePerVertex[index][meshIndex]);

        ASSERT_EQ(reader->getSkinWeightsCount(meshIndex), DecodedDNA::skinWeightsValues[index][meshIndex].size());
        for (std::uint32_t vertexIndex = {}; vertexIndex < vertexPositionCount; ++vertexIndex) {
            const auto skinWeights = reader->getSkinWeightsValues(meshIndex, vertexIndex);
            const auto& expectedSkinWeights = DecodedDNA::skinWeightsValues[index][meshIndex][vertexIndex];
            ASSERT_EQ(skinWeights, ConstArrayView<float>{expectedSkinWeights});

            const auto jointIndices = reader->getSkinWeightsJointIndices(meshIndex, vertexIndex);
            const auto& expectedJointIndices = DecodedDNA::skinWeightsJointIndices[index][meshIndex][vertexIndex];
            ASSERT_EQ(jointIndices, ConstArrayView<std::uint16_t>{expectedJointIndices});
        }

        const auto blendShapeCount = reader->getBlendShapeTargetCount(meshIndex);
        ASSERT_EQ(blendShapeCount, DecodedDNA::correctiveBlendShapeDeltas[index][meshIndex].size());
        for (std::uint16_t blendShapeTargetIndex = {}; blendShapeTargetIndex < blendShapeCount; ++blendShapeTargetIndex) {
            const auto channelIndex = reader->getBlendShapeChannelIndex(meshIndex, blendShapeTargetIndex);
            ASSERT_EQ(channelIndex, DecodedDNA::correctiveBlendShapeIndices[index][meshIndex][blendShapeTargetIndex]);

            const auto deltaCount = reader->getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex);
            ASSERT_EQ(deltaCount, DecodedDNA::correctiveBlendShapeDeltas[index][meshIndex][blendShapeTargetIndex].size());

            for (std::uint32_t deltaIndex = {}; deltaIndex < deltaCount; ++deltaIndex) {
                const auto& delta = reader->getBlendShapeTargetDelta(meshIndex, blendShapeTargetIndex, deltaIndex);
                const auto& expectedDelta =
                    DecodedDNA::correctiveBlendShapeDeltas[index][meshIndex][blendShapeTargetIndex][deltaIndex];
                ASSERT_EQ(delta, expectedDelta);
            }

            const auto vertexIndices = reader->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex);
            const auto& expectedVertexIndices =
                DecodedDNA::correctiveBlendShapeVertexIndices[index][meshIndex][blendShapeTargetIndex];
            ASSERT_EQ(vertexIndices, ConstArrayView<std::uint32_t>{expectedVertexIndices});
        }
    }
}

template<class TAPICopyParameters>
static void verifyMachineLearnedBehavior(MachineLearnedBehaviorReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    const auto mlControlCount = reader->getMLControlCount();
    ASSERT_EQ(mlControlCount, DecodedDNA::mlControlNames.size());
    for (std::uint16_t i = {}; i < mlControlCount; ++i) {
        ASSERT_EQ(reader->getMLControlName(i), StringView{DecodedDNA::mlControlNames[i]});
    }

    ASSERT_EQ(reader->getNeuralNetworkCount(), DecodedDNA::neuralNetworkLayerCount[index].size());

    const auto& expectedRegionNames = DecodedDNA::regionNames[index];
    ASSERT_EQ(reader->getMeshCount(), expectedRegionNames.size());
    for (std::uint16_t mi = {}; mi < reader->getMeshCount(); ++mi) {
        ASSERT_EQ(reader->getMeshRegionCount(mi), expectedRegionNames[mi].size());
        for (std::uint16_t ri = {}; ri < expectedRegionNames[mi].size(); ++ri) {
            ASSERT_EQ(reader->getMeshRegionName(mi, ri), StringView{expectedRegionNames[mi][ri]});
        }
    }

    const auto& expectedNetIndices = DecodedDNA::neuralNetworkIndicesPerMeshRegion[index];
    ASSERT_EQ(reader->getMeshCount(), expectedNetIndices.size());
    for (std::uint16_t meshIdx = {}; meshIdx < expectedNetIndices.size(); ++meshIdx) {
        ASSERT_EQ(reader->getMeshRegionCount(meshIdx), expectedNetIndices[meshIdx].size());
        for (std::uint16_t regionIdx = {}; regionIdx < expectedNetIndices[meshIdx].size(); ++regionIdx) {
            const auto indices = reader->getNeuralNetworkIndicesForMeshRegion(meshIdx, regionIdx);
            ASSERT_EQ(indices.size(), expectedNetIndices[meshIdx][regionIdx].size());
            ASSERT_ELEMENTS_EQ(indices, expectedNetIndices[meshIdx][regionIdx], expectedNetIndices[meshIdx][regionIdx].size());
        }
    }

    for (std::uint16_t neuralNetIdx = {}; neuralNetIdx < reader->getNeuralNetworkCount(); ++neuralNetIdx) {
        ASSERT_EQ(reader->getNeuralNetworkInputIndices(neuralNetIdx), DecodedDNA::neuralNetworkInputIndices[index][neuralNetIdx]);
        ASSERT_EQ(reader->getNeuralNetworkOutputIndices(neuralNetIdx),
                  DecodedDNA::neuralNetworkOutputIndices[index][neuralNetIdx]);
        ASSERT_EQ(reader->getNeuralNetworkLayerCount(neuralNetIdx), DecodedDNA::neuralNetworkLayerCount[index][neuralNetIdx]);
        for (std::uint16_t layerIdx = {}; layerIdx < reader->getNeuralNetworkLayerCount(neuralNetIdx); ++layerIdx) {
            const auto expected =
                static_cast<dna::ActivationFunction>(DecodedDNA::neuralNetworkActivationFunction[index][neuralNetIdx][layerIdx]);
            ASSERT_EQ(reader->getNeuralNetworkLayerActivationFunction(neuralNetIdx, layerIdx), expected);
            ASSERT_EQ(reader->getNeuralNetworkLayerActivationFunctionParameters(neuralNetIdx, layerIdx),
                      DecodedDNA::neuralNetworkActivationFunctionParameters[index][neuralNetIdx][layerIdx]);
            ASSERT_EQ(reader->getNeuralNetworkLayerBiases(neuralNetIdx, layerIdx),
                      DecodedDNA::neuralNetworkBiases[index][neuralNetIdx][layerIdx]);
            ASSERT_EQ(reader->getNeuralNetworkLayerWeights(neuralNetIdx, layerIdx),
                      DecodedDNA::neuralNetworkWeights[index][neuralNetIdx][layerIdx]);
        }
    }
}

template<class TAPICopyParameters>
static void verifyRBFBehavior(RBFBehaviorReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    const std::uint16_t solverCount = reader->getRBFSolverCount();
    ASSERT_EQ(solverCount, DecodedDNA::solverIndicesPerLOD[index].size());

    const auto poseCount = reader->getRBFPoseCount();
    ASSERT_EQ(poseCount, DecodedDNA::poseScale.size());
    for (std::uint16_t pi = {}; pi < poseCount; ++pi) {
        ASSERT_EQ(reader->getRBFPoseName(pi), StringView{DecodedDNA::poseNames[pi]});
        ASSERT_EQ(reader->getRBFPoseScale(pi), DecodedDNA::poseScale[pi]);
    }
    for (std::uint16_t si = {}; si < solverCount; ++si) {
        std::uint16_t esi = DecodedDNA::solverIndicesPerLOD[index][si];
        ASSERT_EQ(reader->getRBFSolverName(si), StringView{DecodedDNA::solverNames[esi]});
        ASSERT_EQ(reader->getRBFSolverRawControlIndices(si),
                  ConstArrayView<std::uint16_t>{DecodedDNA::solverRawControlIndices[esi]});
        ASSERT_EQ(reader->getRBFSolverType(si), static_cast<RBFSolverType>(DecodedDNA::solverType[esi]));
        ASSERT_EQ(reader->getRBFSolverAutomaticRadius(si), static_cast<AutomaticRadius>(DecodedDNA::solverAutomaticRadius[esi]));
        ASSERT_EQ(reader->getRBFSolverDistanceMethod(si), static_cast<RBFDistanceMethod>(DecodedDNA::solverDistanceMethod[esi]));
        ASSERT_EQ(reader->getRBFSolverNormalizeMethod(si),
                  static_cast<RBFNormalizeMethod>(DecodedDNA::solverNormalizeMethod[esi]));
        ASSERT_EQ(reader->getRBFSolverFunctionType(si), static_cast<RBFFunctionType>(DecodedDNA::solverFunctionType[esi]));
        ASSERT_EQ(reader->getRBFSolverTwistAxis(si), static_cast<TwistAxis>(DecodedDNA::solverTwistAxis[esi]));
        ASSERT_EQ(reader->getRBFSolverRadius(si), DecodedDNA::solverRadius[esi]);
        ASSERT_EQ(reader->getRBFSolverWeightThreshold(si), DecodedDNA::solverWeightThreshold[esi]);
        auto rawControlIndices = reader->getRBFSolverRawControlIndices(si);
        const auto& expectedRawControlIndices = DecodedDNA::solverRawControlIndices[esi];
        ASSERT_EQ(rawControlIndices.size(), expectedRawControlIndices.size());
        ASSERT_ELEMENTS_EQ(rawControlIndices, expectedRawControlIndices, rawControlIndices.size());

        auto solverPoseIndices = reader->getRBFSolverPoseIndices(si);
        const auto& expectedSolverPoseIndices = DecodedDNA::solverPoseIndices[esi];
        ASSERT_EQ(solverPoseIndices.size(), expectedSolverPoseIndices.size());
        ASSERT_ELEMENTS_EQ(solverPoseIndices, expectedSolverPoseIndices, expectedSolverPoseIndices.size());

        auto solverRawControlValues = reader->getRBFSolverRawControlValues(si);
        const auto& expectedSolverRawControlValues = DecodedDNA::solverRawControlValues[esi];
        ASSERT_EQ(solverRawControlValues.size(), expectedSolverRawControlValues.size());
        ASSERT_ELEMENTS_EQ(solverRawControlValues, expectedSolverRawControlValues, expectedSolverRawControlValues.size());
    }
}

template<class TAPICopyParameters>
static void verifyRBFBehaviorExt(RBFBehaviorReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;

    const auto poseControlCount = reader->getRBFPoseControlCount();
    ASSERT_EQ(poseControlCount, DecodedDNA::poseControlNames.size());
    for (std::uint16_t pci = {}; pci < poseControlCount; ++pci) {
        ASSERT_EQ(reader->getRBFPoseControlName(pci), StringView{DecodedDNA::poseControlNames[pci]});
    }

    const auto poseCount = reader->getRBFPoseCount();
    for (std::uint16_t pi = {}; pi < poseCount; ++pi) {
        auto poseInputControlIndices = reader->getRBFPoseInputControlIndices(pi);
        const auto& expectedPoseInputControlIndices = DecodedDNA::poseInputControlIndices[pi];
        ASSERT_EQ(poseInputControlIndices.size(), expectedPoseInputControlIndices.size());
        ASSERT_ELEMENTS_EQ(poseInputControlIndices, expectedPoseInputControlIndices, expectedPoseInputControlIndices.size());

        auto poseOutputControlIndices = reader->getRBFPoseOutputControlIndices(pi);
        const auto& expectedPoseOutputControlIndices = DecodedDNA::poseOutputControlIndices[pi];
        ASSERT_EQ(poseOutputControlIndices.size(), expectedPoseOutputControlIndices.size());
        ASSERT_ELEMENTS_EQ(poseOutputControlIndices, expectedPoseOutputControlIndices, expectedPoseOutputControlIndices.size());

        auto poseOutputControlWeights = reader->getRBFPoseOutputControlWeights(pi);
        const auto& expectedPoseOutputControlWeights = DecodedDNA::poseOutputControlWeights[pi];
        ASSERT_EQ(poseOutputControlWeights.size(), expectedPoseOutputControlWeights.size());
        ASSERT_ELEMENTS_EQ(poseOutputControlWeights, expectedPoseOutputControlWeights, expectedPoseOutputControlWeights.size());
    }
}

template<class TAPICopyParameters>
static void verifyJointBehaviorMetadata(JointBehaviorMetadataReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    for (const auto ji : reader->getJointIndicesForLOD(TAPICopyParameters::currentLOD())) {
        ASSERT_EQ(reader->getJointTranslationRepresentation(ji), DecodedDNA::jointTranslationRepresentation[index][ji]);
        ASSERT_EQ(reader->getJointRotationRepresentation(ji), DecodedDNA::jointRotationRepresentation[index][ji]);
        ASSERT_EQ(reader->getJointScaleRepresentation(ji), DecodedDNA::jointScaleRepresentation[index][ji]);
    }
}

template<class TAPICopyParameters>
static void verifyTwistSwingBehavior(TwistSwingBehaviorReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    const auto expectedTwistCount = static_cast<std::uint16_t>(DecodedDNA::twistBlendWeights[index].size());
    const auto twistCount = reader->getTwistCount();
    ASSERT_EQ(twistCount, expectedTwistCount);
    for (std::uint16_t ti = {}; ti < twistCount; ++ti) {
        const auto twistInputIndices = reader->getTwistInputControlIndices(ti);
        const auto expectedTwistInputIndices = DecodedDNA::twistInputControlIndices[index][ti];
        ASSERT_EQ(twistInputIndices.size(), expectedTwistInputIndices.size());
        ASSERT_ELEMENTS_EQ(twistInputIndices, expectedTwistInputIndices, expectedTwistInputIndices.size());

        const auto twistOutputIndices = reader->getTwistOutputJointIndices(ti);
        const auto expectedTwistOutputIndices = DecodedDNA::twistOutputJointIndices[index][ti];
        ASSERT_EQ(twistOutputIndices.size(), expectedTwistOutputIndices.size());
        ASSERT_ELEMENTS_EQ(twistOutputIndices, expectedTwistOutputIndices, expectedTwistOutputIndices.size());

        const auto twistBlendWeights = reader->getTwistBlendWeights(ti);
        const auto expectedTwistBlendWeights = DecodedDNA::twistBlendWeights[index][ti];
        ASSERT_EQ(twistBlendWeights.size(), expectedTwistBlendWeights.size());
        ASSERT_ELEMENTS_EQ(twistBlendWeights, expectedTwistBlendWeights, twistBlendWeights.size());

        const auto twistAxis = reader->getTwistSetupTwistAxis(ti);
        const auto expectedTwistAxis = DecodedDNA::twistTwistAxes[index][ti];
        ASSERT_EQ(twistAxis, expectedTwistAxis);
    }

    const auto expectedSwingCount = static_cast<std::uint16_t>(DecodedDNA::swingBlendWeights[index].size());
    const auto swingCount = reader->getSwingCount();
    ASSERT_EQ(swingCount, expectedSwingCount);
    for (std::uint16_t si = {}; si < swingCount; ++si) {
        const auto swingInputIndices = reader->getSwingInputControlIndices(si);
        const auto expectedSwingInputIndices = DecodedDNA::swingInputControlIndices[index][si];
        ASSERT_EQ(swingInputIndices.size(), expectedSwingInputIndices.size());
        ASSERT_ELEMENTS_EQ(swingInputIndices, expectedSwingInputIndices, expectedSwingInputIndices.size());

        const auto swingOutputIndices = reader->getSwingOutputJointIndices(si);
        const auto expectedSwingOutputIndices = DecodedDNA::swingOutputJointIndices[index][si];
        ASSERT_EQ(swingOutputIndices.size(), expectedSwingOutputIndices.size());
        ASSERT_ELEMENTS_EQ(swingOutputIndices, expectedSwingOutputIndices, expectedSwingOutputIndices.size());

        const auto swingBlendWeights = reader->getSwingBlendWeights(si);
        const auto expectedSwingBlendWeights = DecodedDNA::swingBlendWeights[index][si];
        ASSERT_EQ(swingBlendWeights.size(), expectedSwingBlendWeights.size());
        ASSERT_ELEMENTS_EQ(swingBlendWeights, expectedSwingBlendWeights, expectedSwingBlendWeights.size());

        const auto twistAxis = reader->getSwingSetupTwistAxis(si);
        const auto expectedTwistAxis = DecodedDNA::swingTwistAxes[index][si];
        ASSERT_EQ(twistAxis, expectedTwistAxis);
    }
}

template<class TAPICopyParameters>
static void verifyMachineLearnedBehaviorExt(MachineLearnedBehaviorExtReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    const auto index = DecodedDNA::lodConstraintToIndex(TAPICopyParameters::maxLOD(), TAPICopyParameters::minLOD());

    const auto mlTypeCount = reader->getMLTypeCount();
    ASSERT_EQ(mlTypeCount, DecodedDNA::mlOperationTypes[index].size());

    for (std::uint16_t ti = {}; ti < mlTypeCount; ++ti) {
        const auto mlOperationSetCount = reader->getMLOperationSetCount(ti);
        ASSERT_EQ(mlOperationSetCount, DecodedDNA::mlOperationTypes[index][ti].size());
        for (std::uint16_t si = {}; si < mlOperationSetCount; ++si) {
            const auto mlOperationCount = reader->getMLOperationCount(ti, si);
            ASSERT_EQ(mlOperationCount, DecodedDNA::mlOperationTypes[index][ti][si].size());
            for (std::uint16_t oi = {}; oi < mlOperationCount; ++oi) {
                ASSERT_EQ(reader->getMLOperationType(ti, si, oi), DecodedDNA::mlOperationTypes[index][ti][si][oi]);

                const auto params = reader->getMLOperationParameters(ti, si, oi);
                ASSERT_EQ(params.size(), DecodedDNA::mlOperationParameters[index][ti][si][oi].size());
                ASSERT_ELEMENTS_EQ(params, DecodedDNA::mlOperationParameters[index][ti][si][oi], params.size());

                const auto depOpSets = reader->getMLOperationDependencyOperationSetIndices(ti, si, oi);
                ASSERT_EQ(depOpSets.size(), DecodedDNA::mlDependencyOperationSetIndices[index][ti][si][oi].size());
                ASSERT_ELEMENTS_EQ(depOpSets, DecodedDNA::mlDependencyOperationSetIndices[index][ti][si][oi], depOpSets.size());

                const auto depOps = reader->getMLOperationDependencyOperationIndices(ti, si, oi);
                ASSERT_EQ(depOps.size(), DecodedDNA::mlDependencyOperationIndices[index][ti][si][oi].size());
                ASSERT_ELEMENTS_EQ(depOps, DecodedDNA::mlDependencyOperationIndices[index][ti][si][oi], depOps.size());
            }

            const auto opIndicesForLOD = reader->getMLOperationIndicesForLOD(ti, si, TAPICopyParameters::currentLOD());
            const auto& expectedIndicesForLOD =
                DecodedDNA::mlOperationIndicesPerLOD[index][ti][si][TAPICopyParameters::currentLOD()];
            ASSERT_EQ(opIndicesForLOD.size(), expectedIndicesForLOD.size());
            ASSERT_ELEMENTS_EQ(opIndicesForLOD, expectedIndicesForLOD, expectedIndicesForLOD.size());
        }
    }

    const auto mlJointsInputIndices = reader->getMLJointsInputIndices();
    const auto mlJointsOutputIndices = reader->getMLJointsOutputIndices();
    const auto mlJointsParameterKeys = reader->getMLJointsParameterKeys();
    const auto mlJointsParameterValues = reader->getMLJointsParameterValues();
    ASSERT_EQ(mlJointsInputIndices.size(), 0ul);
    ASSERT_EQ(mlJointsOutputIndices.size(), 0ul);
    ASSERT_EQ(mlJointsParameterKeys.size(), 0ul);
    ASSERT_EQ(mlJointsParameterValues.size(), 0ul);
}

template<class TAPICopyParameters>
static void verifyDescriptorEx(DescriptorReader* reader) {
    using DecodedDNA = typename TAPICopyParameters::DecodedData;
    ASSERT_EQ(reader->getRotationSequence(), DecodedDNA::rotationSequence);

    const auto rotationSign = reader->getRotationSign();
    ASSERT_EQ(rotationSign.x, DecodedDNA::rotationSign.x);
    ASSERT_EQ(rotationSign.y, DecodedDNA::rotationSign.y);
    ASSERT_EQ(rotationSign.z, DecodedDNA::rotationSign.z);

    ASSERT_EQ(reader->getFaceWindingOrder(), DecodedDNA::faceWindingOrder);
}

template<class TAPICopyParameters>
struct ReaderDataVerifier {

    static void assertHasAllData(Reader* reader) {
        verifyDescriptor<TAPICopyParameters>(reader);
        verifyDefinition<TAPICopyParameters>(reader);
        verifyBehavior<TAPICopyParameters>(reader);
        verifyGeometry<TAPICopyParameters>(reader);
    }
};

template<class Reader, class Writer, std::uint16_t MaxLOD, std::uint16_t MinLOD, std::uint16_t CurrentLOD>
struct ReaderDataVerifier<APICopyParameters<Reader, Writer, RawV23, DecodedV23, MaxLOD, MinLOD, CurrentLOD>> {

    static void assertHasAllData(Reader* reader) {
        using TAPICopyParameters = APICopyParameters<Reader, Writer, RawV23, DecodedV23, MaxLOD, MinLOD, CurrentLOD>;
        verifyDescriptor<TAPICopyParameters>(reader);
        verifyDefinition<TAPICopyParameters>(reader);
        verifyBehavior<TAPICopyParameters>(reader);
        verifyGeometry<TAPICopyParameters>(reader);
        verifyMachineLearnedBehavior<TAPICopyParameters>(reader);
    }
};

template<class Reader, class Writer, std::uint16_t MaxLOD, std::uint16_t MinLOD, std::uint16_t CurrentLOD>
struct ReaderDataVerifier<APICopyParameters<Reader, Writer, RawV24, DecodedV24, MaxLOD, MinLOD, CurrentLOD>> {

    static void assertHasAllData(Reader* reader) {
        using TAPICopyParameters = APICopyParameters<Reader, Writer, RawV24, DecodedV24, MaxLOD, MinLOD, CurrentLOD>;
        verifyDescriptor<TAPICopyParameters>(reader);
        verifyDefinition<TAPICopyParameters>(reader);
        verifyBehavior<TAPICopyParameters>(reader);
        verifyGeometry<TAPICopyParameters>(reader);
        verifyMachineLearnedBehavior<TAPICopyParameters>(reader);
        verifyRBFBehavior<TAPICopyParameters>(reader);
        verifyJointBehaviorMetadata<TAPICopyParameters>(reader);
        verifyTwistSwingBehavior<TAPICopyParameters>(reader);
    }
};

template<class Reader, class Writer, std::uint16_t MaxLOD, std::uint16_t MinLOD, std::uint16_t CurrentLOD>
struct ReaderDataVerifier<APICopyParameters<Reader, Writer, RawV25, DecodedV25, MaxLOD, MinLOD, CurrentLOD>> {

    static void assertHasAllData(Reader* reader) {
        using TAPICopyParameters = APICopyParameters<Reader, Writer, RawV25, DecodedV25, MaxLOD, MinLOD, CurrentLOD>;
        verifyDescriptor<TAPICopyParameters>(reader);
        verifyDefinition<TAPICopyParameters>(reader);
        verifyBehavior<TAPICopyParameters>(reader);
        verifyGeometry<TAPICopyParameters>(reader);
        verifyMachineLearnedBehavior<TAPICopyParameters>(reader);
        verifyRBFBehavior<TAPICopyParameters>(reader);
        verifyRBFBehaviorExt<TAPICopyParameters>(reader);
        verifyJointBehaviorMetadata<TAPICopyParameters>(reader);
        verifyTwistSwingBehavior<TAPICopyParameters>(reader);
    }
};

template<class Reader, class Writer, std::uint16_t MaxLOD, std::uint16_t MinLOD, std::uint16_t CurrentLOD>
struct ReaderDataVerifier<APICopyParameters<Reader, Writer, RawV26, DecodedV26, MaxLOD, MinLOD, CurrentLOD>> {

    static void assertHasAllData(Reader* reader) {
        using TAPICopyParameters = APICopyParameters<Reader, Writer, RawV26, DecodedV26, MaxLOD, MinLOD, CurrentLOD>;
        verifyDescriptor<TAPICopyParameters>(reader);
        verifyDefinition<TAPICopyParameters>(reader);
        verifyBehavior<TAPICopyParameters>(reader);
        verifyGeometry<TAPICopyParameters>(reader);
        verifyMachineLearnedBehavior<TAPICopyParameters>(reader);
        verifyRBFBehavior<TAPICopyParameters>(reader);
        verifyRBFBehaviorExt<TAPICopyParameters>(reader);
        verifyJointBehaviorMetadata<TAPICopyParameters>(reader);
        verifyTwistSwingBehavior<TAPICopyParameters>(reader);
        verifyMachineLearnedBehaviorExt<TAPICopyParameters>(reader);
    }
};

template<class Reader, class Writer, std::uint16_t MaxLOD, std::uint16_t MinLOD, std::uint16_t CurrentLOD>
struct ReaderDataVerifier<APICopyParameters<Reader, Writer, RawV27, DecodedV27, MaxLOD, MinLOD, CurrentLOD>> {

    static void assertHasAllData(Reader* reader) {
        using TAPICopyParameters = APICopyParameters<Reader, Writer, RawV27, DecodedV27, MaxLOD, MinLOD, CurrentLOD>;
        verifyDescriptor<TAPICopyParameters>(reader);
        verifyDefinition<TAPICopyParameters>(reader);
        verifyBehavior<TAPICopyParameters>(reader);
        verifyGeometry<TAPICopyParameters>(reader);
        verifyMachineLearnedBehavior<TAPICopyParameters>(reader);
        verifyRBFBehavior<TAPICopyParameters>(reader);
        verifyRBFBehaviorExt<TAPICopyParameters>(reader);
        verifyJointBehaviorMetadata<TAPICopyParameters>(reader);
        verifyTwistSwingBehavior<TAPICopyParameters>(reader);
        verifyMachineLearnedBehaviorExt<TAPICopyParameters>(reader);
        verifyDescriptorEx<TAPICopyParameters>(reader);
    }
};

template<class Reader, class Writer, std::uint16_t MaxLOD, std::uint16_t MinLOD, std::uint16_t CurrentLOD>
struct ReaderDataVerifier<APICopyParameters<Reader, Writer, RawV28, DecodedV28, MaxLOD, MinLOD, CurrentLOD>> {

    static void assertHasAllData(Reader* reader) {
        using TAPICopyParameters = APICopyParameters<Reader, Writer, RawV28, DecodedV28, MaxLOD, MinLOD, CurrentLOD>;
        verifyDescriptor<TAPICopyParameters>(reader);
        verifyDefinition<TAPICopyParameters>(reader);
        verifyBehavior<TAPICopyParameters>(reader);
        verifyGeometry<TAPICopyParameters>(reader);
        verifyMachineLearnedBehavior<TAPICopyParameters>(reader);
        verifyRBFBehavior<TAPICopyParameters>(reader);
        verifyRBFBehaviorExt<TAPICopyParameters>(reader);
        verifyJointBehaviorMetadata<TAPICopyParameters>(reader);
        verifyTwistSwingBehavior<TAPICopyParameters>(reader);
        verifyMachineLearnedBehaviorExt<TAPICopyParameters>(reader);
        verifyDescriptorEx<TAPICopyParameters>(reader);
    }
};

using TAPICopyTestParameters =
    ::testing::Types<APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV21, DecodedV21, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV21, DecodedV21, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV21, DecodedV21, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV21, DecodedV21, 1u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV22, DecodedV22, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV22, DecodedV22, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV22, DecodedV22, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV22, DecodedV22, 1u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV23, DecodedV23, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV23, DecodedV23, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV23, DecodedV23, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV23, DecodedV23, 1u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV24, DecodedV24, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV24, DecodedV24, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV24, DecodedV24, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV24, DecodedV24, 1u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV25, DecodedV25, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV25, DecodedV25, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV25, DecodedV25, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV25, DecodedV25, 1u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV26, DecodedV26, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV26, DecodedV26, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV26, DecodedV26, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV26, DecodedV26, 1u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV27, DecodedV27, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV27, DecodedV27, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV27, DecodedV27, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV27, DecodedV27, 1u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV28, DecodedV28, 0u, 1u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV28, DecodedV28, 0u, 1u, 1u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV28, DecodedV28, 0u, 0u, 0u>,
                     APICopyParameters<dna::BinaryStreamReader, dna::BinaryStreamWriter, RawV28, DecodedV28, 1u, 1u, 0u>
#ifdef DNA_BUILD_WITH_JSON_SUPPORT
                     ,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV21, DecodedV21, 0u, 1u, 0u>,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV22, DecodedV22, 0u, 1u, 0u>,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV23, DecodedV23, 0u, 1u, 0u>,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV24, DecodedV24, 0u, 1u, 0u>,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV25, DecodedV25, 0u, 1u, 0u>,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV26, DecodedV26, 0u, 1u, 0u>,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV27, DecodedV27, 0u, 1u, 0u>,
                     APICopyParameters<dna::JSONStreamReader, dna::JSONStreamWriter, RawV28, DecodedV28, 0u, 1u, 0u>
#endif  // DNA_BUILD_WITH_JSON_SUPPORT
                     >;
TYPED_TEST_SUITE(StreamReadWriteAPICopyIntegrationTest, TAPICopyTestParameters, );

template<typename TReader>
struct ReaderFactory;

template<>
struct ReaderFactory<dna::BinaryStreamReader> {
    static pma::ScopedPtr<dna::BinaryStreamReader> create(trio::BoundedIOStream* stream,
                                                          dna::DataLayer layer,
                                                          dna::UnknownLayerPolicy policy,
                                                          std::uint16_t maxLOD,
                                                          std::uint16_t minLOD) {
        dna::Configuration config;
        config.layer = layer;
        config.unknownLayerPolicy = policy;
        config.maxLOD = maxLOD;
        config.minLOD = minLOD;
        return pma::makeScoped<dna::BinaryStreamReader>(stream, config);
    }
};

#ifdef DNA_BUILD_WITH_JSON_SUPPORT
template<>
struct ReaderFactory<dna::JSONStreamReader> {
    static pma::ScopedPtr<dna::JSONStreamReader> create(trio::BoundedIOStream* stream,
                                                        dna::DataLayer /*unused*/,
                                                        dna::UnknownLayerPolicy /*unused*/,
                                                        std::uint16_t /*unused*/,
                                                        std::uint16_t /*unused*/) {
        return pma::makeScoped<dna::JSONStreamReader>(stream);
    }
};
#endif  // DNA_BUILD_WITH_JSON_SUPPORT

TYPED_TEST(StreamReadWriteAPICopyIntegrationTest, VerifyAllDNADataAfterSetFromThroughAPI) {
    using CurrentParameters = typename TestFixture::Parameters;

    const auto bytes = CurrentParameters::RawBytes::getBytes();
    auto source = pma::makeScoped<trio::MemoryStream>();
    source->write(reinterpret_cast<const char*>(bytes.data()), bytes.size());
    source->seek(0);

    auto sourceReader = pma::makeScoped<BinaryStreamReader>(source.get());
    sourceReader->read();

    auto clone = pma::makeScoped<trio::MemoryStream>();
    auto cloneWriter = pma::makeScoped<typename CurrentParameters::Writer>(clone.get());
    // Due to the abstract Reader type, the API copy method will be invoked
    cloneWriter->setFrom(static_cast<Reader*>(sourceReader.get()));
    cloneWriter->write();

    clone->seek(0ul);
    using Factory = ReaderFactory<typename CurrentParameters::Reader>;
    auto cloneReader = Factory::create(clone.get(),
                                       dna::DataLayer::All,
                                       dna::UnknownLayerPolicy::Preserve,
                                       CurrentParameters::maxLOD(),
                                       CurrentParameters::minLOD());
    cloneReader->read();

    ReaderDataVerifier<CurrentParameters>::assertHasAllData(cloneReader.get());
}

using TRawCopyTestParameters = ::testing::Types<
    // Copy tests
    RawCopyParameters<RawV21, RawV21, UnknownLayerPolicy::Preserve, 2, 1>,
    RawCopyParameters<RawV21, RawV21, UnknownLayerPolicy::Ignore, 2, 1>,
    RawCopyParameters<RawV22, RawV22, UnknownLayerPolicy::Preserve, 2, 2>,
    RawCopyParameters<RawV22, RawV22WithUnknownDataIgnoredAndDNARewritten, UnknownLayerPolicy::Ignore, 2, 2>,
    RawCopyParameters<RawV2xNewer,
                      RawV2xNewerWithUnknownDataPreservedAndDNARewritten,
                      UnknownLayerPolicy::Preserve,
                      2,
                      static_cast<std::uint16_t>(-1)>,
    RawCopyParameters<RawV2xNewer,
                      RawV2xNewerWithUnknownDataIgnoredAndDNARewritten,
                      UnknownLayerPolicy::Ignore,
                      2,
                      static_cast<std::uint16_t>(-1)>,
    RawCopyParameters<RawV23, RawV23, UnknownLayerPolicy::Preserve, 2, 3>,
    RawCopyParameters<RawV23, RawV23, UnknownLayerPolicy::Ignore, 2, 3>,
    RawCopyParameters<RawV24, RawV24, UnknownLayerPolicy::Preserve, 2, 4>,
    RawCopyParameters<RawV24, RawV24, UnknownLayerPolicy::Ignore, 2, 4>,
    RawCopyParameters<RawV25, RawV25, UnknownLayerPolicy::Preserve, 2, 5>,
    RawCopyParameters<RawV25, RawV25, UnknownLayerPolicy::Ignore, 2, 5>,
    RawCopyParameters<RawV26, RawV26, UnknownLayerPolicy::Preserve, 2, 6>,
    RawCopyParameters<RawV26, RawV26, UnknownLayerPolicy::Ignore, 2, 6>,
    RawCopyParameters<RawV27, RawV27, UnknownLayerPolicy::Preserve, 2, 7>,
    RawCopyParameters<RawV27, RawV27, UnknownLayerPolicy::Ignore, 2, 7>,
    RawCopyParameters<RawV28, RawV28, UnknownLayerPolicy::Preserve, 2, 8>,
    RawCopyParameters<RawV28, RawV28, UnknownLayerPolicy::Ignore, 2, 8>,
    // File format conversion tests
    RawCopyParameters<RawV21, RawV22WithUnknownDataIgnoredAndDNARewritten, UnknownLayerPolicy::Preserve, 2, 2>,
    RawCopyParameters<RawV21, RawV22WithUnknownDataIgnoredAndDNARewritten, UnknownLayerPolicy::Ignore, 2, 2>,
    RawCopyParameters<RawV22, RawV21, UnknownLayerPolicy::Preserve, 2, 1>,
    RawCopyParameters<RawV22, RawV21, UnknownLayerPolicy::Ignore, 2, 1>,
    RawCopyParameters<RawV2xNewer, RawV22WithUnknownDataFromNewer2x, UnknownLayerPolicy::Preserve, 2, 2>,
    RawCopyParameters<RawV2xNewer, RawV22Empty, UnknownLayerPolicy::Ignore, 2, 2>,
    RawCopyParameters<RawV22Empty, RawV22Empty, UnknownLayerPolicy::Preserve, 2, 2>,
    RawCopyParameters<RawV22Empty, RawV22Empty, UnknownLayerPolicy::Ignore, 2, 2>,
    RawCopyParameters<RawV23, RawV22DowngradedFromV23, UnknownLayerPolicy::Preserve, 2, 2>,
    RawCopyParameters<RawV23, RawV22WithUnknownDataIgnoredAndDNARewritten, UnknownLayerPolicy::Ignore, 2, 2>,
    RawCopyParameters<RawV24, RawV23DowngradedFromV24, UnknownLayerPolicy::Preserve, 2, 3>,
    RawCopyParameters<RawV24, RawV23, UnknownLayerPolicy::Ignore, 2, 3>,
    RawCopyParameters<RawV25, RawV24DowngradedFromV25, UnknownLayerPolicy::Preserve, 2, 4>,
    RawCopyParameters<RawV25, RawV24, UnknownLayerPolicy::Ignore, 2, 4>,
    RawCopyParameters<RawV26, RawV25DowngradedFromV26, UnknownLayerPolicy::Preserve, 2, 5>,
    RawCopyParameters<RawV26, RawV25, UnknownLayerPolicy::Ignore, 2, 5>,
    RawCopyParameters<RawV27, RawV26DowngradedFromV27, UnknownLayerPolicy::Preserve, 2, 6>,
    RawCopyParameters<RawV27, RawV26, UnknownLayerPolicy::Ignore, 2, 6>,
    RawCopyParameters<RawV28, RawV27DowngradedFromV28, UnknownLayerPolicy::Preserve, 2, 7>,
    RawCopyParameters<RawV28, RawV27, UnknownLayerPolicy::Ignore, 2, 7>>;
TYPED_TEST_SUITE(StreamReadWriteRawCopyIntegrationTest, TRawCopyTestParameters, );

TYPED_TEST(StreamReadWriteRawCopyIntegrationTest, VerifySetFromCopiesEvenUnknownData) {
    using CurrentParameters = typename TestFixture::Parameters;

    const auto bytes = CurrentParameters::RawBytes::getBytes();
    auto source = pma::makeScoped<trio::MemoryStream>();
    source->write(reinterpret_cast<const char*>(bytes.data()), bytes.size());
    source->seek(0);

    dna::Configuration config;
    config.unknownLayerPolicy = CurrentParameters::policy();
    auto sourceReader = pma::makeScoped<BinaryStreamReader>(source.get(), config);
    sourceReader->read();

    auto clone = pma::makeScoped<trio::MemoryStream>();
    auto cloneWriter = pma::makeScoped<BinaryStreamWriter>(clone.get());
    cloneWriter->setFrom(sourceReader.get(), DataLayer::All, CurrentParameters::policy());
    cloneWriter->setFileFormatGeneration(CurrentParameters::generation());
    cloneWriter->setFileFormatVersion(CurrentParameters::version());
    cloneWriter->write();

    clone->seek(0ul);

#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
    const auto cloneSize = static_cast<std::size_t>(clone->size());
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
    std::vector<char> copiedBytes(cloneSize);
    clone->read(copiedBytes.data(), cloneSize);

    const auto expectedBytes = CurrentParameters::ExpectedBytes::getBytes();
    ASSERT_EQ(expectedBytes.size(), copiedBytes.size());
    ASSERT_EQ(expectedBytes, copiedBytes);
}

#ifdef DNA_BUILD_WITH_JSON_SUPPORT
TEST(StreamReadWriteIntegrationTest, ReadWriteJSON) {
    auto stream = pma::makeScoped<trio::MemoryStream>();
    auto writer = pma::makeScoped<JSONStreamWriter>(stream.get(), 4u);

    writer->setMeshName(0, "mesh0");
    const Position vertices[] = {Position{0.0f, 1.0f, 2.0}, Position{3.0f, 4.0f, 5.0}};
    writer->setVertexPositions(0u, vertices, 2u);
    writer->write();

    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
    pma::Vector<char> json(static_cast<std::size_t>(stream->size()));
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif

    pma::String<char> expected = jsonDNA;
    stream->seek(0ul);
    stream->read(json.data(), json.size());
    ASSERT_EQ(json.size(), expected.size());
    ASSERT_ELEMENTS_EQ(json.data(), expected.data(), expected.size());

    stream->seek(0ul);
    auto reader = pma::makeScoped<JSONStreamReader>(stream.get());
    reader->read();
    ASSERT_TRUE(dna::Status::isOk());
}
#endif  // DNA_BUILD_WITH_JSON_SUPPORT

using TReadWriteMultipleParameters =
    ::testing::Types<ReadWriteMultipleParameters<RawV21>,
                     ReadWriteMultipleParameters<RawV22>,
                     ReadWriteMultipleParameters<RawV23>,
                     ReadWriteMultipleParameters<RawV24>,
                     ReadWriteMultipleParameters<RawV25>,
                     ReadWriteMultipleParameters<RawV26>,
                     ReadWriteMultipleParameters<RawV27>,
                     ReadWriteMultipleParameters<RawV28>,
                     ReadWriteMultipleParameters<RawV22Empty>,
                     ReadWriteMultipleParameters<RawV22WithUnknownDataIgnoredAndDNARewritten>,
                     ReadWriteMultipleParameters<RawV2xNewerWithUnknownDataIgnoredAndDNARewritten>,
                     ReadWriteMultipleParameters<RawV2xNewerWithUnknownDataPreservedAndDNARewritten>,
                     ReadWriteMultipleParameters<RawV22WithUnknownDataFromNewer2x>,
                     ReadWriteMultipleParameters<RawV2xNewer>,
                     ReadWriteMultipleParameters<RawV22DowngradedFromV23>>;
TYPED_TEST_SUITE(StreamReadWriteMultipleIntegrationTest, TReadWriteMultipleParameters, );

TYPED_TEST(StreamReadWriteMultipleIntegrationTest, ReadWriteTwoDNAsToSameStream) {
    using CurrentParameters = typename TestFixture::Parameters;

    const auto bytes = CurrentParameters::RawBytes::getBytes();
    auto source = pma::makeScoped<trio::MemoryStream>();
    source->write(reinterpret_cast<const char*>(bytes.data()), bytes.size());

    source->seek(0);
    auto sourceReader = pma::makeScoped<BinaryStreamReader>(source.get());
    sourceReader->read();
    ASSERT_TRUE(dna::Status::isOk());

    auto clone = pma::makeScoped<trio::MemoryStream>();
    auto cloneWriter1 = pma::makeScoped<BinaryStreamWriter>(clone.get());
    cloneWriter1->setFrom(sourceReader.get(), DataLayer::All, UnknownLayerPolicy::Preserve);
    cloneWriter1->write();
    ASSERT_TRUE(dna::Status::isOk());

    // Stream position is reset on open / close of stream (by implementation of trio::MemoryStream)
    const std::uint64_t firstDNASize = clone->size();
    clone->seek(firstDNASize);

    auto cloneWriter2 = pma::makeScoped<BinaryStreamWriter>(clone.get());
    cloneWriter2->setFrom(sourceReader.get(), DataLayer::All, UnknownLayerPolicy::Preserve);
    cloneWriter2->write();
    ASSERT_TRUE(dna::Status::isOk());

    clone->seek(0ul);

    auto cloneReader1 = pma::makeScoped<BinaryStreamReader>(clone.get());
    cloneReader1->read();
    ASSERT_TRUE(dna::Status::isOk());

    // Stream position is reset on open / close of stream (by implementation of trio::MemoryStream)
    clone->seek(firstDNASize);

    auto cloneReader2 = pma::makeScoped<BinaryStreamReader>(clone.get());
    cloneReader2->read();
    ASSERT_TRUE(dna::Status::isOk());

    auto cloneRewritten = pma::makeScoped<trio::MemoryStream>();
    auto cloneRewriter1 = pma::makeScoped<BinaryStreamWriter>(cloneRewritten.get());
    cloneRewriter1->setFrom(cloneReader1.get(), DataLayer::All, UnknownLayerPolicy::Preserve);
    cloneRewriter1->write();
    ASSERT_TRUE(dna::Status::isOk());

    // Stream position is reset on open / close of stream (by implementation of trio::MemoryStream)
    cloneRewritten->seek(cloneRewritten->size());

    auto cloneRewriter2 = pma::makeScoped<BinaryStreamWriter>(cloneRewritten.get());
    cloneRewriter2->setFrom(cloneReader2.get(), DataLayer::All, UnknownLayerPolicy::Preserve);
    cloneRewriter2->write();
    ASSERT_TRUE(dna::Status::isOk());

    clone->seek(0ul);
    cloneRewritten->seek(0ul);

#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
    const auto cloneSize = static_cast<std::size_t>(clone->size());
    const auto cloneRewrittenSize = static_cast<std::size_t>(cloneRewritten->size());
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
    std::vector<char> copiedCloneBytes(cloneSize);
    clone->read(copiedCloneBytes.data(), cloneSize);

    std::vector<char> copiedCloneRewrittenBytes(cloneRewrittenSize);
    cloneRewritten->read(copiedCloneRewrittenBytes.data(), cloneRewrittenSize);

    ASSERT_EQ(cloneSize, cloneRewrittenSize);
    ASSERT_EQ(copiedCloneBytes, copiedCloneRewrittenBytes);
}

TEST(StreamReadWriteIntegrationTest, RawCopyPreservesDataAfterCoordinateSystemTransformOfV21) {
    const auto bytes = RawV21::getBytes();
    auto source = pma::makeScoped<trio::MemoryStream>();
    source->write(bytes.data(), bytes.size());
    source->seek(0);

    // The v2.1 fixture uses (right, up, front); requesting a different destination coordinate
    // system triggers the conversion, which also upgrades the in-memory file version to v2.8.
    dna::Configuration config;
    config.coordinateSystemTransformPolicy = CoordinateSystemTransformPolicy::Transform;
    config.coordinateSystem = {Direction::left, Direction::front, Direction::up};
    auto sourceReader = pma::makeScoped<BinaryStreamReader>(source.get(), config);
    sourceReader->read();
    ASSERT_TRUE(dna::Status::isOk());

    auto clone = pma::makeScoped<trio::MemoryStream>();
    auto cloneWriter = pma::makeScoped<BinaryStreamWriter>(clone.get());
    // The BinaryStreamReader overload of setFrom performs a raw-copy, which serializes only
    // indexed layers (v2.1 sources carry no index in-file, so the entries must be synthesized)
    cloneWriter->setFrom(sourceReader.get(), DataLayer::All, UnknownLayerPolicy::Preserve);
    cloneWriter->write();
    ASSERT_TRUE(dna::Status::isOk());

    clone->seek(0ul);
    auto cloneReader = pma::makeScoped<BinaryStreamReader>(clone.get());
    cloneReader->read();
    ASSERT_TRUE(dna::Status::isOk());

    const auto coordinateSystem = cloneReader->getCoordinateSystem();
    ASSERT_EQ(coordinateSystem.x, Direction::left);
    ASSERT_EQ(coordinateSystem.y, Direction::front);
    ASSERT_EQ(coordinateSystem.z, Direction::up);

    ASSERT_NE(cloneReader->getJointCount(), 0u);
    ASSERT_EQ(cloneReader->getJointCount(), sourceReader->getJointCount());
    for (std::uint16_t ji = {}; ji < cloneReader->getJointCount(); ++ji) {
        ASSERT_EQ(cloneReader->getNeutralJointTranslation(ji), sourceReader->getNeutralJointTranslation(ji));
    }

    ASSERT_NE(cloneReader->getMeshCount(), 0u);
    ASSERT_EQ(cloneReader->getMeshCount(), sourceReader->getMeshCount());
    for (std::uint16_t mi = {}; mi < cloneReader->getMeshCount(); ++mi) {
        ASSERT_NE(cloneReader->getVertexPositionCount(mi), 0u);
        ASSERT_EQ(cloneReader->getVertexPositionCount(mi), sourceReader->getVertexPositionCount(mi));
    }
}

TEST(StreamReadWriteIntegrationTest, CoordinateSystemTransformRemapsTwistAxes) {
    const auto bytes = RawV24::getBytes();
    auto source = pma::makeScoped<trio::MemoryStream>();
    source->write(bytes.data(), bytes.size());
    source->seek(0);

    // The v2.4 fixture uses (right, up, front); the destination cycles all three axis labels
    // (X -> Z, Y -> X, Z -> Y) while keeping handedness, so every stored twist axis label
    // must follow the relabeling of the joint-local frames.
    dna::Configuration config;
    config.coordinateSystemTransformPolicy = CoordinateSystemTransformPolicy::Transform;
    config.coordinateSystem = {Direction::up, Direction::front, Direction::right};
    auto reader = pma::makeScoped<BinaryStreamReader>(source.get(), config);
    reader->read();
    ASSERT_TRUE(dna::Status::isOk());

    ASSERT_EQ(reader->getTwistCount(), 3u);
    ASSERT_EQ(reader->getTwistSetupTwistAxis(0), TwistAxis::Z);
    ASSERT_EQ(reader->getTwistSetupTwistAxis(1), TwistAxis::X);
    ASSERT_EQ(reader->getTwistSetupTwistAxis(2), TwistAxis::Y);

    ASSERT_EQ(reader->getSwingCount(), 3u);
    ASSERT_EQ(reader->getSwingSetupTwistAxis(0), TwistAxis::Z);
    ASSERT_EQ(reader->getSwingSetupTwistAxis(1), TwistAxis::X);
    ASSERT_EQ(reader->getSwingSetupTwistAxis(2), TwistAxis::Y);

    ASSERT_EQ(reader->getRBFSolverCount(), 3u);
    ASSERT_EQ(reader->getRBFSolverTwistAxis(0), TwistAxis::Z);
    ASSERT_EQ(reader->getRBFSolverTwistAxis(1), TwistAxis::X);
    ASSERT_EQ(reader->getRBFSolverTwistAxis(2), TwistAxis::Z);
}

TEST(StreamReadWriteIntegrationTest, CoordinateSystemTransformPreservesRBFQuaternionHemisphere) {
    auto stream = pma::makeScoped<trio::MemoryStream>();
    auto writer = pma::makeScoped<BinaryStreamWriter>(stream.get());
    writer->setCoordinateSystem({Direction::left, Direction::front, Direction::up});
    writer->setRBFSolverName(0, "solver");
    const float rawControlValues[] = {
        0.0f,
        0.0f,
        0.0f,
        1.0f,  // Identity
        0.0f,
        0.0f,
        0.0f,
        -1.0f,  // Negative identity: same rotation, opposite hemisphere
        0.70710678f,
        0.0f,
        0.0f,
        0.70710678f,  // +90 degree twist about X
        0.0f,
        0.86602540f,
        0.0f,
        -0.5f  // 240 degrees about Y, stored on the negative hemisphere
    };
    writer->setRBFSolverRawControlValues(0, rawControlValues, 16u);
    writer->write();
    ASSERT_TRUE(dna::Status::isOk());

    stream->seek(0);
    // (left, front, up) -> (left, back, up) is a handedness-flipping sign change of the Y
    // component, under which quaternion imaginary parts map as v -> det(C) * (v * C) =
    // (-vx, vy, -vz), with w untouched. The quaternion hemisphere must survive: half-rotation
    // RBF solvers encode driver rotations beyond 180 degrees purely in the sign of the pose
    // quaternion, which an euler-angle roundtrip would silently canonicalize away.
    dna::Configuration config;
    config.coordinateSystemTransformPolicy = CoordinateSystemTransformPolicy::Transform;
    config.coordinateSystem = {Direction::left, Direction::back, Direction::up};
    auto reader = pma::makeScoped<BinaryStreamReader>(stream.get(), config);
    reader->read();
    ASSERT_TRUE(dna::Status::isOk());

    ASSERT_EQ(reader->getRBFSolverCount(), 1u);
    const auto converted = reader->getRBFSolverRawControlValues(0);
    ASSERT_EQ(converted.size(), 16u);
    const float expected[] = {
        0.0f,
        0.0f,
        0.0f,
        1.0f,  // Identity
        0.0f,
        0.0f,
        0.0f,
        -1.0f,  // Hemisphere must survive the conversion
        -0.70710678f,
        0.0f,
        0.0f,
        0.70710678f,  // Twist sense flips with handedness
        0.0f,
        0.86602540f,
        0.0f,
        -0.5f  // Imaginary Y and hemisphere both preserved
    };
    for (std::size_t i = {}; i < converted.size(); ++i) {
        ASSERT_NEAR(converted[i], expected[i], 0.0001f) << "quaternion component " << i;
    }
}

TEST(StreamReadWriteMultipleIntegrationTest, DNAv25LayerIsBackFilledFromv24) {
    const auto bytes = RawV24::getBytes();
    auto source = pma::makeScoped<trio::MemoryStream>();
    source->write(bytes.data(), bytes.size());
    source->seek(0);
    auto reader = pma::makeScoped<BinaryStreamReader>(source.get());
    reader->read();

    ASSERT_TRUE(dna::Status::isOk());
    ASSERT_EQ(reader->getRBFPoseControlCount(), reader->getRBFPoseCount());
    for (std::uint16_t pi = {}; pi < reader->getRBFPoseCount(); ++pi) {
        const auto inputControlIndices = reader->getRBFPoseInputControlIndices(pi);
        const auto outputControlIndices = reader->getRBFPoseOutputControlIndices(pi);
        const auto outputControlWeights = reader->getRBFPoseOutputControlWeights(pi);
        ASSERT_EQ(inputControlIndices.size(), 0ul);
        ASSERT_EQ(outputControlIndices.size(), 1ul);
        ASSERT_EQ(outputControlWeights.size(), 1ul);
        const auto offset = reader->getRawControlCount() + reader->getPSDCount() + reader->getMLControlCount();
        ASSERT_EQ(outputControlIndices[0], offset + pi);
        ASSERT_EQ(outputControlWeights[0], 1.0f);
    }
}

}  // namespace dna
