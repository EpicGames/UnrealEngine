// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "dnatests/Defs.h"

#include "dna/DNA.h"

template<typename TReader,
         typename TWriter,
         typename TRawBytes,
         typename TDecodedData,
         std::uint16_t MaxLOD,
         std::uint16_t MinLOD,
         std::uint16_t CurrentLOD>
struct APICopyParameters {
    using Reader = TReader;
    using Writer = TWriter;
    using RawBytes = TRawBytes;
    using DecodedData = TDecodedData;

    static constexpr std::uint16_t maxLOD() {
        return MaxLOD;
    }

    static constexpr std::uint16_t minLOD() {
        return MinLOD;
    }

    static constexpr std::uint16_t currentLOD() {
        return CurrentLOD;
    }
};

template<typename TAPICopyParameters>
class StreamReadWriteAPICopyIntegrationTest : public ::testing::Test {
protected:
    using Parameters = TAPICopyParameters;
};

template<typename TRawBytes,
         typename TExpectedBytes,
         dna::UnknownLayerPolicy Policy,
         std::uint16_t SaveAsGeneration,
         std::uint16_t SaveAsVersion>
struct RawCopyParameters {
    using RawBytes = TRawBytes;
    using ExpectedBytes = TExpectedBytes;

    static constexpr dna::UnknownLayerPolicy policy() {
        return Policy;
    }

    static constexpr std::uint16_t generation() {
        return SaveAsGeneration;
    }

    static constexpr std::uint16_t version() {
        return SaveAsVersion;
    }
};

template<typename TRawCopyParameters>
class StreamReadWriteRawCopyIntegrationTest : public ::testing::Test {
protected:
    using Parameters = TRawCopyParameters;
};

template<typename TRawBytes>
struct ReadWriteMultipleParameters {
    using RawBytes = TRawBytes;
};

template<typename TReadWriteMultipleParameters>
class StreamReadWriteMultipleIntegrationTest : public ::testing::Test {
protected:
    using Parameters = TReadWriteMultipleParameters;
};
