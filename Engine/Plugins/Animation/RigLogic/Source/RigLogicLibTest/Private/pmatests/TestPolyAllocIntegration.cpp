// Copyright Epic Games, Inc. All Rights Reserved.

#include "pmatests/Defs.h"

#include "pma/TypeDefs.h"

// Old clang compiler in combination with new libstdc++
#if defined(__clang__) && (__clang_major__ < 9) && defined(_GLIBCXX_RELEASE) && (_GLIBCXX_RELEASE >= 10)
    #define PMA_OLD_CLANG_NEW_LIBSTDCPP
#endif

TEST(PolyAllocIntegrationTest, InstantiateTypes) {
    pma::String<char> str;
    pma::Vector<int> vec;
    pma::Matrix<int> mat;
#ifndef PMA_OLD_CLANG_NEW_LIBSTDCPP
    pma::Set<int> set;
    pma::Map<int, int> map;
#endif  // PMA_OLD_CLANG_NEW_LIBSTDCPP
    pma::UnorderedSet<int> uset;
    pma::UnorderedMap<int, int> umap;
    ASSERT_TRUE(true);
}
