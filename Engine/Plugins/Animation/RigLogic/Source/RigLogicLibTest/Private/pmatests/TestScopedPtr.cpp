// Copyright Epic Games, Inc. All Rights Reserved.

#include "pmatests/Defs.h"

#include <pma/ScopedPtr.h>

namespace pmatests {

namespace {

struct Counters {
    int constructed;
    int destructed;
};

class Client {
public:
    static Client* create(Counters& counters) {
        return new Client(counters);
    }

    static void destroy(Client* instance) {
        delete instance;
    }

    Client(Counters& counters_) :
        counters{&counters_} {
        counters->constructed += 1;
    }

    ~Client() {
        counters->destructed += 1;
    }

    Client(const Client&) = default;
    Client& operator=(const Client&) = default;

    Client(Client&&) = default;
    Client& operator=(Client&&) = default;

private:
    Counters* counters;
};

struct Base {
    virtual ~Base() = default;
};

struct Derived : public Base {

    static Derived* create() {
        return new Derived{};
    }

    static void destroy(Derived* ptr) {
        delete ptr;
    }
};

}  // namespace

}  // namespace pmatests

TEST(ScopedPtrTest, EmptyConstruction) {
    pma::ScopedPtr<pmatests::Client> sp;
    ASSERT_EQ(sp.get(), nullptr);
    ASSERT_FALSE(sp);
}

TEST(ScopedPtrTest, ProperCleanup) {
    pmatests::Counters counters{};
    ASSERT_EQ(counters.constructed, 0);
    ASSERT_EQ(counters.destructed, 0);
    {
        auto sp = pma::makeScoped<pmatests::Client, pma::FactoryCreate, pma::FactoryDestroy>(counters);
        ASSERT_EQ(counters.constructed, 1);
        ASSERT_EQ(counters.destructed, 0);
    }

    ASSERT_EQ(counters.constructed, 1);
    ASSERT_EQ(counters.destructed, 1);
}

TEST(ScopedPtrTest, MoveAssign) {
    pmatests::Counters counters{};
    ASSERT_EQ(counters.constructed, 0);
    ASSERT_EQ(counters.destructed, 0);
    {
        pma::ScopedPtr<pmatests::Client, pma::FactoryDestroy<pmatests::Client>> sp;
        sp = pma::makeScoped<pmatests::Client, pma::FactoryCreate, pma::FactoryDestroy>(counters);
        ASSERT_EQ(counters.constructed, 1);
        ASSERT_EQ(counters.destructed, 0);
    }
    ASSERT_EQ(counters.constructed, 1);
    ASSERT_EQ(counters.destructed, 1);
}

TEST(ScopedPtrTest, MoveConstruct) {
    pmatests::Counters counters{};
    ASSERT_EQ(counters.constructed, 0);
    ASSERT_EQ(counters.destructed, 0);
    {
        auto sp = pma::makeScoped<pmatests::Client, pma::FactoryCreate, pma::FactoryDestroy>(counters);
        pma::ScopedPtr<pmatests::Client, pma::FactoryDestroy<pmatests::Client>> dest = std::move(sp);
        ASSERT_EQ(counters.constructed, 1);
        ASSERT_EQ(counters.destructed, 0);
    }
    ASSERT_EQ(counters.constructed, 1);
    ASSERT_EQ(counters.destructed, 1);
}

TEST(ScopedPtrTest, ReAssign) {
    pmatests::Counters counters{};
    ASSERT_EQ(counters.constructed, 0);
    ASSERT_EQ(counters.destructed, 0);
    {
        auto sp = pma::makeScoped<pmatests::Client, pma::FactoryCreate, pma::FactoryDestroy>(counters);
        ASSERT_EQ(counters.constructed, 1);
        ASSERT_EQ(counters.destructed, 0);
        sp = nullptr;
        ASSERT_EQ(counters.constructed, 1);
        ASSERT_EQ(counters.destructed, 1);
    }
    ASSERT_EQ(counters.constructed, 1);
    ASSERT_EQ(counters.destructed, 1);
}

TEST(ScopedPtrTest, Reset) {
    pmatests::Counters counters{};
    ASSERT_EQ(counters.constructed, 0);
    ASSERT_EQ(counters.destructed, 0);
    {
        auto sp = pma::makeScoped<pmatests::Client, pma::FactoryCreate, pma::FactoryDestroy>(counters);
        ASSERT_EQ(counters.constructed, 1);
        ASSERT_EQ(counters.destructed, 0);
        sp.reset(new pmatests::Client(counters));
        ASSERT_EQ(counters.constructed, 2);
        ASSERT_EQ(counters.destructed, 1);
    }
    ASSERT_EQ(counters.constructed, 2);
    ASSERT_EQ(counters.destructed, 2);
}

TEST(ScopedPtrTest, UseNewDelete) {
    auto sp = pma::makeScoped<int, pma::New, pma::Delete>(42);
    ASSERT_EQ(*sp, 42);
}

TEST(ScopedPtrTest, UseNewDeleteForArrays) {
    auto sp = pma::makeScoped<int[], pma::New, pma::Delete>(100ul);
    ASSERT_EQ(sp[0], 0);
}

TEST(ScopedPtrTest, CustomDestroyer) {
    std::size_t timesCalled = 0ul;
    {
        pma::ScopedPtr<int, std::function<void(int*)>> sp{new int{}, [&timesCalled](int* ptr) {
                                                              delete ptr;
                                                              ++timesCalled;
                                                          }};
        ASSERT_EQ(timesCalled, 0ul);
    }
    ASSERT_EQ(timesCalled, 1ul);
}

TEST(ScopedPtrTest, MoveAssignWithCustomDestroyer) {
    std::size_t timesCalled = 0ul;
    {
        using PtrType = pma::ScopedPtr<pmatests::Base, std::function<void(pmatests::Base*)>>;

        PtrType sp;
        sp = PtrType{new pmatests::Derived{}, [&timesCalled](pmatests::Base* ptr) {
                         delete ptr;
                         ++timesCalled;
                     }};
        ASSERT_EQ(timesCalled, 0ul);
    }
    ASSERT_EQ(timesCalled, 1ul);
}

TEST(ScopedPtrTest, MoveConstructWithCustomDestroyer) {
    std::size_t timesCalled = 0ul;
    {
        pma::ScopedPtr<pmatests::Base, std::function<void(pmatests::Base*)>> sp{new pmatests::Derived{},
                                                                                [&timesCalled](pmatests::Base* ptr) {
                                                                                    delete ptr;
                                                                                    ++timesCalled;
                                                                                }};
        ASSERT_EQ(timesCalled, 0ul);
    }
    ASSERT_EQ(timesCalled, 1ul);
}

TEST(ScopedPtrTest, UseDefaultCreateDestroy) {
    auto spPrimitive = pma::makeScoped<int>(42);
    ASSERT_EQ(*spPrimitive, 42);

    auto spArray = pma::makeScoped<int[]>(10ul);
    ASSERT_EQ(spArray[0], 0);
}

TEST(ScopedPtrTest, StoreDerivedInBasePointer) {
    using NewCreator = pma::New<pmatests::Derived, pmatests::Base>;
    using NewDestroyer = pma::Delete<pmatests::Derived, pmatests::Base>;
    pma::ScopedPtr<pmatests::Base, NewDestroyer> vbp = pma::makeScoped<pmatests::Derived, NewCreator, NewDestroyer>();

    using FactoryCreator = pma::FactoryCreate<pmatests::Derived, pmatests::Base>;
    using FactoryDestroyer = pma::FactoryDestroy<pmatests::Derived, pmatests::Base>;
    pma::ScopedPtr<pmatests::Base, FactoryDestroyer> fbp = pma::makeScoped<pmatests::Derived, FactoryCreator, FactoryDestroyer>();
}
