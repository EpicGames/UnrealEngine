// Copyright Epic Games, Inc. All Rights Reserved.

#include "pmatests/Defs.h"

#include "pma/utils/ManagedInstance.h"

namespace pmatests {

namespace {

struct Stats {
    int constructed;
    int destructed;
};

class BaseType {
public:
    virtual ~BaseType() = default;
};

class ManageableType : public BaseType {
public:
    ManageableType(Stats& stats_) :
        stats{&stats_} {
        stats->constructed += 1;
    }

    ~ManageableType() {
        stats->destructed += 1;
    }

    ManageableType(const ManageableType&) = default;
    ManageableType& operator=(const ManageableType&) = default;

    ManageableType(ManageableType&&) = default;
    ManageableType& operator=(ManageableType&&) = default;

private:
    Stats* stats;
};

}  // namespace

}  // namespace pmatests

TEST(ManagedInstanceTest, UniqueInstance) {
    pma::DefaultMemoryResource memRes;
    pmatests::Stats stats{};
    ASSERT_EQ(stats.constructed, 0);
    ASSERT_EQ(stats.destructed, 0);
    {
        auto ptr = pma::UniqueInstance<pmatests::ManageableType, pmatests::BaseType>::with(&memRes).create(stats);
        ASSERT_EQ(stats.constructed, 1);
        ASSERT_EQ(stats.destructed, 0);
    }
    ASSERT_EQ(stats.constructed, 1);
    ASSERT_EQ(stats.destructed, 1);
}

TEST(ManagedInstanceTest, SharedInstance) {
    pma::DefaultMemoryResource memRes;
    pmatests::Stats stats{};
    ASSERT_EQ(stats.constructed, 0);
    ASSERT_EQ(stats.destructed, 0);
    {
        std::shared_ptr<pmatests::BaseType> ptr;
        {
            auto ptr2 = pma::SharedInstance<pmatests::ManageableType, pmatests::BaseType>::with(&memRes).create(stats);
            ptr = ptr2;
            ASSERT_EQ(stats.constructed, 1);
            ASSERT_EQ(stats.destructed, 0);
        }
        ASSERT_EQ(stats.constructed, 1);
        ASSERT_EQ(stats.destructed, 0);
    }
    ASSERT_EQ(stats.constructed, 1);
    ASSERT_EQ(stats.destructed, 1);
}
