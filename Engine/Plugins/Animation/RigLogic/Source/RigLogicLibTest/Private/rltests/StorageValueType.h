// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <cstdint>

#ifdef RL_BUILD_WITH_HALF_FLOATS
using StorageValueType = std::uint16_t;
#else
using StorageValueType = float;
#endif
