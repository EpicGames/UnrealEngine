// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/animatedmaps/AnimatedMapFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsImpl.h"
#include "riglogic/animatedmaps/AnimatedMapsImplOutputInstance.h"
#include "riglogic/conditionaltable/ConditionalTable.h"

namespace defaults {

const std::size_t lodCount = 1ul;
const std::uint16_t lods[lodCount] = {2};

}  // namespace defaults

rl4::AnimatedMapsOutputInstance::Factory AnimatedMapsFactory::getInstanceFactory(std::uint16_t animatedMapCount) {
    return [animatedMapCount](rl4::MemoryResource* memRes) {
        return pma::UniqueInstance<rl4::AnimatedMapsImplOutputInstance, rl4::AnimatedMapsOutputInstance>::with(memRes).create(
            animatedMapCount,
            memRes);
    };
}

rl4::AnimatedMapsImpl AnimatedMapsFactory::withDefaults(rl4::ConditionalTable&& conditionals, rl4::MemoryResource* memRes) {
    const auto animatedMapCount = conditionals.getOutputCount();
    return rl4::AnimatedMapsImpl{rl4::Vector<std::uint16_t>{defaults::lods, defaults::lods + defaults::lodCount, memRes},
                                 std::move(conditionals),
                                 getInstanceFactory(animatedMapCount)};
}
