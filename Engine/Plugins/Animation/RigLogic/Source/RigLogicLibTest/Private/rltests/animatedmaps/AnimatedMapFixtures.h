// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMapsImpl.h"
#include "riglogic/conditionaltable/ConditionalTable.h"

#include <cstddef>

struct AnimatedMapsFactory {

    static rl4::AnimatedMapsImpl withDefaults(rl4::ConditionalTable&& conditionals, rl4::MemoryResource* memRes);
    static rl4::AnimatedMapsOutputInstance::Factory getInstanceFactory(std::uint16_t animatedMapCount);
};
