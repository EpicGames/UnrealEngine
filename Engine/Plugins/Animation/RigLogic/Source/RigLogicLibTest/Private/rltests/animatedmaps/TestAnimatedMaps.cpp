// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/animatedmaps/AnimatedMapFixtures.h"
#include "rltests/conditionaltable/ConditionalTableFixtures.h"
#include "rltests/controls/ControlFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/animatedmaps/AnimatedMaps.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace {

class AnimatedMapsTest : public ::testing::TestWithParam<std::uint16_t> {};

}  // namespace

TEST_P(AnimatedMapsTest, LODLimitsCondTableSize) {
    pma::AlignedMemoryResource amr;
    rl4::Vector<std::uint16_t> lods{2, 1};
    auto conditionals = ConditionalTableFactory::withMultipleIODefaults(&amr);
    auto inputInstanceFactory =
        ControlsFactory::getInstanceFactory(0, static_cast<std::uint16_t>(conditionalTableInputs.size()), 0, 0, 0);
    auto outputInstanceFactory = AnimatedMapsFactory::getInstanceFactory(conditionals.getOutputCount());
    rl4::AnimatedMapsImpl animatedMaps{std::move(lods), std::move(conditionals), outputInstanceFactory};

    const auto lod = GetParam();
    const float expected[2][2] = {
        {0.3f, 0.6f},  // LOD0
        {0.3f, 0.0f}   // LOD1
    };
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &amr);
    auto inputBuffer = inputInstance->getInputBuffer();
    std::copy(conditionalTableInputs.begin(), conditionalTableInputs.end(), inputBuffer.begin());

    auto outputInstance = animatedMaps.createInstance(&amr);
    animatedMaps.calculate(inputInstance.get(), outputInstance.get(), lod);
    ASSERT_ELEMENTS_EQ(outputInstance->getOutputBuffer(), expected[lod], 2ul);
}

INSTANTIATE_TEST_SUITE_P(AnimatedMapsTest, AnimatedMapsTest, ::testing::Values(0u, 1u));
