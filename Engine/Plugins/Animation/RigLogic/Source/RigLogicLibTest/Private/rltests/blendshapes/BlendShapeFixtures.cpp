// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/blendshapes/BlendShapeFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapesImpl.h"
#include "riglogic/blendshapes/BlendShapesImplOutputInstance.h"

namespace rltests {

rl4::BlendShapesOutputInstance::Factory BlendShapesFactory::getInstanceFactory(std::uint16_t bsCount) {
    return [bsCount](rl4::MemoryResource* memRes) {
        return pma::UniqueInstance<rl4::BlendShapesImplOutputInstance, rl4::BlendShapesOutputInstance>::with(memRes).create(
            bsCount,
            memRes);
    };
}

rl4::BlendShapesImpl BlendShapesFactory::createTestBlendShapes(rl4::MemoryResource* memRes) {
    return rl4::BlendShapesImpl{
        rl4::Vector<std::uint16_t>{std::begin(LODs), std::end(LODs), memRes},
        rl4::Vector<std::uint16_t>{std::begin(blendShapeInputIndices), std::end(blendShapeInputIndices), memRes},
        rl4::Vector<std::uint16_t>{std::begin(blendShapeOutputIndices), std::end(blendShapeOutputIndices), memRes},
        getInstanceFactory(blendShapeCount)};
}

}  // namespace rltests
