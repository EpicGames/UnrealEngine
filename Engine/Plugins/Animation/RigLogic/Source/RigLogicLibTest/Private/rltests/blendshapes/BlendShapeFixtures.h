// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapesImpl.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cstddef>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rltests {

static const std::size_t LODCount = 4ul;
static const std::uint16_t LOD0 = 10u;
static const std::uint16_t LOD1 = 7u;
static const std::uint16_t LOD2 = 4u;
static const std::uint16_t LOD3 = 1u;

static const std::uint16_t LODs[] = {LOD0, LOD1, LOD2, LOD3};

static const std::array<float, 15ul> blendShapeInputs =
    {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f, 11.0f, 12.0f, 13.0f, 14.0f, 15.0f};
static const std::uint16_t blendShapeCount = 10u;
static const std::uint16_t blendShapeInputIndices[blendShapeCount] = {0, 1, 3, 4, 6, 7, 8, 10, 12, 14};
static const std::uint16_t blendShapeOutputIndices[blendShapeCount] = {0, 2, 1, 3, 4, 5, 6, 7, 8, 9};
static const std::array<float, 10ul> blendShapeOutputs = {1.0f, 4.0f, 2.0f, 5.0f, 7.0f, 8.0f, 9.0f, 11.0f, 13.0f, 15.0f};

struct BlendShapesFactory {
    static rl4::BlendShapesImpl createTestBlendShapes(rl4::MemoryResource* memRes);
    static rl4::BlendShapesOutputInstance::Factory getInstanceFactory(std::uint16_t animatedMapCount);
};

}  // namespace rltests
