// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/blendshapes/BlendShapeFixtures.h"
#include "rltests/controls/ControlFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/blendshapes/BlendShapes.h"
#include "riglogic/utils/Extd.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace {

class BlendShapesTest : public ::testing::TestWithParam<std::uint16_t> {};

}  // namespace

TEST_P(BlendShapesTest, Calculate) {
    using namespace rltests;

    // Test input parameter - Which LOD level to test
    const auto lod = GetParam();

    pma::AlignedMemoryResource amr;

    auto inputInstanceFactory =
        ControlsFactory::getInstanceFactory(0, static_cast<std::uint16_t>(blendShapeInputs.size()), 0, 0, 0);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &amr);
    auto inputBuffer = inputInstance->getInputBuffer();
    std::copy(blendShapeInputs.begin(), blendShapeInputs.end(), inputBuffer.begin());

    auto blendShapes = BlendShapesFactory::createTestBlendShapes(&amr);
    auto outputInstance = blendShapes.createInstance(&amr);

    blendShapes.calculate(inputInstance.get(), outputInstance.get(), lod);

    std::vector<float> expectedData{blendShapeOutputs.begin(), blendShapeOutputs.end()};
    std::fill(extd::advanced(expectedData.begin(), LODs[lod]), expectedData.end(), 0.0f);
    rl4::ConstArrayView<float> expected{expectedData};

    auto outputBuffer = outputInstance->getOutputBuffer();
    for (std::size_t i = 0ul; i < blendShapeOutputs.size(); ++i) {
        ASSERT_EQ(outputBuffer[i], expected[i]);
    }
}

INSTANTIATE_TEST_SUITE_P(BlendShapesTest, BlendShapesTest, ::testing::Values(0u, 1u, 2u, 3u));
