// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/conditionaltable/ConditionalTableFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/conditionaltable/ConditionalTable.h"

namespace single {

const std::uint16_t inputIndices[] = {0, 0};
const std::uint16_t outputIndices[] = {0, 0};
const float fromValues[] = {0.0f, 0.0f};
const float toValues[] = {1.0f, 1.0f};
const float slopeValues[] = {1.0f, 1.0f};
const float cutValues[] = {0.1f, 0.3f};
const std::uint16_t size = 2ul;
const std::uint16_t distinctInputs = 1ul;
const std::uint16_t distinctOutputs = 1ul;

}  // namespace single

namespace multiple {

const std::uint16_t inputIndices[] = {0, 1};
const std::uint16_t outputIndices[] = {0, 1};
const float fromValues[] = {0.0f, 0.0f};
const float toValues[] = {1.0f, 1.0f};
const float slopeValues[] = {1.0f, 1.0f};
const float cutValues[] = {0.2f, 0.4f};
const std::uint16_t size = 2ul;
const std::uint16_t distinctInputs = 2ul;
const std::uint16_t distinctOutputs = 2ul;

}  // namespace multiple

namespace interleaved {

/*
// Conditional table                                      Recommended input test ranges
//
I: 0 O: 0  F: 0    T: 1    S: 1        C: 0              -1.0  0.0  0.5  1.0  1.5
I: 1 O: 1  F: 0    T: 1    S: 1        C: 0              -1.0  0.0  0.5  1.0  1.5
I: 2 O: 4  F: -1   T: 0    S: -1       C: 0              -1.5 -1.0 -0.5  0.0  0.5  1.0  1.5
I: 3 O: 5  F: -1   T: 0    S: -1       C: 0              -1.5 -1.0 -0.5  0.0  0.5  1.0  1.5
I: 2 O: 2  F: 0    T: 1    S: 1        C: 0
I: 3 O: 3  F: 0    T: 1    S: 1        C: 0
I: 4 O: 10 F: 0    T: 1    S: 1        C: 0              -1.5 -1.0 -0.5  0.0  0.5  1.0  1.5
I: 5 O: 11 F: 0    T: 1    S: 1        C: 0              -1.5 -1.0 -0.5  0.0  0.5  1.0  1.5
I: 6 O: 12 F: 0    T: 1    S: 1        C: 0              -1.5 -1.0 -0.5  0.0  0.5  1.0  1.5
I: 7 O: 13 F: 0    T: 1    S: 1        C: 0              -1.5 -1.0 -0.5  0.0  0.5  1.0  1.5
I: 4 O: 6  F: -1   T: 0    S: -1       C: 0
I: 5 O: 7  F: -1   T: 0    S: -1       C: 0
I: 6 O: 8  F: -1   T: 0    S: -1       C: 0
I: 7 O: 9  F: -1   T: 0    S: -1       C: 0
I: 8 O: 14 F: 0    T: 0.33 S: 3.0303   C: 0              -0.5  0.0  0.15 0.33 0.5  0.66 0.75 1.0 1.5
I: 8 O: 14 F: 0.33 T: 0.66 S: -3.0303  C: 2
I: 8 O: 15 F: 0.33 T: 0.66 S: 3.0303   C: -1
I: 8 O: 15 F: 0.66 T: 1    S: -2.94118 C: 2.94118
I: 8 O: 16 F: 0.66 T: 1    S: 2.94118  C: -1.94118
*/
const std::uint16_t inputIndices[] = {0, 1, 2, 3, 2, 3, 4, 5, 6, 7, 4, 5, 6, 7, 8, 8, 8, 8, 8};
const std::uint16_t outputIndices[] = {0, 1, 4, 5, 2, 3, 10, 11, 12, 13, 6, 7, 8, 9, 14, 14, 15, 15, 16};
const float fromValues[] =
    {0.0f, 0.0f, -1.0f, -1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, -1.0f, -1.0f, -1.0f, -1.0f, 0.0f, 0.33f, 0.33f, 0.66f, 0.66f};
const float toValues[] =
    {1.0f, 1.0f, 0.0f, 0.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.33f, 0.66f, 0.66f, 1.0f, 1.0f};
const float slopeValues[] = {1.0f,
                             1.0f,
                             -1.0f,
                             -1.0f,
                             1.0f,
                             1.0f,
                             1.0f,
                             1.0f,
                             1.0f,
                             1.0f,
                             -1.0f,
                             -1.0f,
                             -1.0f,
                             -1.0f,
                             3.0303f,
                             -3.0303f,
                             3.0303f,
                             -2.94118f,
                             2.94118f};
const float cutValues[] =
    {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 2.0f, -1.0f, 2.94118f, -1.94118f};
const std::uint16_t size = 19ul;
const std::uint16_t distinctInputs = 9ul;
const std::uint16_t distinctOutputs = 17ul;

}  // namespace interleaved

rl4::ConditionalTable ConditionalTableFactory::withSingleIO(rl4::Vector<float>&& fromValues,
                                                            rl4::Vector<float>&& toValues,
                                                            rl4::Vector<float>&& cutValues,
                                                            rl4::MemoryResource* memRes) {
    return rl4::ConditionalTable{rl4::Vector<std::uint16_t>{single::inputIndices, single::inputIndices + single::size, memRes},
                                 rl4::Vector<std::uint16_t>{single::outputIndices, single::outputIndices + single::size, memRes},
                                 std::move(fromValues),
                                 std::move(toValues),
                                 rl4::Vector<float>{single::slopeValues, single::slopeValues + single::size, memRes},
                                 std::move(cutValues),
                                 single::distinctInputs,
                                 single::distinctOutputs,
                                 memRes};
}

rl4::ConditionalTable ConditionalTableFactory::withMultipleIO(rl4::Vector<float>&& fromValues,
                                                              rl4::Vector<float>&& toValues,
                                                              rl4::MemoryResource* memRes) {
    return rl4::ConditionalTable{
        rl4::Vector<std::uint16_t>{multiple::inputIndices, multiple::inputIndices + multiple::size, memRes},
        rl4::Vector<std::uint16_t>{multiple::outputIndices, multiple::outputIndices + multiple::size, memRes},
        std::move(fromValues),
        std::move(toValues),
        rl4::Vector<float>{multiple::slopeValues, multiple::slopeValues + multiple::size, memRes},
        rl4::Vector<float>{multiple::cutValues, multiple::cutValues + multiple::size, memRes},
        multiple::distinctInputs,
        multiple::distinctOutputs,
        memRes};
}

rl4::ConditionalTable ConditionalTableFactory::withSingleIODefaults(rl4::MemoryResource* memRes) {
    return withSingleIO(rl4::Vector<float>{single::fromValues, single::fromValues + single::size, memRes},
                        rl4::Vector<float>{single::toValues, single::toValues + single::size, memRes},
                        rl4::Vector<float>{single::cutValues, single::cutValues + single::size, memRes},
                        memRes);
}

rl4::ConditionalTable ConditionalTableFactory::withMultipleIODefaults(rl4::MemoryResource* memRes) {
    return withMultipleIO(rl4::Vector<float>{multiple::fromValues, multiple::fromValues + multiple::size, memRes},
                          rl4::Vector<float>{multiple::toValues, multiple::toValues + multiple::size, memRes},
                          memRes);
}

rl4::ConditionalTable ConditionalTableFactory::withInterleavedIO(rl4::MemoryResource* memRes) {
    return rl4::ConditionalTable{
        rl4::Vector<std::uint16_t>{interleaved::inputIndices, interleaved::inputIndices + interleaved::size, memRes},
        rl4::Vector<std::uint16_t>{interleaved::outputIndices, interleaved::outputIndices + interleaved::size, memRes},
        rl4::Vector<float>{interleaved::fromValues, interleaved::fromValues + interleaved::size, memRes},
        rl4::Vector<float>{interleaved::toValues, interleaved::toValues + interleaved::size, memRes},
        rl4::Vector<float>{interleaved::slopeValues, interleaved::slopeValues + interleaved::size, memRes},
        rl4::Vector<float>{interleaved::cutValues, interleaved::cutValues + interleaved::size, memRes},
        interleaved::distinctInputs,
        interleaved::distinctOutputs,
        memRes};
}
