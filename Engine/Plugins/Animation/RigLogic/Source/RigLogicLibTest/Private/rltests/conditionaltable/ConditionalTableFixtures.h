// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/conditionaltable/ConditionalTable.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#include <cstddef>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

static const std::array<float, 9ul> conditionalTableInputs = {0.1f, 0.2f, 0.3f, 0.4f, 0.5f, 0.6f, 0.7f, 0.8f, 0.9f};

struct ConditionalTableFactory {

    static rl4::ConditionalTable withSingleIO(rl4::Vector<float>&& fromValues,
                                              rl4::Vector<float>&& toValues,
                                              rl4::Vector<float>&& cutValues,
                                              rl4::MemoryResource* memRes);
    static rl4::ConditionalTable withMultipleIO(rl4::Vector<float>&& fromValues,
                                                rl4::Vector<float>&& toValues,
                                                rl4::MemoryResource* memRes);
    static rl4::ConditionalTable withSingleIODefaults(rl4::MemoryResource* memRes);
    static rl4::ConditionalTable withMultipleIODefaults(rl4::MemoryResource* memRes);
    static rl4::ConditionalTable withInterleavedIO(rl4::MemoryResource* memRes);
};
