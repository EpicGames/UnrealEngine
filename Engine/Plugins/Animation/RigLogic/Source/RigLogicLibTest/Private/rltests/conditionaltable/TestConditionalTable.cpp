// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/conditionaltable/ConditionalTableFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/conditionaltable/ConditionalTable.h"

namespace {

template<std::size_t EntryCount, std::size_t InputCount, std::size_t OutputCount>
struct TCalcTestData {
    float fromValues[EntryCount];
    float toValues[EntryCount];
    float cutValues[EntryCount];
    float inValues[InputCount];
    float expected[OutputCount];
};
using CalcTestData = TCalcTestData<2, 1, 1>;

class ConditionalTableTest : public ::testing::TestWithParam<CalcTestData> {};

}  // namespace

TEST_P(ConditionalTableTest, CheckCalculationBorderCases) {
    pma::AlignedMemoryResource amr;
    const std::size_t conditionalsSize = 2ul;
    auto testData = GetParam();
    auto conditionals = ConditionalTableFactory::withSingleIO(
        rl4::Vector<float>{testData.fromValues, testData.fromValues + conditionalsSize, &amr},
        rl4::Vector<float>{testData.toValues, testData.toValues + conditionalsSize, &amr},
        rl4::Vector<float>{testData.cutValues, testData.cutValues + conditionalsSize, &amr},
        &amr);
    float outputs[] = {0.0f};
    conditionals.calculateForward(testData.inValues, outputs);
    ASSERT_ELEMENTS_NEAR(outputs, testData.expected, 1, 0.00001f);
}

INSTANTIATE_TEST_SUITE_P(ConditionalTableTest,
                         ConditionalTableTest,
                         ::testing::Values(
                             // {{fromValues}, {toValues}, {expected}}
                             // In-value below from-value
                             CalcTestData{{0.3f, 0.6f}, {0.6f, 1.0f}, {0.1f, 0.3f}, {0.1f}, {0.0f}},
                             // In-value equals from-value
                             CalcTestData{{0.1f, 0.6f}, {0.6f, 1.0f}, {0.1f, 0.3f}, {0.1f}, {0.2f}},
                             // In-value equals to-value
                             CalcTestData{{0.0f, 0.2f}, {0.1f, 1.0f}, {0.1f, 0.3f}, {0.1f}, {0.2f}},
                             // In-value equals both from-value and to-value
                             CalcTestData{{0.0f, 0.1f}, {0.1f, 1.0f}, {0.1f, 0.3f}, {0.1f}, {0.2f}},
                             // In-value between from-value and to-value
                             CalcTestData{{0.0f, 0.6f}, {0.6f, 1.0f}, {0.1f, 0.3f}, {0.1f}, {0.2f}},
                             // In-value above to-value
                             CalcTestData{{0.0f, 0.04f}, {0.04f, 0.09f}, {0.1f, 0.3f}, {0.1f}, {0.0f}},
                             // In-value equals lower-bound from-value
                             CalcTestData{{-1.0f, 0.0f}, {0.0f, 1.0f}, {1.4f, 0.3f}, {-1.0f}, {0.4f}}));

TEST(ConditionalTableTest, OutputClamped) {
    pma::AlignedMemoryResource amr;
    const std::uint16_t inputCount = 1u;
    const std::uint16_t outputCount = 1u;
    rl4::Vector<std::uint16_t> inputIndices{0};
    rl4::Vector<std::uint16_t> outputIndices{0};
    rl4::Vector<float> slopeValues{1.0f};
    rl4::Vector<float> cutValues{2.0f};
    rl4::Vector<float> fromValues{0.0f};
    rl4::Vector<float> toValues{1.0f};
    float outputs[1] = {};
    rl4::ConditionalTable conditionals{std::move(inputIndices),
                                       std::move(outputIndices),
                                       std::move(fromValues),
                                       std::move(toValues),
                                       std::move(slopeValues),
                                       std::move(cutValues),
                                       inputCount,
                                       outputCount,
                                       &amr};
    conditionals.calculateForward(conditionalTableInputs.data(), outputs);
    const float expected[] = {1.0f};
    ASSERT_ELEMENTS_EQ(outputs, expected, 1ul);
}

TEST(ConditionalTableTest, OutputIsAccumulated) {
    pma::AlignedMemoryResource amr;
    const std::uint16_t inputCount = 2u;
    const std::uint16_t outputCount = 1u;
    rl4::Vector<std::uint16_t> inputIndices{0, 1};
    rl4::Vector<std::uint16_t> outputIndices{0, 0};
    rl4::Vector<float> slopeValues{1.0f, 1.0f};
    rl4::Vector<float> cutValues{0.2f, 0.4f};
    rl4::Vector<float> fromValues{0.0f, 0.0f};
    rl4::Vector<float> toValues{0.2f, 0.2f};
    float outputs[1] = {};
    rl4::ConditionalTable conditionals{std::move(inputIndices),
                                       std::move(outputIndices),
                                       std::move(fromValues),
                                       std::move(toValues),
                                       std::move(slopeValues),
                                       std::move(cutValues),
                                       inputCount,
                                       outputCount,
                                       &amr};
    conditionals.calculateForward(conditionalTableInputs.data(), outputs);
    const float expected[] = {0.9f};
    ASSERT_ELEMENTS_NEAR(outputs, expected, 1ul, 0.00001f);
}

TEST(ConditionalTableTest, OutputIsResetOnEachCalculation) {
    pma::AlignedMemoryResource amr;
    auto conditionals = ConditionalTableFactory::withSingleIODefaults(&amr);

    float outputs[1ul] = {};
    const float expected[] = {0.2f};

    conditionals.calculateForward(conditionalTableInputs.data(), outputs);
    ASSERT_ELEMENTS_NEAR(outputs, expected, 1ul, 0.00001f);

    conditionals.calculateForward(conditionalTableInputs.data(), outputs);
    ASSERT_ELEMENTS_NEAR(outputs, expected, 1ul, 0.00001f);
}

namespace {

template<std::size_t InputCount, std::size_t OutputCount>
struct TIOTestData {

    static constexpr std::size_t inputCount() {
        return InputCount;
    }

    static constexpr std::size_t outputCount() {
        return OutputCount;
    }

    float inputs[InputCount];
    float outputs[OutputCount];
    float reverseOutputs[InputCount];
};
using IOTestData = TIOTestData<9, 17>;

class IRLConditionalTableTest : public ::testing::TestWithParam<IOTestData> {};

}  // namespace

TEST_P(IRLConditionalTableTest, InterleavedInputsAndOutputs) {
    pma::AlignedMemoryResource amr;
    auto conditionals = ConditionalTableFactory::withInterleavedIO(&amr);

    auto testData = GetParam();
    std::vector<float> inputs{testData.inputs, testData.inputs + testData.inputCount()};
    float outputs[testData.outputCount()] = {};

    conditionals.calculateForward(inputs.data(), outputs);
    ASSERT_ELEMENTS_NEAR(outputs, testData.outputs, testData.outputCount(), 0.00001f);

    conditionals.calculateReverse(inputs.data(), outputs);
    ASSERT_ELEMENTS_NEAR(inputs, testData.reverseOutputs, testData.inputCount(), 0.00001f);
}

INSTANTIATE_TEST_SUITE_P(
    IRLConditionalTableTest,
    IRLConditionalTableTest,
    ::testing::Values(
        IOTestData{{-1.0f, -1.0f, -1.5f, -1.5f, -1.5f, -1.5f, -1.5f, -1.5f, -0.5f},
                   {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},
                   {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}},
        IOTestData{{0.0f, 0.0f, -1.0f, -1.0f, -1.0f, -1.0f, -1.0f, -1.0f, 0.0f},
                   {0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},
                   {0.0f, 0.0f, -1.0f, -1.0f, -1.0f, -1.0f, -1.0f, -1.0f, 0.0f}},
        IOTestData{{0.5f, 0.5f, -0.5f, -0.5f, -0.5f, -0.5f, -0.5f, -0.5f, 0.15f},
                   {0.5f, 0.5f, 0.0f, 0.0f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.454545f, 0.0f, 0.0f},
                   {0.5f, 0.5f, -0.5f, -0.5f, -0.5f, -0.5f, -0.5f, -0.5f, 0.15f}},
        IOTestData{{1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.33f},
                   {1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.999999f, 0.0f, 0.0f},
                   {1.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.33f}},
        IOTestData{{1.5f, 1.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f},
                   {0.0f, 0.0f, 0.5f, 0.5f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.5f, 0.5f, 0.5f, 0.5f, 0.48485f, 0.51515f, 0.0f},
                   {0.0f, 0.0f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f}},
        IOTestData{{0.0f, 0.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 0.66f},
                   {0.0f,
                    0.0f,
                    1.0f,
                    1.0f,
                    0.0f,
                    0.0f,
                    0.0f,
                    0.0f,
                    0.0f,
                    0.0f,
                    1.0f,
                    1.0f,
                    1.0f,
                    1.0f,
                    0.000002f,
                    0.999998f,
                    -0.0000012f},
                   {0.0f, 0.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 0.66f}},
        IOTestData{
            {0.0f, 0.0f, 1.5f, 1.5f, 1.5f, 1.5f, 1.5f, 1.5f, 0.75f},
            {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.735295f, 0.264705f},
            {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.75f}},
        IOTestData{{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f},
                   {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f},
                   {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f}},
        IOTestData{{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.5f},
                   {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},
                   {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}));
