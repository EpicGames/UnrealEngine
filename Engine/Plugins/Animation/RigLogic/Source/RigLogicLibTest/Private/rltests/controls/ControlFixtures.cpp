// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/controls/ControlFixtures.h"

#include "riglogic/controls/instances/StandardControlsInputInstance.h"

#include <cstddef>

rl4::ControlsInputInstance::Factory ControlsFactory::getInstanceFactory(std::uint16_t guiControlCount,
                                                                        std::uint16_t rawControlCount,
                                                                        std::uint16_t psdControlCount,
                                                                        std::uint16_t mlControlCount,
                                                                        std::uint16_t rbfControlCount) {
    return [=](rl4::ConstArrayView<rl4::ControlInitializer> initialValues, rl4::MemoryResource* memRes) {
        return pma::UniqueInstance<rl4::StandardControlsInputInstance, rl4::ControlsInputInstance>::with(memRes)
            .create(guiControlCount, rawControlCount, psdControlCount, mlControlCount, rbfControlCount, initialValues, memRes);
    };
}
