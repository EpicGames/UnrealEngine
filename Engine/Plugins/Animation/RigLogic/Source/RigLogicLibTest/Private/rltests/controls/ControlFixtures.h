// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"

#include <cstddef>

struct ControlsFactory {
    static rl4::ControlsInputInstance::Factory getInstanceFactory(std::uint16_t guiControlCount,
                                                                  std::uint16_t rawControlCount,
                                                                  std::uint16_t psdControlCount,
                                                                  std::uint16_t mlControlCount,
                                                                  std::uint16_t rbfControlCount);
};
