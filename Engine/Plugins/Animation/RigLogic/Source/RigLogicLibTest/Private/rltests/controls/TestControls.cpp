// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/conditionaltable/ConditionalTableFixtures.h"
#include "rltests/controls/ControlFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/controls/Controls.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <array>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

TEST(ControlsTest, GUIToRawMapping) {
    pma::AlignedMemoryResource amr;
    auto conditionals = ConditionalTableFactory::withMultipleIODefaults(&amr);

    const std::uint16_t guiControlCount = conditionals.getInputCount();
    const std::uint16_t rawControlCount = conditionals.getOutputCount();
    auto instanceFactory = ControlsFactory::getInstanceFactory(guiControlCount, rawControlCount, 0u, 0u, 0u);

    rl4::Vector<rl4::ControlInitializer> initialValues;
    rl4::Controls controls{std::move(conditionals), std::move(initialValues), instanceFactory, 1ul, &amr};

    const rl4::Vector<float> guiControls{0.1f, 0.2f};
    const rl4::Vector<float> expected{0.3f, 0.6f};

    auto instance = controls.createInstance(&amr);
    auto guiBuffer = instance->getGUIControlBuffer();
    auto rawBuffer = instance->getInputBuffer();
    std::copy(guiControls.begin(), guiControls.end(), guiBuffer.begin());
    controls.mapGUIToRaw(instance.get());

    ASSERT_EQ(rawBuffer.size(), expected.size());
    ASSERT_ELEMENTS_EQ(rawBuffer, expected, expected.size());
}
