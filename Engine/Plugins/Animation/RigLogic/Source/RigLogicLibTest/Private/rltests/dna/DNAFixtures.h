// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/TypeDefs.h"

namespace rltests {

using namespace dna;
using namespace pma;

namespace raw {

extern const unsigned char header[91];
extern const unsigned char descriptor[87];
extern const unsigned char definition[794];
extern const unsigned char controls[2];
extern const unsigned char conditionals[324];
extern const unsigned char psds[204];
extern const unsigned char joints[444];
extern const unsigned char blendshapes[44];
extern const unsigned char animatedmaps[8];
extern const unsigned char geometry[1080];
extern const unsigned char machineLearnedBehavior[762];

std::vector<char> getBytes();

}  // namespace raw

namespace decoded {

using VectorOfCharStringMatrix = Vector<Matrix<String<char>>>;
using StringPair = std::pair<String<char>, String<char>>;

// Descriptor
extern const String<char> name;
extern const Archetype archetype;
extern const Gender gender;
extern const std::uint16_t age;

extern const Vector<StringPair> metadata;
extern const TranslationUnit translationUnit;
extern const RotationUnit rotationUnit;
extern const CoordinateSystem coordinateSystem;
extern const std::uint16_t lodCount[3ul];
extern const std::uint16_t maxLODs[3ul];
extern const String<char> complexity;
extern const String<char> dbName;

// Definition
extern const Vector<String<char>> guiControlNames;
extern const Vector<String<char>> rawControlNames;
extern const VectorOfCharStringMatrix jointNames;
extern const VectorOfCharStringMatrix blendShapeNames;
extern const VectorOfCharStringMatrix animatedMapNames;
extern const VectorOfCharStringMatrix meshNames;
extern const Vector<Matrix<std::uint16_t>> meshBlendShapeIndices;
extern const Matrix<std::uint16_t> jointHierarchy;
extern const Vector<Matrix<Vector3>> neutralJointTranslations;
extern const Vector<Matrix<Vector3>> neutralJointRotations;

// Behavior
extern const std::uint16_t guiControlCount;
extern const std::uint16_t rawControlCount;
extern const std::uint16_t psdControlCount;
extern const Vector<Matrix<std::uint16_t>> conditionalInputIndices;
extern const Vector<Matrix<std::uint16_t>> conditionalOutputIndices;
extern const Vector<Matrix<float>> conditionalFromValues;
extern const Vector<Matrix<float>> conditionalToValues;
extern const Vector<Matrix<float>> conditionalSlopeValues;
extern const Vector<Matrix<float>> conditionalCutValues;
extern const Vector<std::uint16_t> psdRowIndices;
extern const Vector<std::uint16_t> psdColumnIndices;
extern const Vector<float> psdValues;
extern const Vector<std::uint16_t> jointRowCount;
extern const std::uint16_t jointColumnCount;
extern const Vector<Matrix<std::uint16_t>> jointVariableIndices;
extern const Vector<Matrix<std::uint16_t>> jointGroupLODs;
extern const Matrix<Matrix<std::uint16_t>> jointGroupInputIndices;
extern const Matrix<Matrix<std::uint16_t>> jointGroupOutputIndices;
extern const Matrix<Matrix<float>> jointGroupValues;
extern const Matrix<Matrix<std::uint16_t>> jointGroupJointIndices;
extern const Matrix<std::uint16_t> blendShapeLODs;
extern const Vector<Matrix<std::uint16_t>> blendShapeInputIndices;
extern const Vector<Matrix<std::uint16_t>> blendShapeOutputIndices;
extern const Vector<std::uint16_t> animatedMapCount;
extern const Matrix<std::uint16_t> animatedMapLODs;

// Geometry
extern const Vector<std::uint32_t> meshCount;
extern const Vector<Matrix<Vector3>> vertexPositions;
extern const Vector<Matrix<TextureCoordinate>> vertexTextureCoordinates;
extern const Vector<Matrix<Vector3>> vertexNormals;
extern const Vector<Matrix<VertexLayout>> vertexLayouts;
extern const Matrix<Matrix<std::uint32_t>> faces;
extern const Matrix<std::uint16_t> maxInfluencePerVertex;
extern const Matrix<Matrix<float>> skinWeightsValues;
extern const Matrix<Matrix<std::uint16_t>> skinWeightsJointIndices;
extern const Vector<Matrix<std::uint16_t>> correctiveBlendShapeIndices;
extern const Matrix<Matrix<Vector3>> correctiveBlendShapeDeltas;
extern const Matrix<Matrix<std::uint32_t>> correctiveBlendShapeVertexIndices;

// Machine learned behavior
extern const Vector<String<char>> mlControlNames;
extern const VectorOfCharStringMatrix regionNames;
extern const Matrix<std::uint16_t> neuralNetworkIndicesPerLOD;
extern const Matrix<Matrix<std::uint16_t>> neuralNetworkIndicesPerMeshRegion;
extern const Vector<Matrix<std::uint16_t>> neuralNetworkInputIndices;
extern const Vector<Matrix<std::uint16_t>> neuralNetworkOutputIndices;
extern const Matrix<std::uint16_t> neuralNetworkLayerCount;
extern const Vector<Matrix<std::uint16_t>> neuralNetworkActivationFunction;
extern const Matrix<Matrix<float>> neuralNetworkActivationFunctionParameters;
extern const Matrix<Matrix<float>> neuralNetworkBiases;
extern const Matrix<Matrix<float>> neuralNetworkWeights;

}  // namespace decoded

}  // namespace rltests
