// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/dna/FakeReader.h"

namespace dna {

FakeReader::~FakeReader() = default;

}  // namespace dna
