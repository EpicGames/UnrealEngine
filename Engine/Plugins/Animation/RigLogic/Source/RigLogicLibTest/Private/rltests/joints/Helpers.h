// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/cpu/quaternions/RotationAdapters.h"
#include "riglogic/riglogic/Configuration.h"

#include <cstddef>

template<class TRotationAdapter>
struct RotationOutputTypeSelector;

template<>
struct RotationOutputTypeSelector<rl4::PassthroughAdapter> {
    static constexpr std::size_t value() {
        return 0ul;
    }

    static constexpr rl4::RotationType rotation() {
        return rl4::RotationType::Quaternions;
    }
};

template<typename T>
struct RotationOutputTypeSelector<rl4::QuaternionsToEulerAngles<T, tdm::rot_seq::zyx>> {
    static constexpr std::size_t value() {
        return 1ul;
    }

    static constexpr rl4::RotationType rotation() {
        return rl4::RotationType::EulerAngles;
    }
};
