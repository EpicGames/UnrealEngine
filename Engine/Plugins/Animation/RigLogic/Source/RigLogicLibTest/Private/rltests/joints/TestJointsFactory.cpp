// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/joints/bpcm/BPCMFixturesBlock4.h"

#include "riglogic/RigLogic.h"
#include "riglogic/TypeDefs.h"
#include "riglogic/controls/ControlsFactory.h"
#include "riglogic/joints/JointsFactory.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/riglogic/RigMetadata.h"

TEST(ScalarJointsFactoryTest, NeutralJointsAreCopied) {
    pma::AlignedMemoryResource memRes;
    block4::CanonicalReader reader;
    rl4::Configuration config{};
    config.calculationType = rl4::CalculationType::Scalar;
    auto meta = rl4::RigMetadata::create(config, &reader, &memRes, rl4::InitializationMethod::Create);
    auto controls = rl4::ControlsFactory::create(config, meta.get(), &reader, &memRes);
    auto joints = rl4::JointsFactory::create(config, meta.get(), &reader, controls.get(), &memRes);
    const float expected[] = {0.0f,  1.0f, 2.0f, 3.0f, 4.0f,  5.0f,  1.0f,  1.0f,  1.0f,  6.0f,  7.0f, 8.0f, 9.0f, 10.0f,
                              11.0f, 1.0f, 1.0f, 1.0f, 12.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 1.0f, 1.0f, 1.0f};
    ASSERT_ELEMENTS_EQ(joints->getNeutralValues(), expected, 27ul);
}
