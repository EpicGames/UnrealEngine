// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "rltests/Defs.h"

#include "riglogic/joints/cpu/CPUJointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"

namespace rl4 {

struct CPUJointsEvaluator::Accessor {

    static JointsEvaluator* getBPCMEvaluator(CPUJointsEvaluator* parent) {
        return parent->bpcmEvaluator.get();
    }

    static JointsEvaluator* getQuaternionEvaluator(CPUJointsEvaluator* parent) {
        return parent->quaternionEvaluator.get();
    }
};

namespace bpcm {

struct Evaluator::Accessor {

    template<typename T>
    static void assertRawDataEqual(const Evaluator& result, const Evaluator& expected) {
        ASSERT_EQ(result.storage.values.size<T>(), expected.storage.values.size<T>());
        ASSERT_ELEMENTS_EQ(result.storage.values.data<T>(), expected.storage.values.data<T>(), expected.storage.values.size<T>());
        ASSERT_EQ(result.storage.inputIndices, expected.storage.inputIndices);
        ASSERT_EQ(result.storage.outputIndices, expected.storage.outputIndices);
        ASSERT_EQ(result.storage.outputRotationIndices, expected.storage.outputRotationIndices);
        ASSERT_EQ(result.storage.outputRotationLODs, expected.storage.outputRotationLODs);
    }

    template<typename T>
    static void assertJointGroupsEqual(const Evaluator& result, const Evaluator& expected) {
        ASSERT_EQ(result.storage.jointGroups.size(), expected.storage.jointGroups.size());
        for (std::size_t jgIdx = 0ul; jgIdx < expected.storage.jointGroups.size(); ++jgIdx) {
            const auto& jointGroup = result.storage.jointGroups[jgIdx];
            const auto& expectedJointGroup = expected.storage.jointGroups[jgIdx];
            ASSERT_EQ(jointGroup.inputIndicesOffset, expectedJointGroup.inputIndicesOffset);
            ASSERT_EQ(jointGroup.lodsOffset, expectedJointGroup.lodsOffset);
            ASSERT_EQ(jointGroup.outputIndicesOffset, expectedJointGroup.outputIndicesOffset);
            ASSERT_EQ(jointGroup.outputRotationIndicesOffset, expectedJointGroup.outputRotationIndicesOffset);
            ASSERT_EQ(jointGroup.outputRotationLODsOffset, expectedJointGroup.outputRotationLODsOffset);
            ASSERT_EQ(jointGroup.valuesOffset, expectedJointGroup.valuesOffset);
            ASSERT_EQ(jointGroup.valuesSize, expectedJointGroup.valuesSize);
            ASSERT_EQ(jointGroup.colCount, expectedJointGroup.colCount);
            ASSERT_EQ(jointGroup.rowCount, expectedJointGroup.rowCount);
        }
    }

    template<typename T>
    static void assertLODsEqual(const Evaluator& result, const Evaluator& expected) {
        ASSERT_EQ(result.storage.lodRegions.size(), expected.storage.lodRegions.size());
        for (std::size_t lod = 0; lod < expected.storage.lodRegions.size(); ++lod) {
            const auto& lodRegion = result.storage.lodRegions[lod];
            const auto& expectedLodRegion = expected.storage.lodRegions[lod];
            ASSERT_EQ(lodRegion.inputLODs.size, expectedLodRegion.inputLODs.size);
            ASSERT_EQ(lodRegion.inputLODs.sizeAlignedTo4, expectedLodRegion.inputLODs.sizeAlignedTo4);
            ASSERT_EQ(lodRegion.inputLODs.sizeAlignedTo8, expectedLodRegion.inputLODs.sizeAlignedTo8);
            ASSERT_EQ(lodRegion.outputLODs.size, expectedLodRegion.outputLODs.size);
            ASSERT_EQ(lodRegion.outputLODs.sizePaddedToLastFullBlock, expectedLodRegion.outputLODs.sizePaddedToLastFullBlock);
            ASSERT_EQ(lodRegion.outputLODs.sizePaddedToSecondLastFullBlock,
                      expectedLodRegion.outputLODs.sizePaddedToSecondLastFullBlock);
        }
    }
};

}  // namespace bpcm

}  // namespace rl4
