// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/joints/bpcm/BPCMFixturesBlock4.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/Storage.h"
#include "riglogic/types/Extent.h"

namespace block4 {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif

// clang-format off

namespace unoptimized {

const tdm::rot_sign rotationSigns = {tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
const std::uint16_t lodCount = 4u;
const std::uint16_t rawControlCount = 13u;
const Extent dimensions = {75ul, 13ul};

const Matrix<float> values = {
    {// Joint group 0 - Pad to block-4 => block-4
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f},
    {// Joint group 1 - No padding => block-4
     2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f,
     3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f,
     4.0f, 4.0f, 4.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f},
    {// Joint group 2 - Pad to block-8 => block-8
     6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  7.0f,  7.0f,  7.0f,
     7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,
     8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,
     9.0f,  9.0f,  9.0f,  9.0f,  10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f,
     10.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 12.0f, 12.0f,
     12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f},
    {// Joint group 3 - No padding => block-8
     13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f,
     14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f,
     15.0f, 15.0f, 15.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 17.0f, 17.0f,
     17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f,
     18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f,
     19.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f},
    {
        // Joint group 4 - Pad to block-4 => block-8 + block-4
        21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 22.0f, 22.0f, 22.0f, 22.0f,
        22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f,
        23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f,
        24.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 26.0f, 26.0f, 26.0f,
        26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f,
        27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f,
        28.0f, 28.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f,
    },
    {
        // Joint group 5 - No padding => block-8 + block-4
        30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 31.0f, 31.0f, 31.0f,
        31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f,
        32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f,
        33.0f, 33.0f, 33.0f, 33.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f,
        34.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 36.0f, 36.0f,
        36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f,
        37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f,
        38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f,
        39.0f, 39.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 41.0f,
        41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f,
    },
    {
        // Joint group 6 - Pad to block-8 => block-8 + block-8
        42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 43.0f, 43.0f, 43.0f, 43.0f,
        43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f,
        44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f,
        45.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 47.0f, 47.0f, 47.0f,
        47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f,
        48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f,
        49.0f, 49.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 51.0f, 51.0f,
        51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f,
        52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f,
        53.0f, 53.0f, 53.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 55.0f,
        55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f,
    },
    {
        // Joint group 7 - No padding => block-8 + block-8
        56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 57.0f, 57.0f, 57.0f,
        57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f,
        58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f,
        59.0f, 59.0f, 59.0f, 59.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f,
        60.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 62.0f, 62.0f,
        62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f,
        63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f,
        64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f,
        65.0f, 65.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 67.0f,
        67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 68.0f, 68.0f, 68.0f, 68.0f,
        68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f,
        69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f,
        70.0f, 70.0f, 70.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f,
    },
    {
        // Joint group 8 - No padding => block-8 + block-8 => Mixed order of output indices
        65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 66.0f, 66.0f, 66.0f,
        66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f,
        67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f,
        68.0f, 68.0f, 68.0f, 68.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f,
        69.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 71.0f, 71.0f,
        71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f,
        56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f,
        57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f,
        58.0f, 58.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 60.0f,
        60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 61.0f, 61.0f, 61.0f, 61.0f,
        61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f,
        62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f,
        63.0f, 63.0f, 63.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f,
    },
    {
        // Joint group 9 - Pad to block-4 => block-4
        1.0f,
        1.0f,
        1.0f,
        1.0f,
        1.0f,
        2.0f,
        2.0f,
        2.0f,
        2.0f,
        2.0f,
    },
    {// Joint group 10 - Pad to block-4 => block-4
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     1.0f,
     2.0f,
     2.0f,
     2.0f,
     2.0f,
     2.0f},
    {// Joint group 11 - No padding => block-4
     1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f},
    {
        // Joint group 12 - Test case for joint group optimizer, order columns by LODs
        // LOD-3
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        0.0f,
        8.0f,  // tx
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        0.0f,
        8.0f,  // ty
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        0.0f,
        8.0f,  // tz
               // LOD-2
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        7.0f,
        0.0f,  // rx
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        7.0f,
        0.0f,  // ry
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        7.0f,
        0.0f,  // rz
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        7.0f,
        0.0f,  // sx
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        7.0f,
        0.0f,  // sy
        0.0f,
        1.0f,
        0.0f,
        0.0f,
        0.0f,
        4.0f,
        5.0f,
        0.0f,
        7.0f,
        0.0f,  // sz
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        // LOD-1
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        5.0f,
        6.0f,
        7.0f,
        0.0f,  // tx
        0.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        5.0f,
        6.0f,
        7.0f,
        0.0f,  // ty
        0.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        6.0f,
        7.0f,
        0.0f,  // tz
        0.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        5.0f,
        6.0f,
        7.0f,
        0.0f,  // rx
        0.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        5.0f,
        6.0f,
        7.0f,
        0.0f,  // ry
        0.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        6.0f,
        7.0f,
        0.0f,  // rz
               // LOD-0
        3.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        6.0f,
        7.0f,
        0.0f,  // sx
        3.0f,
        1.0f,
        2.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        6.0f,
        7.0f,
        0.0f,  // sy
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
    },
};

const Matrix<std::uint16_t> inputIndices = {
    {// Joint group 0
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 1
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 2
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 3
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 4
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 5
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 6
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 7
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 8
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 9
     1,
     2,
     3,
     4,
     5},
    {// Joint group 10
     7,
     8,
     9,
     10,
     11},
    {// Joint group 11
     1,
     2,
     3,
     4,
     5},
    {// Joint group 12
     1,
     2,
     3,
     4,
     10,
     5,
     6,
     7,
     8,
     9},
};
const Matrix<std::uint16_t> outputIndices = {
    {// Joint group 0
     0},
    {// Joint group 1
     9,
     10,
     11,
     12},
    {// Joint group 2
     27,
     28,
     29,
     30,
     31,
     32,
     33},
    {// Joint group 3
     36,
     37,
     38,
     39,
     40,
     41,
     42,
     43},
    {// Joint group 4
     45,
     46,
     47,
     48,
     49,
     50,
     51,
     52,
     53},
    {// Joint group 5
     54,
     55,
     56,
     57,
     58,
     59,
     60,
     61,
     62,
     63,
     64,
     65},
    {// Joint group 6
     72,
     73,
     74,
     75,
     76,
     77,
     78,
     79,
     80,
     81,
     82,
     83,
     84,
     85},
    {// Joint group 7
     90,
     91,
     92,
     93,
     94,
     95,
     96,
     97,
     98,
     99,
     100,
     101,
     102,
     103,
     104,
     105},
    {// Joint group 8
     117,
     118,
     119,
     120,
     121,
     122,
     123,
     108,
     109,
     110,
     111,
     112,
     113,
     114,
     115,
     116},
    {// Joint group 9
     126,
     127},
    {// Joint group 10
     135,
     136},
    {// Joint group 11
     147,
     149,
     156,
     148},
    {// Joint group 12
     153, 162, 163, 164, 165, 166, 167, 168, 169, 170, 154, 155, 171, 172, 173, 174, 175, 176, 177, 178, 156, 157}};
const Matrix<std::uint16_t> lods = {{// Joint group 0
                                     1,
                                     1,
                                     1,
                                     1},
                                    {// Joint group 1
                                     4,
                                     3,
                                     2,
                                     1},
                                    {// Joint group 2
                                     7,
                                     5,
                                     4,
                                     1},
                                    {// Joint group 3
                                     8,
                                     6,
                                     4,
                                     3},
                                    {// Joint group 4
                                     9,
                                     7,
                                     4,
                                     2},
                                    {// Joint group 5
                                     12,
                                     9,
                                     7,
                                     3},
                                    {// Joint group 6
                                     14,
                                     13,
                                     11,
                                     9},
                                    {// Joint group 7
                                     16,
                                     12,
                                     8,
                                     4},
                                    {// Joint group 8
                                     16,
                                     12,
                                     8,
                                     4},
                                    {// Joint group 9
                                     2,
                                     2,
                                     2,
                                     2},
                                    {// Joint group 10
                                     2,
                                     2,
                                     2,
                                     2},
                                    {// Joint group 11
                                     4,
                                     4,
                                     2,
                                     2},
                                    {// Joint group 11
                                     22,
                                     18,
                                     11,
                                     4}};

const Vector<float> neutralJointTranslationXs = {0.0f, 6.0f, 12.0f};
const Vector<float> neutralJointTranslationYs = {1.0f, 7.0f, 13.0f};
const Vector<float> neutralJointTranslationZs = {2.0f, 8.0f, 14.0f};
const Vector<float> neutralJointRotationXs = {3.0f, 9.0f, 15.0f};
const Vector<float> neutralJointRotationYs = {4.0f, 10.0f, 16.0f};
const Vector<float> neutralJointRotationZs = {5.0f, 11.0f, 17.0f};

}  // namespace unoptimized

namespace optimized {

const Extent dimensions{88ul, 13ul};

const AlignedMatrix<float> floatValues = {
    {// Joint group 0
     1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f,
     0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f,
     1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f},
    {// Joint group 1
     2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f,
     4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f,
     2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f, 2.0f, 3.0f, 4.0f, 5.0f},
    {
        // Joint group 2
        6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f,
        0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f,
        12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f,
        11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,
        10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,
        9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,
        8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,  6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 0.0f,
    },
    {
        // Joint group 3
        13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f,
        20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f,
        19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f,
        18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f,
        17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f,
        16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f,
        15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 13.0f, 14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f,
    },
    {
        // Joint group 4
        21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
        21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
        21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
        21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
        21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
        21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
        21.0f, 22.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,
        29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,
        29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,
        29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,  29.0f, 0.0f,  0.0f,  0.0f,
    },
    {
        // Joint group 5
        30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f,
        30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f,
        30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f,
        30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f,
        30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f,
        30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f,
        30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f,
        38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f,
        38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f,
        38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f, 38.0f, 39.0f, 40.0f, 41.0f,
    },
    {
        // Joint group 6
        42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f,
        42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f,
        42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f,
        42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f,
        42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f,
        42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f,
        42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,
        50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,  50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,
        50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,  50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,
        50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,  50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,
        50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,  50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,
        50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,  50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,
        50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,  50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,
    },
    {
        // Joint group 7
        56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f,
        56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f,
        56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f,
        56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f,
        56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f,
        56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f,
        56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
        64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
        64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
        64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
        64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
        64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
        64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
    },
    {
        // Joint group 8
        65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f,
        65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f,
        65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f,
        65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f,
        65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f,
        65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f,
        65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f,
        57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f,
        57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f,
        57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f,
        57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f,
        57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f,
        57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f,
    },
    {
        // Joint group 9
        1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f,
    },
    {// Joint group 10
     1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f, 1.0f, 2.0f, 0.0f, 0.0f},
    {// Joint group 11
     1.0f, 2.0f, 3.0f, 4.0f, 1.0f, 2.0f, 3.0f, 4.0f, 1.0f, 2.0f, 3.0f, 4.0f, 1.0f, 2.0f, 3.0f, 4.0f, 1.0f, 2.0f, 3.0f, 4.0f},
    {
        // Joint group 12
        8.0f, 8.0f, 8.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 5.0f, 5.0f, 5.0f, 5.0f,
        5.0f, 5.0f, 5.0f, 5.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 0.0f, 0.0f, 0.0f, 7.0f, 7.0f, 7.0f, 7.0f, 7.0f,
        0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
        0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f,
        5.0f, 5.0f, 5.0f, 0.0f, 5.0f, 5.0f, 0.0f, 0.0f, 4.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 7.0f, 7.0f, 7.0f, 7.0f,
        7.0f, 7.0f, 7.0f, 7.0f, 0.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 0.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f,
        0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 3.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
        0.0f, 0.0f, 0.0f, 0.0f, 7.0f, 0.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 0.0f, 6.0f, 0.0f, 0.0f, 0.0f, 3.0f, 0.0f, 0.0f, 0.0f,
    }};

const AlignedMatrix<std::uint16_t> halfFloatValues = {
    {
        // Joint group 0
        15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0,
        15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0, 15360, 0, 0, 0,
    },
    {
        // Joint group 1
        16384, 16896, 17408, 17664, 16384, 16896, 17408, 17664, 16384, 16896, 17408, 17664, 16384,
        16896, 17408, 17664, 16384, 16896, 17408, 17664, 16384, 16896, 17408, 17664, 16384, 16896,
        17408, 17664, 16384, 16896, 17408, 17664, 16384, 16896, 17408, 17664, 16384, 16896, 17408,
        17664, 16384, 16896, 17408, 17664, 16384, 16896, 17408, 17664, 16384, 16896, 17408, 17664,
    },
    {
        // Joint group 2
        17920, 18176, 18432, 18560, 18688, 18816, 18944, 0,     17920, 18176, 18432, 18560, 18688, 18816, 18944,
        0,     17920, 18176, 18432, 18560, 18688, 18816, 18944, 0,     17920, 18176, 18432, 18560, 18688, 18816,
        18944, 0,     17920, 18176, 18432, 18560, 18688, 18816, 18944, 0,     17920, 18176, 18432, 18560, 18688,
        18816, 18944, 0,     17920, 18176, 18432, 18560, 18688, 18816, 18944, 0,     17920, 18176, 18432, 18560,
        18688, 18816, 18944, 0,     17920, 18176, 18432, 18560, 18688, 18816, 18944, 0,     17920, 18176, 18432,
        18560, 18688, 18816, 18944, 0,     17920, 18176, 18432, 18560, 18688, 18816, 18944, 0,     17920, 18176,
        18432, 18560, 18688, 18816, 18944, 0,     17920, 18176, 18432, 18560, 18688, 18816, 18944, 0,
    },
    {
        // Joint group 3
        19072, 19200, 19328, 19456, 19520, 19584, 19648, 19712, 19072, 19200, 19328, 19456, 19520, 19584, 19648,
        19712, 19072, 19200, 19328, 19456, 19520, 19584, 19648, 19712, 19072, 19200, 19328, 19456, 19520, 19584,
        19648, 19712, 19072, 19200, 19328, 19456, 19520, 19584, 19648, 19712, 19072, 19200, 19328, 19456, 19520,
        19584, 19648, 19712, 19072, 19200, 19328, 19456, 19520, 19584, 19648, 19712, 19072, 19200, 19328, 19456,
        19520, 19584, 19648, 19712, 19072, 19200, 19328, 19456, 19520, 19584, 19648, 19712, 19072, 19200, 19328,
        19456, 19520, 19584, 19648, 19712, 19072, 19200, 19328, 19456, 19520, 19584, 19648, 19712, 19072, 19200,
        19328, 19456, 19520, 19584, 19648, 19712, 19072, 19200, 19328, 19456, 19520, 19584, 19648, 19712,
    },
    {
        // Joint group 4
        19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224, 19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224,
        19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224, 19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224,
        19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224, 19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224,
        19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224, 19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224,
        19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224, 19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224,
        19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224, 19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224,
        19776, 19840, 19904, 19968, 20032, 20096, 20160, 20224, 20288, 0,     0,     0,     20288, 0,     0,     0,
        20288, 0,     0,     0,     20288, 0,     0,     0,     20288, 0,     0,     0,     20288, 0,     0,     0,
        20288, 0,     0,     0,     20288, 0,     0,     0,     20288, 0,     0,     0,     20288, 0,     0,     0,
        20288, 0,     0,     0,     20288, 0,     0,     0,     20288, 0,     0,     0,
    },
    {
        // Joint group 5
        20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640,
        20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640,
        20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640,
        20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640,
        20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640,
        20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640,
        20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768,
        20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768,
        20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768,
        20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768, 20672, 20704, 20736, 20768,
    },
    {
        // Joint group 6
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024,
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024,
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024,
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024,
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024,
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024,
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 21216, 0,     0,
        21056, 21088, 21120, 21152, 21184, 21216, 0,     0,     21056, 21088, 21120, 21152, 21184, 21216, 0,     0,
        21056, 21088, 21120, 21152, 21184, 21216, 0,     0,     21056, 21088, 21120, 21152, 21184, 21216, 0,     0,
        21056, 21088, 21120, 21152, 21184, 21216, 0,     0,     21056, 21088, 21120, 21152, 21184, 21216, 0,     0,
        21056, 21088, 21120, 21152, 21184, 21216, 0,     0,     21056, 21088, 21120, 21152, 21184, 21216, 0,     0,
        21056, 21088, 21120, 21152, 21184, 21216, 0,     0,     21056, 21088, 21120, 21152, 21184, 21216, 0,     0,
        21056, 21088, 21120, 21152, 21184, 21216, 0,     0,     21056, 21088, 21120, 21152, 21184, 21216, 0,     0,
    },
    {
        // Joint group 7
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472,
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472,
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472,
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472,
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472,
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472,
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616,
        21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616,
        21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616,
        21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616,
        21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616,
        21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616,
        21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616,
    },
    {
        // Joint group 8
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248,
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248,
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248,
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248,
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248,
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248,
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
        21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
        21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
        21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
        21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
        21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
        21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
    },
    {
        // Joint group 9
        15360, 16384, 0, 0, 15360, 16384, 0, 0, 15360, 16384, 0, 0, 15360, 16384, 0, 0, 15360, 16384, 0, 0,
    },
    {
        // Joint group 10
        15360, 16384, 0, 0, 15360, 16384, 0, 0, 15360, 16384, 0, 0, 15360, 16384, 0, 0, 15360, 16384, 0, 0,
    },
    {
        // Joint group 11
        15360, 16384, 16896, 17408, 15360, 16384, 16896, 17408, 15360, 16384,
        16896, 17408, 15360, 16384, 16896, 17408, 15360, 16384, 16896, 17408,
    },
    {
        // Joint group 12
        18432, 18432, 18432, 0,     0,     0,     0,     0,     15360, 15360, 15360, 15360, 15360, 15360, 15360, 15360,
        17664, 17664, 17664, 17664, 17664, 17664, 17664, 17664, 17408, 17408, 17408, 17408, 17408, 17408, 17408, 17408,
        0,     0,     0,     18176, 18176, 18176, 18176, 18176, 0,     0,     0,     0,     0,     0,     0,     0,
        0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,
        0,     0,     0,     0,     0,     0,     0,     0,     15360, 15360, 15360, 15360, 15360, 15360, 15360, 15360,
        17664, 17664, 17664, 0,     17664, 17664, 0,     0,     17408, 0,     0,     0,     0,     0,     0,     0,
        18176, 18176, 18176, 18176, 18176, 18176, 18176, 18176, 0,     16384, 16384, 16384, 16384, 16384, 16384, 16384,
        0,     17920, 17920, 17920, 17920, 17920, 17920, 17920, 0,     0,     0,     0,     0,     0,     0,     16896,
        0,     0,     0,     0,     15360, 0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,
        18176, 0,     0,     0,     16384, 0,     0,     0,     17920, 0,     0,     0,     16896, 0,     0,     0,
    }};

const AlignedMatrix<std::uint16_t> inputIndices = {{// Joint group 0
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 1
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 2
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 3
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 4
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 5
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 6
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 7
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 8
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 9
                                                    1,
                                                    2,
                                                    3,
                                                    4,
                                                    5},
                                                   {// Joint group 10
                                                    7,
                                                    8,
                                                    9,
                                                    10,
                                                    11},
                                                   {// Joint group 11
                                                    1,
                                                    2,
                                                    3,
                                                    4,
                                                    5},
                                                   {// Joint group 12
                                                    9,
                                                    2,
                                                    6,
                                                    5,
                                                    8,
                                                    3,
                                                    7,
                                                    1}};

const Vector<Vector<AlignedVector<std::uint16_t>>> outputIndices = {
    { // Quaternion outputs
     {// Joint group 0
      0,
      0,
      0,
      0},
     {// Joint group 1
      10,
      11,
      12,
      13},
     {// Joint group 2
      30,
      31,
      32,
      33,
      34,
      35,
      37,
      0},
     {// Joint group 3
      40,
      41,
      42,
      43,
      44,
      45,
      47,
      48},
     {// Joint group 4
      50,
      51,
      52,
      53,
      54,
      55,
      57,
      58,
      59,
      0,
      0,
      0},
     {// Joint group 5
      60,
      61,
      62,
      63,
      64,
      65,
      67,
      68,
      69,
      70,
      71,
      72},
     {// Joint group 6
      80,
      81,
      82,
      83,
      84,
      85,
      87,
      88,
      89,
      90,
      91,
      92,
      93,
      94,
      0,
      0},
     {// Joint group 7
      100,
      101,
      102,
      103,
      104,
      105,
      107,
      108,
      109,
      110,
      111,
      112,
      113,
      114,
      115,
      117},
     {// Joint group 8
      130,
      131,
      132,
      133,
      134,
      135,
      137,
      120,
      121,
      122,
      123,
      124,
      125,
      127,
      128,
      129},
     {// Joint group 9
      140,
      141,
      0,
      0},
     {// Joint group 10
      150,
      151,
      0,
      0},
     {// Joint group 11
      163,
      165,
      173,
      164},
     {// Joint group 12
      180, 181, 182, 183, 184, 185, 187, 188, 189, 190, 191, 192, 193, 194, 195, 197, 198, 0, 0, 0}},
    { // Euler-angle outputs
     {// Joint group 0
      0,
      0,
      0,
      0},
     {// Joint group 1
      9,
      10,
      11,
      12},
     {// Joint group 2
      27,
      28,
      29,
      30,
      31,
      32,
      33,
      0},
     {// Joint group 3
      36,
      37,
      38,
      39,
      40,
      41,
      42,
      43},
     {// Joint group 4
      45,
      46,
      47,
      48,
      49,
      50,
      51,
      52,
      53,
      0,
      0,
      0},
     {// Joint group 5
      54,
      55,
      56,
      57,
      58,
      59,
      60,
      61,
      62,
      63,
      64,
      65},
     {// Joint group 6
      72,
      73,
      74,
      75,
      76,
      77,
      78,
      79,
      80,
      81,
      82,
      83,
      84,
      85,
      0,
      0},
     {// Joint group 7
      90,
      91,
      92,
      93,
      94,
      95,
      96,
      97,
      98,
      99,
      100,
      101,
      102,
      103,
      104,
      105},
     {// Joint group 8
      117,
      118,
      119,
      120,
      121,
      122,
      123,
      108,
      109,
      110,
      111,
      112,
      113,
      114,
      115,
      116},
     {// Joint group 9
      126,
      127,
      0,
      0},
     {// Joint group 10
      135,
      136,
      0,
      0},
     {// Joint group 11
      147,
      149,
      156,
      148},
     {// Joint group 12
      162, 163, 164, 165, 166, 167, 168, 169, 170, 171, 172, 173, 174, 175, 176, 177, 178, 0, 0, 0}}};

const Vector<Vector<AlignedVector<std::uint16_t>>> outputRotationIndices = {{// Quaternion outputs
                                                                             {
                                                                                 // Joint group 0
                                                                             },
                                                                             {// Joint group 1
                                                                              13},
                                                                             {// Joint group 2
                                                                              33},
                                                                             {// Joint group 3
                                                                              43},
                                                                             {// Joint group 4
                                                                              53},
                                                                             {// Joint group 5
                                                                              63},
                                                                             {// Joint group 6
                                                                              83,
                                                                              93},
                                                                             {// Joint group 7
                                                                              103,
                                                                              113},
                                                                             {// Joint group 8
                                                                              133,
                                                                              123},
                                                                             {
                                                                                 // Joint group 9
                                                                             },
                                                                             {
                                                                                 // Joint group 10
                                                                             },
                                                                             {// Joint group 11
                                                                              173,
                                                                              163},
                                                                             {// Joint group 12
                                                                              183,
                                                                              193}},
                                                                            {// Euler-angle outputs
                                                                             {
                                                                                 // Joint group 0
                                                                             },
                                                                             {
                                                                                 // Joint group 1
                                                                             },
                                                                             {
                                                                                 // Joint group 2
                                                                             },
                                                                             {
                                                                                 // Joint group 3
                                                                             },
                                                                             {
                                                                                 // Joint group 4
                                                                             },
                                                                             {
                                                                                 // Joint group 5
                                                                             },
                                                                             {
                                                                                 // Joint group 6
                                                                             },
                                                                             {
                                                                                 // Joint group 7
                                                                             },
                                                                             {
                                                                                 // Joint group 8
                                                                             },
                                                                             {
                                                                                 // Joint group 9
                                                                             },
                                                                             {
                                                                                 // Joint group 10
                                                                             },
                                                                             {
                                                                                 // Joint group 11
                                                                             },
                                                                             {
                                                                                 // Joint group 11
                                                                             }}};

const Vector<Vector<AlignedVector<std::uint16_t>>> outputRotationLODs = {{ // Quaternion outputs
                                                                          {// Joint group 0
                                                                           0,
                                                                           0,
                                                                           0,
                                                                           0},
                                                                          {// Joint group 1
                                                                           1,
                                                                           0,
                                                                           0,
                                                                           0},
                                                                          {// Joint group 2
                                                                           1,
                                                                           1,
                                                                           1,
                                                                           0},
                                                                          {// Joint group 3
                                                                           1,
                                                                           1,
                                                                           1,
                                                                           0},
                                                                          {// Joint group 4
                                                                           1,
                                                                           1,
                                                                           1,
                                                                           0},
                                                                          {// Joint group 5
                                                                           1,
                                                                           1,
                                                                           1,
                                                                           0},
                                                                          {// Joint group 6
                                                                           2,
                                                                           2,
                                                                           1,
                                                                           1},
                                                                          {// Joint group 7
                                                                           2,
                                                                           1,
                                                                           1,
                                                                           1},
                                                                          {// Joint group 8
                                                                           2,
                                                                           2,
                                                                           1,
                                                                           1},
                                                                          {// Joint group 9
                                                                           0,
                                                                           0,
                                                                           0,
                                                                           0},
                                                                          {// Joint group 10
                                                                           0,
                                                                           0,
                                                                           0,
                                                                           0},
                                                                          {// Joint group 11
                                                                           2,
                                                                           2,
                                                                           2,
                                                                           2},
                                                                          {// Joint group 12
                                                                           2,
                                                                           2,
                                                                           1,
                                                                           0}},
                                                                         {// Euler-angle outputs
                                                                          {
                                                                              // Joint group 0
                                                                          },
                                                                          {
                                                                              // Joint group 1
                                                                          },
                                                                          {
                                                                              // Joint group 2
                                                                          },
                                                                          {
                                                                              // Joint group 3
                                                                          },
                                                                          {
                                                                              // Joint group 4
                                                                          },
                                                                          {
                                                                              // Joint group 5
                                                                          },
                                                                          {
                                                                              // Joint group 6
                                                                          },
                                                                          {
                                                                              // Joint group 7
                                                                          },
                                                                          {
                                                                              // Joint group 8
                                                                          },
                                                                          {
                                                                              // Joint group 9
                                                                          },
                                                                          {
                                                                              // Joint group 10
                                                                          },
                                                                          {
                                                                              // Joint group 11
                                                                          },
                                                                          {
                                                                              // Joint group 12
                                                                          }}};

const Vector<Vector<bpcm::JointGroup>> jointGroups = {
    // {valueOffset, inputOffset, outputOffset, lodOffset, outputRotationIndicesOffset, outputRotationLODsOffset, valueSize,
    // colCount, rowCount}
    { // Quaternion outputs
     {// Joint group 0
      0,
      0,
      0,
      0,
      0,
      0,
      52,
      13,
      4},
     {// Joint group 1
      52,
      13,
      4,
      4,
      0,
      4,
      52,
      13,
      4},
     {// Joint group 2
      104,
      26,
      8,
      8,
      1,
      8,
      104,
      13,
      8},
     {// Joint group 3
      208,
      39,
      16,
      12,
      2,
      12,
      104,
      13,
      8},
     {// Joint group 4
      312,
      52,
      24,
      16,
      3,
      16,
      156,
      13,
      12},
     {// Joint group 5
      468,
      65,
      36,
      20,
      4,
      20,
      156,
      13,
      12},
     {// Joint group 6
      624,
      78,
      48,
      24,
      5,
      24,
      208,
      13,
      16},
     {// Joint group 7
      832,
      91,
      64,
      28,
      7,
      28,
      208,
      13,
      16},
     {// Joint group 8
      1040,
      104,
      80,
      32,
      9,
      32,
      208,
      13,
      16},
     {// Joint group 9
      1248,
      117,
      96,
      36,
      11,
      36,
      20,
      5,
      4},
     {// Joint group 10
      1268,
      122,
      100,
      40,
      11,
      40,
      20,
      5,
      4},
     {// Joint group 11
      1288,
      127,
      104,
      44,
      11,
      44,
      20,
      5,
      4},
     {// Joint group 12
      1308,
      132,
      108,
      48,
      13,
      48,
      160,
      8,
      20}},
    { // Euler-angle outputs
     {// Joint group 0
      0,
      0,
      0,
      0,
      0,
      0,
      52,
      13,
      4},
     {// Joint group 1
      52,
      13,
      4,
      4,
      0,
      0,
      52,
      13,
      4},
     {// Joint group 2
      104,
      26,
      8,
      8,
      0,
      0,
      104,
      13,
      8},
     {// Joint group 3
      208,
      39,
      16,
      12,
      0,
      0,
      104,
      13,
      8},
     {// Joint group 4
      312,
      52,
      24,
      16,
      0,
      0,
      156,
      13,
      12},
     {// Joint group 5
      468,
      65,
      36,
      20,
      0,
      0,
      156,
      13,
      12},
     {// Joint group 6
      624,
      78,
      48,
      24,
      0,
      0,
      208,
      13,
      16},
     {// Joint group 7
      832,
      91,
      64,
      28,
      0,
      0,
      208,
      13,
      16},
     {// Joint group 8
      1040,
      104,
      80,
      32,
      0,
      0,
      208,
      13,
      16},
     {// Joint group 9
      1248,
      117,
      96,
      36,
      0,
      0,
      20,
      5,
      4},
     {// Joint group 10
      1268,
      122,
      100,
      40,
      0,
      0,
      20,
      5,
      4},
     {// Joint group 11
      1288,
      127,
      104,
      44,
      0,
      0,
      20,
      5,
      4},
     {// Joint group 12
      1308,
      132,
      108,
      48,
      0,
      0,
      160,
      8,
      20}}};

const Matrix<LODRegion> lodRegions = {
    // {unaligned, aligned to last block-8, aligned to second last block-8}
    {
        // Joint group 0
        {{13, 12, 8}, {1, 0, 0}},  // LOD-0
        {{13, 12, 8}, {1, 0, 0}},  // LOD-1
        {{13, 12, 8}, {1, 0, 0}},  // LOD-2
        {{13, 12, 8}, {1, 0, 0}},  // LOD-3
    },
    {
        // Joint group 1
        {{13, 12, 8}, {4, 0, 0}},  // LOD-0
        {{13, 12, 8}, {3, 0, 0}},  // LOD-1
        {{13, 12, 8}, {2, 0, 0}},  // LOD-2
        {{13, 12, 8}, {1, 0, 0}},  // LOD-3
    },
    {
        // Joint group 2
        {{13, 12, 8}, {7, 8, 0}},  // LOD-0
        {{13, 12, 8}, {5, 8, 0}},  // LOD-1
        {{13, 12, 8}, {4, 8, 0}},  // LOD-2
        {{13, 12, 8}, {1, 8, 0}},  // LOD-3
    },
    {
        // Joint group 3
        {{13, 12, 8}, {8, 8, 8}},  // LOD-0
        {{13, 12, 8}, {6, 8, 0}},  // LOD-1
        {{13, 12, 8}, {4, 8, 0}},  // LOD-2
        {{13, 12, 8}, {3, 8, 0}},  // LOD-3
    },
    {
        // Joint group 4
        {{13, 12, 8}, {9, 8, 8}},  // LOD-0
        {{13, 12, 8}, {7, 8, 0}},  // LOD-1
        {{13, 12, 8}, {4, 8, 0}},  // LOD-2
        {{13, 12, 8}, {2, 8, 0}},  // LOD-3
    },
    {
        // Joint group 5
        {{13, 12, 8}, {12, 8, 8}},  // LOD-0
        {{13, 12, 8}, {9, 8, 8}},   // LOD-1
        {{13, 12, 8}, {7, 8, 0}},   // LOD-2
        {{13, 12, 8}, {3, 8, 0}},   // LOD-3
    },
    {
        // Joint group 6
        {{13, 12, 8}, {14, 16, 8}},  // LOD-0
        {{13, 12, 8}, {13, 16, 8}},  // LOD-1
        {{13, 12, 8}, {11, 16, 8}},  // LOD-2
        {{13, 12, 8}, {9, 16, 8}},   // LOD-3
    },
    {
        // Joint group 7
        {{13, 12, 8}, {16, 16, 16}},  // LOD-0
        {{13, 12, 8}, {12, 16, 8}},   // LOD-1
        {{13, 12, 8}, {8, 8, 8}},     // LOD-2
        {{13, 12, 8}, {4, 8, 0}},     // LOD-3
    },
    {
        // Joint group 8
        {{13, 12, 8}, {16, 16, 16}},  // LOD-0
        {{13, 12, 8}, {12, 16, 8}},   // LOD-1
        {{13, 12, 8}, {8, 8, 8}},     // LOD-2
        {{13, 12, 8}, {4, 8, 0}},     // LOD-3
    },
    {
        // Joint group 9
        {{5, 4, 0}, {2, 0, 0}},  // LOD-0
        {{5, 4, 0}, {2, 0, 0}},  // LOD-1
        {{5, 4, 0}, {2, 0, 0}},  // LOD-2
        {{5, 4, 0}, {2, 0, 0}},  // LOD-3
    },
    {
        // Joint group 10
        {{5, 4, 0}, {2, 0, 0}},  // LOD-0
        {{5, 4, 0}, {2, 0, 0}},  // LOD-1
        {{5, 4, 0}, {2, 0, 0}},  // LOD-2
        {{5, 4, 0}, {2, 0, 0}},  // LOD-3
    },
    {
        // Joint group 11
        {{5, 4, 0}, {4, 0, 0}},  // LOD-0
        {{5, 4, 0}, {4, 0, 0}},  // LOD-1
        {{5, 4, 0}, {2, 0, 0}},  // LOD-2
        {{5, 4, 0}, {2, 0, 0}},  // LOD-3
    },
    {
        // Joint group 12
        {{8, 8, 8}, {17, 16, 16}},  // LOD-0
        {{7, 4, 0}, {15, 16, 8}},   // LOD-1
        {{5, 4, 0}, {9, 16, 8}},    // LOD-2
        {{4, 4, 0}, {3, 8, 0}},     // LOD-3
    }};

}  // namespace optimized

namespace input {

// Calculation input values
const Vector<float> values = {1.0f, 2.0f, 3.0f, 4.0f, 0.0f, 6.0f, 7.0f, 8.0f, 9.0f, 0.0f, 11.0f, 12.0f, 13.0f};

}  // namespace input

namespace output {

// Expected output results for each LOD
const Vector<Matrix<float>> valuesPerLOD = {
    { // Quaternion outputs
     {// LOD-0
      76.0f,   0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      152.0f,  228.0f,  304.0f,  -0.1736482f,  0.0f,           0.0f,         -0.9848078f,   0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      456.0f,  532.0f,  608.0f,  0.121975f,    0.4186294f,     0.7018941f,   0.5632195f,    912.0f,  0.0f,    0.0f,
      988.0f,  1064.0f, 1140.0f, -0.4184162f,  0.1411047f,     0.7817417f,   0.4403377f,    1444.0f, 1520.0f, 0.0f,
      1596.0f, 1672.0f, 1748.0f, -0.7535122f,  0.1074113f,     -0.6339166f,  0.1372298f,    2052.0f, 2128.0f, 2204.0f,
      2280.0f, 2356.0f, 2432.0f, -0.4649149f,  -0.2635494f,    -0.773594f,   -0.3405117f,   2736.0f, 2812.0f, 2888.0f,
      2964.0f, 3040.0f, 3116.0f, 0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      3192.0f, 3268.0f, 3344.0f, -0.5973737f,  -0.148942f,     0.7646035f,   0.1906371f,    3648.0f, 3724.0f, 3800.0f,
      3876.0f, 3952.0f, 4028.0f, -0.3252805f,  0.290381f,      0.8937008f,   -0.10569f,     0.0f,    0.0f,    0.0f,
      4256.0f, 4332.0f, 4408.0f, 0.5042307f,   0.0737294f,     -0.821218f,   -0.2567421f,   4712.0f, 4788.0f, 4864.0f,
      4940.0f, 5016.0f, 5092.0f, 0.4187725f,   -0.2083559f,    -0.7320557f,  -0.4952896f,   5396.0f, 0.0f,    0.0f,
      4256.0f, 4332.0f, 4408.0f, 0.5042307f,   0.0737294f,     -0.821218f,   -0.2567421f,   4712.0f, 4788.0f, 4864.0f,
      4940.0f, 5016.0f, 5092.0f, 0.4187725f,   -0.2083559f,    -0.7320557f,  -0.4952896f,   5396.0f, 0.0f,    0.0f,
      15.0f,   30.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      40.0f,   80.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.237489700f, 0.449574471f,   0.285265595f, 0.812468469f,  0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.382683456f, 0.0f,           0.0f,         0.923879504f,  0.0f,    0.0f,    0.0f,
      62.0f,   62.0f,   62.0f,   0.552419543f, -0.174177229f,  0.552419543f, -0.599439502f, 125.0f,  125.0f,  125.0f,
      157.0f,  157.0f,  122.0f,  0.265585661f, -0.0761554763f, 0.500303566f, -0.820585668f, 128.0f,  128.0f,  0.0f},
     {// LOD-1
      76.0f,   0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      152.0f,  228.0f,  304.0f,  0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      456.0f,  532.0f,  608.0f,  -0.290381f,   0.3252805f,     -0.10569f,    0.8937008f,    0.0f,    0.0f,    0.0f,
      988.0f,  1064.0f, 1140.0f, -0.4184162f,  0.1411047f,     0.7817417f,   0.4403377f,    0.0f,    0.0f,    0.0f,
      1596.0f, 1672.0f, 1748.0f, -0.7535122f,  0.1074113f,     -0.6339166f,  0.1372298f,    2052.0f, 0.0f,    0.0f,
      2280.0f, 2356.0f, 2432.0f, -0.4649149f,  -0.2635494f,    -0.773594f,   -0.3405117f,   2736.0f, 2812.0f, 2888.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      3192.0f, 3268.0f, 3344.0f, -0.5973737f,  -0.148942f,     0.7646035f,   0.1906371f,    3648.0f, 3724.0f, 3800.0f,
      3876.0f, 3952.0f, 4028.0f, -0.9510565f,  0.0f,           0.0f,         -0.309017f,    0.0f,    0.0f,    0.0f,
      4256.0f, 4332.0f, 4408.0f, 0.5042307f,   0.0737294f,     -0.821218f,   -0.2567421f,   4712.0f, 4788.0f, 4864.0f,
      4940.0f, 5016.0f, 5092.0f, 0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      4256.0f, 4332.0f, 4408.0f, -0.495134f,   0.1205274f,     0.8575973f,   -0.0695866f,   0.0f,    0.0f,    0.0f,
      4940.0f, 5016.0f, 5092.0f, 0.4187725f,   -0.2083559f,    -0.7320557f,  -0.4952896f,   5396.0f, 0.0f,    0.0f,
      15.0f,   30.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      40.0f,   80.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.237489700f, 0.449574471f,   0.285265595f, 0.812468469f,  0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.382683456f, 0.0f,           0.0f,         0.923879504f,  0.0f,    0.0f,    0.0f,
      62.0f,   62.0f,   62.0f,   0.552419543f, -0.174177229f,  0.552419543f, -0.599439502f, 125.0f,  125.0f,  125.0f,
      157.0f,  157.0f,  122.0f,  0.265585661f, -0.0761554763f, 0.500303566f, -0.820585668f, 0.0f,    0.0f,    0.0f},
     {// LOD-2
      76.0f,   0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      152.0f,  228.0f,  0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      456.0f,  532.0f,  608.0f,  -0.309017f,   0.0f,           0.0f,         0.9510565f,    0.0f,    0.0f,    0.0f,
      988.0f,  1064.0f, 1140.0f, -0.9271839f,  0.0f,           0.0f,         -0.3746066f,   0.0f,    0.0f,    0.0f,
      1596.0f, 1672.0f, 1748.0f, -0.2079117f,  0.0f,           0.0f,         -0.9781476f,   0.0f,    0.0f,    0.0f,
      2280.0f, 2356.0f, 2432.0f, -0.4649149f,  -0.2635494f,    -0.773594f,   -0.3405117f,   2736.0f, 0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      3192.0f, 3268.0f, 3344.0f, -0.5973737f,  -0.148942f,     0.7646035f,   0.1906371f,    3648.0f, 3724.0f, 3800.0f,
      3876.0f, 3952.0f, 0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      4256.0f, 4332.0f, 4408.0f, 0.5042307f,   0.0737294f,     -0.821218f,   -0.2567421f,   4712.0f, 4788.0f, 0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      4256.0f, 0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      4940.0f, 5016.0f, 5092.0f, 0.4187725f,   -0.2083559f,    -0.7320557f,  -0.4952896f,   5396.0f, 0.0f,    0.0f,
      15.0f,   30.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      40.0f,   80.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.126078621f, -0.0337826647f, 0.256604820f, 0.957662225f,  0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f,
      62.0f,   62.0f,   62.0f,   0.552419543f, -0.174177229f,  0.552419543f, -0.599439502f, 125.0f,  125.0f,  125.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,          0.0f,    0.0f,    0.0f},
     {// LOD-3
      76.0f,   0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      152.0f,  0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      456.0f,  0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      988.0f,  1064.0f, 1140.0f, 0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      1596.0f, 1672.0f, 0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      2280.0f, 2356.0f, 2432.0f, 0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      3192.0f, 3268.0f, 3344.0f, -0.5973737f,  -0.148942f,     0.7646035f,   0.1906371f,   3648.0f, 3724.0f, 3800.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      4256.0f, 4332.0f, 4408.0f, 0.9902681f,   0.0f,           0.0f,         0.1391731f,   0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      4940.0f, 5016.0f, 5092.0f, 0.898794f,    0.0f,           0.0f,         0.4383711f,   0.0f,    0.0f,    0.0f,
      15.0f,   30.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      40.0f,   80.0f,   0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.126078621f, -0.0337826647f, 0.256604820f, 0.957662225f, 0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      62.0f,   62.0f,   62.0f,   0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f,
      0.0f,    0.0f,    0.0f,    0.0f,         0.0f,           0.0f,         1.0f,         0.0f,    0.0f,    0.0f}},
    {// Euler-angle outputs
     {
         // LOD-0
         76.0f,  // Joint group 0
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    152.0f,  228.0f,  304.0f,
         380.0f,  // Joint group 1
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,
         0.0f,    0.0f,    0.0f,    456.0f,  532.0f,  608.0f,  684.0f,  760.0f,  836.0f,  912.0f,   // Joint group 2
         0.0f,    0.0f,    988.0f,  1064.0f, 1140.0f, 1216.0f, 1292.0f, 1368.0f, 1444.0f, 1520.0f,  // Joint group 3
         0.0f,    1596.0f, 1672.0f, 1748.0f, 1824.0f, 1900.0f, 1976.0f, 2052.0f, 2128.0f, 2204.0f,  // Joint group 4
         2280.0f, 2356.0f, 2432.0f, 2508.0f, 2584.0f, 2660.0f,                                      // Joint group 5
         2736.0f, 2812.0f, 2888.0f, 2964.0f, 3040.0f, 3116.0f,                                      // Joint group 5
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    3192.0f, 3268.0f, 3344.0f, 3420.0f, 3496.0f,
         3572.0f, 3648.0f,                                               // Joint group 6
         3724.0f, 3800.0f, 3876.0f, 3952.0f, 4028.0f, 4104.0f, 4180.0f,  // Joint group 6
         0.0f,    0.0f,    0.0f,    0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 4636.0f, 4712.0f,
         4788.0f,                                                                                   // Joint group 7
         4864.0f, 4940.0f, 5016.0f, 5092.0f, 5168.0f, 5244.0f, 5320.0f, 5396.0f,                    // Joint group 7
         0.0f,    0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 4636.0f, 4712.0f, 4788.0f,  // Joint group 8
         4864.0f, 4940.0f, 5016.0f, 5092.0f, 5168.0f, 5244.0f, 5320.0f, 5396.0f,                    // Joint group 8
         0.0f,    0.0f,    15.0f,   30.0f,                                                          // Joint group 9
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    40.0f,   80.0f,             // Joint group 10
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    15.0f,
         60.0f,   30.0f,   0.0f,    0.0f,    0.0f,                                        // Joint group 11
         0.0f,    0.0f,    0.0f,    45.0f,   0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    // Joint group 11
         62.0f,   62.0f,   62.0f,   125.0f,  125.0f,  125.0f,  125.0f,  125.0f,  125.0f,  // Joint group 12
         157.0f,  157.0f,  122.0f,  157.0f,  157.0f,  122.0f,  128.0f,  128.0f,  0.0f     // Joint group 12
     },
     {
         // LOD-1
         76.0f,  // Joint group 0
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    152.0f,  228.0f,  304.0f,
         0.0f,  // Joint group 1
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,
         0.0f,    0.0f,    0.0f,    456.0f,  532.0f,  608.0f,  684.0f,  760.0f,  0.0f,    0.0f,  // Joint group 2
         0.0f,    0.0f,    988.0f,  1064.0f, 1140.0f, 1216.0f, 1292.0f, 1368.0f, 0.0f,    0.0f,  // Joint group 3
         0.0f,    1596.0f, 1672.0f, 1748.0f, 1824.0f, 1900.0f, 1976.0f, 2052.0f, 0.0f,    0.0f,  // Joint group 4
         2280.0f, 2356.0f, 2432.0f, 2508.0f, 2584.0f, 2660.0f,                                   // Joint group 5
         2736.0f, 2812.0f, 2888.0f, 0.0f,    0.0f,    0.0f,                                      // Joint group 5
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    3192.0f, 3268.0f, 3344.0f, 3420.0f, 3496.0f,
         3572.0f, 3648.0f,                                            // Joint group 6
         3724.0f, 3800.0f, 3876.0f, 3952.0f, 4028.0f, 4104.0f, 0.0f,  // Joint group 6
         0.0f,    0.0f,    0.0f,    0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 4636.0f, 4712.0f,
         4788.0f,                                                                                // Joint group 7
         4864.0f, 4940.0f, 5016.0f, 5092.0f, 0.0f,    0.0f,    0.0f,    0.0f,                    // Joint group 7
         0.0f,    0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 0.0f,    0.0f,    0.0f,  // Joint group 8
         0.0f,    4940.0f, 5016.0f, 5092.0f, 5168.0f, 5244.0f, 5320.0f, 5396.0f,                 // Joint group 8
         0.0f,    0.0f,    15.0f,   30.0f,                                                       // Joint group 9
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    40.0f,   80.0f,          // Joint group 10
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    15.0f,
         60.0f,   30.0f,   0.0f,    0.0f,    0.0f,                                        // Joint group 11
         0.0f,    0.0f,    0.0f,    45.0f,   0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    // Joint group 11
         62.0f,   62.0f,   62.0f,   125.0f,  125.0f,  125.0f,  125.0f,  125.0f,  125.0f,  // Joint group 12
         157.0f,  157.0f,  122.0f,  157.0f,  157.0f,  122.0f,  0.0f,    0.0f,    0.0f     // Joint group 12
     },
     {
         // LOD-2
         76.0f,  // Joint group 0
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    152.0f,  228.0f,  0.0f,
         0.0f,  // Joint group 1
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,
         0.0f,    0.0f,    0.0f,    456.0f,  532.0f,  608.0f,  684.0f,  0.0f,    0.0f,    0.0f,  // Joint group 2
         0.0f,    0.0f,    988.0f,  1064.0f, 1140.0f, 1216.0f, 0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 3
         0.0f,    1596.0f, 1672.0f, 1748.0f, 1824.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 4
         2280.0f, 2356.0f, 2432.0f, 2508.0f, 2584.0f, 2660.0f,                                   // Joint group 5
         2736.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                      // Joint group 5
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    3192.0f, 3268.0f, 3344.0f, 3420.0f, 3496.0f,
         3572.0f, 3648.0f,                                            // Joint group 6
         3724.0f, 3800.0f, 3876.0f, 3952.0f, 0.0f,    0.0f,    0.0f,  // Joint group 6
         0.0f,    0.0f,    0.0f,    0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 4636.0f, 4712.0f,
         4788.0f,                                                                                // Joint group 7
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                    // Joint group 7
         0.0f,    0.0f,    4256.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 8
         0.0f,    4940.0f, 5016.0f, 5092.0f, 5168.0f, 5244.0f, 5320.0f, 5396.0f,                 // Joint group 8
         0.0f,    0.0f,    15.0f,   30.0f,                                                       // Joint group 9
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    40.0f,   80.0f,          // Joint group 10
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    15.0f,
         0.0f,    30.0f,   0.0f,    0.0f,    0.0f,                                        // Joint group 11
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    // Joint group 11
         62.0f,   62.0f,   62.0f,   125.0f,  125.0f,  125.0f,  125.0f,  125.0f,  125.0f,  // Joint group 12
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f     // Joint group 12
     },
     {
         // LOD-3
         76.0f,  // Joint group 0
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    152.0f,  0.0f,    0.0f,
         0.0f,  // Joint group 1
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,
         0.0f,    0.0f,    0.0f,    456.0f,  0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 2
         0.0f,    0.0f,    988.0f,  1064.0f, 1140.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 3
         0.0f,    1596.0f, 1672.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 4
         2280.0f, 2356.0f, 2432.0f, 0.0f,    0.0f,    0.0f,                                      // Joint group 5
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                      // Joint group 5
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    3192.0f, 3268.0f, 3344.0f, 3420.0f, 3496.0f,
         3572.0f, 3648.0f,                                            // Joint group 6
         3724.0f, 3800.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 6
         0.0f,    0.0f,    0.0f,    0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 0.0f,    0.0f,    0.0f,
         0.0f,                                                                                   // Joint group 7
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                    // Joint group 7
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 8
         0.0f,    4940.0f, 5016.0f, 5092.0f, 5168.0f, 0.0f,    0.0f,    0.0f,                    // Joint group 8
         0.0f,    0.0f,    15.0f,   30.0f,                                                       // Joint group 9
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    40.0f,   80.0f,          // Joint group 10
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    15.0f,
         0.0f,    30.0f,   0.0f,    0.0f,    0.0f,                                      // Joint group 11
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 11
         62.0f,   62.0f,   62.0f,   0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 12
         0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f   // Joint group 12
     }}};

}  // namespace output

// clang-format on

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

CanonicalReader::~CanonicalReader() = default;

template<typename TValue>
struct OptimizedValues;

template<>
struct OptimizedValues<float> {
    static const AlignedMatrix<float>& get() {
        return optimized::floatValues;
    }
};

template<>
struct OptimizedValues<std::uint16_t> {
    static const AlignedMatrix<std::uint16_t>& get() {
        return optimized::halfFloatValues;
    }
};

template<typename TValue>
bpcm::Evaluator OptimizedStorage<TValue>::create(StrategyPtr strategy,
                                                 std::size_t rotationSelectorIndex,
                                                 rl4::RotationType rotationType,
                                                 MemoryResource* memRes) {
    bpcm::JointStorage storage{memRes};
    const auto& values = OptimizedValues<TValue>::get();
    for (std::uint16_t i = 0u; i < static_cast<std::uint16_t>(unoptimized::values.size()); ++i) {
        const auto offset = storage.values.size<TValue>();
        storage.values.resize<TValue>(offset + values[i].size());
        extd::copy(ConstArrayView<TValue>(values[i]),
                   ArrayView<TValue>(storage.values.data<TValue>() + offset, values[i].size()));
        storage.inputIndices.insert(storage.inputIndices.end(),
                                    optimized::inputIndices[i].begin(),
                                    optimized::inputIndices[i].end());
        storage.outputIndices.insert(storage.outputIndices.end(),
                                     optimized::outputIndices[rotationSelectorIndex][i].begin(),
                                     optimized::outputIndices[rotationSelectorIndex][i].end());
        storage.outputRotationIndices.insert(storage.outputRotationIndices.end(),
                                             optimized::outputRotationIndices[rotationSelectorIndex][i].begin(),
                                             optimized::outputRotationIndices[rotationSelectorIndex][i].end());
        storage.outputRotationLODs.insert(storage.outputRotationLODs.end(),
                                          optimized::outputRotationLODs[rotationSelectorIndex][i].begin(),
                                          optimized::outputRotationLODs[rotationSelectorIndex][i].end());
        storage.lodRegions.insert(storage.lodRegions.end(), optimized::lodRegions[i].begin(), optimized::lodRegions[i].end());
        storage.jointGroups.push_back(optimized::jointGroups[rotationSelectorIndex][i]);
    }

    const auto lastJointGroupIndex = optimized::outputIndices[rotationSelectorIndex].size() - 1ul;
    const auto numAttrsPerJoint =
        static_cast<std::uint16_t>(static_cast<std::uint8_t>(rl4::TranslationType::Vector) +
                                   static_cast<std::uint8_t>(rotationType) + static_cast<std::uint8_t>(rl4::ScaleType::Vector));
    const auto maxOutputIndex = extd::maxOf(optimized::outputIndices[rotationSelectorIndex][lastJointGroupIndex]);
    const auto outputCount = extd::roundUp(maxOutputIndex, numAttrsPerJoint);
    auto instanceFactory = [outputCount, rotationType](MemoryResource* instanceMemRes) {
        return pma::UniqueInstance<CPUJointsOutputInstance, JointsOutputInstance>::with(instanceMemRes)
            .create(outputCount, rl4::TranslationType::Vector, rotationType, rl4::ScaleType::Vector, instanceMemRes);
    };
    auto jointGroups = rl4::bpcm::StorageSnapshot<TValue, void, void>()(storage, memRes);
    return bpcm::Evaluator{std::move(storage), std::move(jointGroups), std::move(strategy), instanceFactory, memRes};
}

template<typename TValue>
bpcm::Evaluator OptimizedStorage<TValue>::create(StrategyPtr strategy,
                                                 std::size_t rotationSelectorIndex,
                                                 rl4::RotationType rotationType,
                                                 std::uint16_t jointGroupIndex,
                                                 MemoryResource* memRes) {
    bpcm::JointStorage storage{memRes};
    const auto& values = OptimizedValues<TValue>::get();
    storage.values.resize<TValue>(values[jointGroupIndex].size());
    extd::copy(ConstArrayView<TValue>(values[jointGroupIndex]),
               ArrayView<TValue>(storage.values.data<TValue>(), storage.values.size<TValue>()));
    storage.inputIndices.assign(optimized::inputIndices[jointGroupIndex].begin(), optimized::inputIndices[jointGroupIndex].end());
    storage.outputIndices.assign(optimized::outputIndices[rotationSelectorIndex][jointGroupIndex].begin(),
                                 optimized::outputIndices[rotationSelectorIndex][jointGroupIndex].end());
    storage.outputRotationIndices.assign(optimized::outputRotationIndices[rotationSelectorIndex][jointGroupIndex].begin(),
                                         optimized::outputRotationIndices[rotationSelectorIndex][jointGroupIndex].end());
    storage.outputRotationLODs.assign(optimized::outputRotationLODs[rotationSelectorIndex][jointGroupIndex].begin(),
                                      optimized::outputRotationLODs[rotationSelectorIndex][jointGroupIndex].end());
    storage.lodRegions.assign(optimized::lodRegions[jointGroupIndex].begin(), optimized::lodRegions[jointGroupIndex].end());
    storage.jointGroups.push_back(bpcm::JointGroup{0u,  // values offset
                                                   0u,  // inputIndicesOffset
                                                   0u,  // outputIndicesOffset
                                                   0u,  // lodsOffset
                                                   0u,  // outputRotationIndicesOffset
                                                   0u,  // outputRotationLODsOffset
                                                   static_cast<std::uint32_t>(storage.values.size<TValue>()),
                                                   optimized::jointGroups[rotationSelectorIndex][jointGroupIndex].colCount,
                                                   optimized::jointGroups[rotationSelectorIndex][jointGroupIndex].rowCount});

    const auto lastJointGroupIndex = optimized::outputIndices[rotationSelectorIndex].size() - 1ul;
    const auto numAttrsPerJoint =
        static_cast<std::uint16_t>(static_cast<std::uint8_t>(rl4::TranslationType::Vector) +
                                   static_cast<std::uint8_t>(rotationType) + static_cast<std::uint8_t>(rl4::ScaleType::Vector));
    const auto maxOutputIndex = extd::maxOf(optimized::outputIndices[rotationSelectorIndex][lastJointGroupIndex]);
    const auto outputCount = extd::roundUp(maxOutputIndex, numAttrsPerJoint);
    auto instanceFactory = [outputCount, rotationType](MemoryResource* instanceMemRes) {
        return pma::UniqueInstance<CPUJointsOutputInstance, JointsOutputInstance>::with(instanceMemRes)
            .create(outputCount, rl4::TranslationType::Vector, rotationType, rl4::ScaleType::Vector, instanceMemRes);
    };
    auto jointGroups = rl4::bpcm::StorageSnapshot<TValue, void, void>()(storage, memRes);
    return bpcm::Evaluator{std::move(storage), std::move(jointGroups), std::move(strategy), instanceFactory, memRes};
}

template struct OptimizedStorage<StorageValueType>;

}  // namespace block4
