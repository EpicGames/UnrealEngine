// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/joints/bpcm/BPCMFixturesBlock8.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"
#include "riglogic/types/Extent.h"

namespace block8 {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif

// clang-format off

namespace unoptimized {

const tdm::rot_sign rotationSigns = {tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
const std::uint16_t lodCount = 4u;
const std::uint16_t rawControlCount = 13u;
const Extent dimensions = {136ul, 13ul};

const Matrix<float> values = {
    {// Joint group 0
     1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 2.0f, 2.0f, 2.0f, 2.0f,
     2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f,
     3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f,
     4.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f},
    {// Joint group 1
     6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  6.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,
     7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  7.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,  8.0f,
     8.0f,  8.0f,  8.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  9.0f,  10.0f, 10.0f,
     10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 10.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f,
     11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 11.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f, 12.0f,
     12.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f, 13.0f},
    {// Joint group 2
     14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 14.0f, 15.0f, 15.0f, 15.0f, 15.0f,
     15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 15.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 16.0f,
     16.0f, 16.0f, 16.0f, 16.0f, 16.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f, 17.0f,
     17.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 18.0f, 19.0f, 19.0f, 19.0f,
     19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 19.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f,
     20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 20.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f, 21.0f,
     21.0f, 21.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f, 22.0f},
    {// Joint group 3
     23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 23.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f,
     24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 24.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f, 25.0f,
     25.0f, 25.0f, 25.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 26.0f, 27.0f, 27.0f,
     27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 27.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f,
     28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 28.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f, 29.0f,
     29.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 30.0f, 31.0f, 31.0f, 31.0f, 31.0f,
     31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 31.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f, 32.0f,
     32.0f, 32.0f, 32.0f, 32.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 33.0f, 34.0f,
     34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 34.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f,
     35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 35.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f, 36.0f,
     36.0f, 36.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 37.0f, 38.0f, 38.0f, 38.0f,
     38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f, 38.0f},
    {// Joint group 4
     39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 39.0f, 40.0f, 40.0f, 40.0f, 40.0f,
     40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 40.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 41.0f,
     41.0f, 41.0f, 41.0f, 41.0f, 41.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f,
     42.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 43.0f, 44.0f, 44.0f, 44.0f,
     44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 44.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f,
     45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 45.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f, 46.0f,
     46.0f, 46.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 47.0f, 48.0f, 48.0f,
     48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 48.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f,
     49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 49.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f, 50.0f,
     50.0f, 50.0f, 50.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 51.0f, 52.0f,
     52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 52.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f,
     53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 53.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f, 54.0f,
     54.0f, 54.0f, 54.0f, 54.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f, 55.0f},
    {// Joint group 5
     56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 56.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f,
     57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 57.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f, 58.0f,
     58.0f, 58.0f, 58.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 59.0f, 60.0f, 60.0f,
     60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 60.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f,
     61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 61.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f, 62.0f,
     62.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 63.0f, 64.0f, 64.0f, 64.0f, 64.0f,
     64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 64.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f, 65.0f,
     65.0f, 65.0f, 65.0f, 65.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 66.0f, 67.0f,
     67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 67.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f,
     68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 68.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f, 69.0f,
     69.0f, 69.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 70.0f, 71.0f, 71.0f, 71.0f,
     71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 71.0f, 72.0f, 72.0f, 72.0f, 72.0f, 72.0f, 72.0f, 72.0f, 72.0f,
     72.0f, 72.0f, 72.0f, 72.0f, 72.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f, 73.0f,
     74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 74.0f, 75.0f, 75.0f, 75.0f, 75.0f, 75.0f,
     75.0f, 75.0f, 75.0f, 75.0f, 75.0f, 75.0f, 75.0f, 75.0f, 76.0f, 76.0f, 76.0f, 76.0f, 76.0f, 76.0f, 76.0f, 76.0f, 76.0f, 76.0f,
     76.0f, 76.0f, 76.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 77.0f, 78.0f, 78.0f,
     78.0f, 78.0f, 78.0f, 78.0f, 78.0f, 78.0f, 78.0f, 78.0f, 78.0f, 78.0f, 78.0f, 79.0f, 79.0f, 79.0f, 79.0f, 79.0f, 79.0f, 79.0f,
     79.0f, 79.0f, 79.0f, 79.0f, 79.0f, 79.0f},
    {// Joint group 6
     80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  80.0f,  81.0f,  81.0f,
     81.0f,  81.0f,  81.0f,  81.0f,  81.0f,  81.0f,  81.0f,  81.0f,  81.0f,  81.0f,  81.0f,  82.0f,  82.0f,  82.0f,  82.0f,
     82.0f,  82.0f,  82.0f,  82.0f,  82.0f,  82.0f,  82.0f,  82.0f,  82.0f,  83.0f,  83.0f,  83.0f,  83.0f,  83.0f,  83.0f,
     83.0f,  83.0f,  83.0f,  83.0f,  83.0f,  83.0f,  83.0f,  84.0f,  84.0f,  84.0f,  84.0f,  84.0f,  84.0f,  84.0f,  84.0f,
     84.0f,  84.0f,  84.0f,  84.0f,  84.0f,  85.0f,  85.0f,  85.0f,  85.0f,  85.0f,  85.0f,  85.0f,  85.0f,  85.0f,  85.0f,
     85.0f,  85.0f,  85.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,  86.0f,
     86.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  87.0f,  88.0f,
     88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  88.0f,  89.0f,  89.0f,  89.0f,
     89.0f,  89.0f,  89.0f,  89.0f,  89.0f,  89.0f,  89.0f,  89.0f,  89.0f,  89.0f,  90.0f,  90.0f,  90.0f,  90.0f,  90.0f,
     90.0f,  90.0f,  90.0f,  90.0f,  90.0f,  90.0f,  90.0f,  90.0f,  91.0f,  91.0f,  91.0f,  91.0f,  91.0f,  91.0f,  91.0f,
     91.0f,  91.0f,  91.0f,  91.0f,  91.0f,  91.0f,  92.0f,  92.0f,  92.0f,  92.0f,  92.0f,  92.0f,  92.0f,  92.0f,  92.0f,
     92.0f,  92.0f,  92.0f,  92.0f,  93.0f,  93.0f,  93.0f,  93.0f,  93.0f,  93.0f,  93.0f,  93.0f,  93.0f,  93.0f,  93.0f,
     93.0f,  93.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,  94.0f,
     95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  95.0f,  96.0f,  96.0f,
     96.0f,  96.0f,  96.0f,  96.0f,  96.0f,  96.0f,  96.0f,  96.0f,  96.0f,  96.0f,  96.0f,  97.0f,  97.0f,  97.0f,  97.0f,
     97.0f,  97.0f,  97.0f,  97.0f,  97.0f,  97.0f,  97.0f,  97.0f,  97.0f,  98.0f,  98.0f,  98.0f,  98.0f,  98.0f,  98.0f,
     98.0f,  98.0f,  98.0f,  98.0f,  98.0f,  98.0f,  98.0f,  99.0f,  99.0f,  99.0f,  99.0f,  99.0f,  99.0f,  99.0f,  99.0f,
     99.0f,  99.0f,  99.0f,  99.0f,  99.0f,  100.0f, 100.0f, 100.0f, 100.0f, 100.0f, 100.0f, 100.0f, 100.0f, 100.0f, 100.0f,
     100.0f, 100.0f, 100.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f, 101.0f,
     101.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 102.0f, 103.0f,
     103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 103.0f, 104.0f, 104.0f, 104.0f,
     104.0f, 104.0f, 104.0f, 104.0f, 104.0f, 104.0f, 104.0f, 104.0f, 104.0f, 104.0f},
    {// Joint group 7
     105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 105.0f, 106.0f, 106.0f,
     106.0f, 106.0f, 106.0f, 106.0f, 106.0f, 106.0f, 106.0f, 106.0f, 106.0f, 106.0f, 106.0f, 107.0f, 107.0f, 107.0f, 107.0f,
     107.0f, 107.0f, 107.0f, 107.0f, 107.0f, 107.0f, 107.0f, 107.0f, 107.0f, 108.0f, 108.0f, 108.0f, 108.0f, 108.0f, 108.0f,
     108.0f, 108.0f, 108.0f, 108.0f, 108.0f, 108.0f, 108.0f, 109.0f, 109.0f, 109.0f, 109.0f, 109.0f, 109.0f, 109.0f, 109.0f,
     109.0f, 109.0f, 109.0f, 109.0f, 109.0f, 110.0f, 110.0f, 110.0f, 110.0f, 110.0f, 110.0f, 110.0f, 110.0f, 110.0f, 110.0f,
     110.0f, 110.0f, 110.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f, 111.0f,
     111.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 112.0f, 113.0f,
     113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 113.0f, 114.0f, 114.0f, 114.0f,
     114.0f, 114.0f, 114.0f, 114.0f, 114.0f, 114.0f, 114.0f, 114.0f, 114.0f, 114.0f, 115.0f, 115.0f, 115.0f, 115.0f, 115.0f,
     115.0f, 115.0f, 115.0f, 115.0f, 115.0f, 115.0f, 115.0f, 115.0f, 116.0f, 116.0f, 116.0f, 116.0f, 116.0f, 116.0f, 116.0f,
     116.0f, 116.0f, 116.0f, 116.0f, 116.0f, 116.0f, 117.0f, 117.0f, 117.0f, 117.0f, 117.0f, 117.0f, 117.0f, 117.0f, 117.0f,
     117.0f, 117.0f, 117.0f, 117.0f, 118.0f, 118.0f, 118.0f, 118.0f, 118.0f, 118.0f, 118.0f, 118.0f, 118.0f, 118.0f, 118.0f,
     118.0f, 118.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f, 119.0f,
     120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 120.0f, 121.0f, 121.0f,
     121.0f, 121.0f, 121.0f, 121.0f, 121.0f, 121.0f, 121.0f, 121.0f, 121.0f, 121.0f, 121.0f, 122.0f, 122.0f, 122.0f, 122.0f,
     122.0f, 122.0f, 122.0f, 122.0f, 122.0f, 122.0f, 122.0f, 122.0f, 122.0f, 123.0f, 123.0f, 123.0f, 123.0f, 123.0f, 123.0f,
     123.0f, 123.0f, 123.0f, 123.0f, 123.0f, 123.0f, 123.0f, 124.0f, 124.0f, 124.0f, 124.0f, 124.0f, 124.0f, 124.0f, 124.0f,
     124.0f, 124.0f, 124.0f, 124.0f, 124.0f, 125.0f, 125.0f, 125.0f, 125.0f, 125.0f, 125.0f, 125.0f, 125.0f, 125.0f, 125.0f,
     125.0f, 125.0f, 125.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f, 126.0f,
     126.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 127.0f, 128.0f,
     128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 128.0f, 129.0f, 129.0f, 129.0f,
     129.0f, 129.0f, 129.0f, 129.0f, 129.0f, 129.0f, 129.0f, 129.0f, 129.0f, 129.0f, 130.0f, 130.0f, 130.0f, 130.0f, 130.0f,
     130.0f, 130.0f, 130.0f, 130.0f, 130.0f, 130.0f, 130.0f, 130.0f, 131.0f, 131.0f, 131.0f, 131.0f, 131.0f, 131.0f, 131.0f,
     131.0f, 131.0f, 131.0f, 131.0f, 131.0f, 131.0f, 132.0f, 132.0f, 132.0f, 132.0f, 132.0f, 132.0f, 132.0f, 132.0f, 132.0f,
     132.0f, 132.0f, 132.0f, 132.0f, 133.0f, 133.0f, 133.0f, 133.0f, 133.0f, 133.0f, 133.0f, 133.0f, 133.0f, 133.0f, 133.0f,
     133.0f, 133.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f, 134.0f,
     135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 135.0f, 136.0f, 136.0f,
     136.0f, 136.0f, 136.0f, 136.0f, 136.0f, 136.0f, 136.0f, 136.0f, 136.0f, 136.0f, 136.0f}};

const Matrix<std::uint16_t> inputIndices = {
    {// Joint group 0
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 1
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 2
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 3
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 4
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 5
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 6
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
    {// Joint group 7
     12,
     11,
     10,
     9,
     8,
     7,
     6,
     5,
     4,
     3,
     2,
     1,
     0},
};

const Matrix<std::uint16_t> outputIndices = {
    {// Joint group 0
     0,
     1,
     2,
     3,
     4},
    {// Joint group 1
     9,
     10,
     11,
     12,
     13,
     14,
     15,
     16},
    {// Joint group 2
     27,
     28,
     29,
     30,
     31,
     32,
     33,
     34,
     35},
    {// Joint group 3
     36,
     37,
     38,
     39,
     40,
     41,
     42,
     43,
     44,
     45,
     46,
     47,
     48,
     49,
     50,
     51},
    {// Joint group 4
     54,
     55,
     56,
     57,
     58,
     59,
     60,
     61,
     62,
     63,
     64,
     65,
     66,
     67,
     68,
     69,
     70},
    {// Joint group 5
     72, 73, 74, 75, 76, 77, 78, 79, 80, 81, 82, 83, 84, 85, 86, 87, 88, 89, 90, 91, 92, 93, 94, 95},
    {// Joint group 6
     99, 100, 101, 102, 103, 104, 105, 106, 107, 108, 109, 110, 111, 112, 113, 114, 115, 116, 117, 118, 119, 120, 121, 122, 123},
    {// Joint group 7
     126, 127, 128, 129, 130, 131, 132, 133, 134, 135, 136, 137, 138, 139, 140, 141,
     142, 143, 144, 145, 146, 147, 148, 149, 150, 151, 152, 153, 154, 155, 156, 157}};

const Matrix<std::uint16_t> lods = {{// Joint group 0
                                     5,
                                     4,
                                     2,
                                     1},
                                    {// Joint group 1
                                     8,
                                     5,
                                     3,
                                     1},
                                    {// Joint group 2
                                     9,
                                     7,
                                     4,
                                     2},
                                    {// Joint group 3
                                     16,
                                     13,
                                     8,
                                     4},
                                    {// Joint group 4
                                     17,
                                     16,
                                     15,
                                     12},
                                    {// Joint group 5
                                     24,
                                     16,
                                     15,
                                     8},
                                    {// Joint group 6
                                     25,
                                     17,
                                     12,
                                     7},
                                    {// Joint group 7
                                     32,
                                     25,
                                     17,
                                     9}};

}  // namespace unoptimized

namespace optimized {

const Extent dimensions = {150ul, 13ul};

const AlignedMatrix<float> floatValues = {
    {// Joint group 0 - Pad to block-8
     1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f,
     0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f,
     3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f,
     0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f,
     5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f, 1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 0.0f, 0.0f, 0.0f},
    {// Joint group 1 - No padding => block-8
     6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,
     8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,
     10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f,
     12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f,
     6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,
     8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f, 6.0f,  7.0f,  8.0f,  9.0f,  10.0f, 11.0f, 12.0f, 13.0f},
    {// Joint group 2 - Pad to block-8 => block-16
     14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f,
     16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f,
     18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f,
     20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f,
     22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,
     0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,
     0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f,
     16.0f, 17.0f, 18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f,
     18.0f, 19.0f, 20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  14.0f, 15.0f, 16.0f, 17.0f, 18.0f, 19.0f,
     20.0f, 21.0f, 22.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f},
    {// Joint group 3 - No padding => block-16
     23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f,
     25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f,
     27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
     29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f,
     31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f,
     33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f,
     35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f,
     37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f,
     23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f,
     25.0f, 26.0f, 27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f,
     27.0f, 28.0f, 29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f, 23.0f, 24.0f, 25.0f, 26.0f, 27.0f, 28.0f,
     29.0f, 30.0f, 31.0f, 32.0f, 33.0f, 34.0f, 35.0f, 36.0f, 37.0f, 38.0f},
    {// Joint group 4 - Pad to block-8 => block-16 + block-8
     39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f,
     41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f,
     43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f,
     45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f,
     47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f,
     49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f,
     51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f,
     53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f,
     39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f,
     41.0f, 42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f,
     43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 39.0f, 40.0f, 41.0f, 42.0f, 43.0f, 44.0f,
     45.0f, 46.0f, 47.0f, 48.0f, 49.0f, 50.0f, 51.0f, 52.0f, 53.0f, 54.0f, 55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,
     0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,
     0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  55.0f, 0.0f,
     0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f},
    {// Joint group 5 - No padding -> block-16 + block-8
     56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f,
     58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f,
     60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f,
     62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f,
     64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f,
     66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f,
     68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f,
     70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f,
     56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f,
     58.0f, 59.0f, 60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f,
     60.0f, 61.0f, 62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 56.0f, 57.0f, 58.0f, 59.0f, 60.0f, 61.0f,
     62.0f, 63.0f, 64.0f, 65.0f, 66.0f, 67.0f, 68.0f, 69.0f, 70.0f, 71.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f,
     72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f,
     74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f,
     76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f,
     78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f,
     72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f, 74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f, 72.0f, 73.0f,
     74.0f, 75.0f, 76.0f, 77.0f, 78.0f, 79.0f},
    {// Joint group 6 - Pad to block-16 -> block-16 + block-16
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     80.0f, 81.0f, 82.0f, 83.0f, 84.0f,  85.0f,  86.0f,  87.0f,  88.0f,  89.0f, 90.0f, 91.0f, 92.0f, 93.0f, 94.0f, 95.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,
     96.0f, 97.0f, 98.0f, 99.0f, 100.0f, 101.0f, 102.0f, 103.0f, 104.0f, 0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f,  0.0f},
    {// Joint group 7 - No padding -> block-16 + block-16
     105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f,
     120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f,
     119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f,
     118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f,
     117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f,
     116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f,
     115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f,
     114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f, 112.0f,
     113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f, 111.0f,
     112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f, 110.0f,
     111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f, 109.0f,
     110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f, 108.0f,
     109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 105.0f, 106.0f, 107.0f,
     108.0f, 109.0f, 110.0f, 111.0f, 112.0f, 113.0f, 114.0f, 115.0f, 116.0f, 117.0f, 118.0f, 119.0f, 120.0f, 121.0f, 122.0f,
     123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f, 121.0f,
     122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f,
     121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f,
     136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f,
     135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f,
     134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f,
     133.0f, 134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f,
     132.0f, 133.0f, 134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f, 130.0f,
     131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f, 129.0f,
     130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f, 128.0f,
     129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f, 127.0f,
     128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f, 126.0f,
     127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f, 121.0f, 122.0f, 123.0f, 124.0f, 125.0f,
     126.0f, 127.0f, 128.0f, 129.0f, 130.0f, 131.0f, 132.0f, 133.0f, 134.0f, 135.0f, 136.0f}};

const AlignedMatrix<std::uint16_t> halfFloatValues = {
    {
        // Joint group 0 - Pad to block-8
        15360, 16384, 16896, 17408, 17664, 0,     0,     0,     15360, 16384, 16896, 17408, 17664, 0,     0,
        0,     15360, 16384, 16896, 17408, 17664, 0,     0,     0,     15360, 16384, 16896, 17408, 17664, 0,
        0,     0,     15360, 16384, 16896, 17408, 17664, 0,     0,     0,     15360, 16384, 16896, 17408, 17664,
        0,     0,     0,     15360, 16384, 16896, 17408, 17664, 0,     0,     0,     15360, 16384, 16896, 17408,
        17664, 0,     0,     0,     15360, 16384, 16896, 17408, 17664, 0,     0,     0,     15360, 16384, 16896,
        17408, 17664, 0,     0,     0,     15360, 16384, 16896, 17408, 17664, 0,     0,     0,     15360, 16384,
        16896, 17408, 17664, 0,     0,     0,     15360, 16384, 16896, 17408, 17664, 0,     0,     0,
    },
    {
        // Joint group 1 - No padding => block-8
        17920, 18176, 18432, 18560, 18688, 18816, 18944, 19072, 17920, 18176, 18432, 18560, 18688, 18816, 18944,
        19072, 17920, 18176, 18432, 18560, 18688, 18816, 18944, 19072, 17920, 18176, 18432, 18560, 18688, 18816,
        18944, 19072, 17920, 18176, 18432, 18560, 18688, 18816, 18944, 19072, 17920, 18176, 18432, 18560, 18688,
        18816, 18944, 19072, 17920, 18176, 18432, 18560, 18688, 18816, 18944, 19072, 17920, 18176, 18432, 18560,
        18688, 18816, 18944, 19072, 17920, 18176, 18432, 18560, 18688, 18816, 18944, 19072, 17920, 18176, 18432,
        18560, 18688, 18816, 18944, 19072, 17920, 18176, 18432, 18560, 18688, 18816, 18944, 19072, 17920, 18176,
        18432, 18560, 18688, 18816, 18944, 19072, 17920, 18176, 18432, 18560, 18688, 18816, 18944, 19072,
    },
    {
        // Joint group 2 - Pad to block-8 => block-16
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
        19200, 19328, 19456, 19520, 19584, 19648, 19712, 19776, 19840, 0, 0, 0, 0, 0, 0, 0,
    },
    {
        // Joint group 3 - No padding => block-16
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
        19904, 19968, 20032, 20096, 20160, 20224, 20288, 20352, 20416, 20480, 20512, 20544, 20576, 20608, 20640, 20672,
    },
    {
        // Joint group 4 - Pad to block-8 => block-16 + block-8
        20704, 20736, 20768, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704,
        20736, 20768, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736,
        20768, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768,
        20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800,
        20832, 20864, 20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832,
        20864, 20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832, 20864,
        20896, 20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832, 20864, 20896,
        20928, 20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832, 20864, 20896, 20928,
        20960, 20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832, 20864, 20896, 20928, 20960,
        20992, 21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832, 20864, 20896, 20928, 20960, 20992,
        21024, 21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024,
        21056, 21088, 21120, 21152, 21184, 20704, 20736, 20768, 20800, 20832, 20864, 20896, 20928, 20960, 20992, 21024, 21056,
        21088, 21120, 21152, 21184, 21216, 0,     0,     0,     0,     0,     0,     0,     21216, 0,     0,     0,     0,
        0,     0,     0,     21216, 0,     0,     0,     0,     0,     0,     0,     21216, 0,     0,     0,     0,     0,
        0,     0,     21216, 0,     0,     0,     0,     0,     0,     0,     21216, 0,     0,     0,     0,     0,     0,
        0,     21216, 0,     0,     0,     0,     0,     0,     0,     21216, 0,     0,     0,     0,     0,     0,     0,
        21216, 0,     0,     0,     0,     0,     0,     0,     21216, 0,     0,     0,     0,     0,     0,     0,     21216,
        0,     0,     0,     0,     0,     0,     0,     21216, 0,     0,     0,     0,     0,     0,     0,     21216, 0,
        0,     0,     0,     0,     0,     0,
    },
    {
        // Joint group 5 - No padding -> block-16 + block-8
        21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248,
        21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280,
        21312, 21344, 21376, 21408, 21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312,
        21344, 21376, 21408, 21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344,
        21376, 21408, 21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376,
        21408, 21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408,
        21440, 21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408, 21440,
        21472, 21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472,
        21504, 21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504,
        21520, 21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21520,
        21536, 21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21520, 21536,
        21552, 21568, 21584, 21600, 21616, 21248, 21280, 21312, 21344, 21376, 21408, 21440, 21472, 21504, 21520, 21536, 21552,
        21568, 21584, 21600, 21616, 21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632, 21648, 21664, 21680, 21696,
        21712, 21728, 21744, 21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632, 21648, 21664, 21680, 21696, 21712,
        21728, 21744, 21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632, 21648, 21664, 21680, 21696, 21712, 21728,
        21744, 21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744,
        21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632,
        21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632, 21648, 21664, 21680, 21696, 21712, 21728, 21744, 21632, 21648,
        21664, 21680, 21696, 21712, 21728, 21744,
    },
    {
        // Joint group 6 - Pad to block-16 -> block-16 + block-16
        21760, 21776, 21792, 21808, 21824, 21840, 21856, 21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760,
        21776, 21792, 21808, 21824, 21840, 21856, 21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776,
        21792, 21808, 21824, 21840, 21856, 21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792,
        21808, 21824, 21840, 21856, 21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808,
        21824, 21840, 21856, 21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824,
        21840, 21856, 21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824, 21840,
        21856, 21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824, 21840, 21856,
        21872, 21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824, 21840, 21856, 21872,
        21888, 21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824, 21840, 21856, 21872, 21888,
        21904, 21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824, 21840, 21856, 21872, 21888, 21904,
        21920, 21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824, 21840, 21856, 21872, 21888, 21904, 21920,
        21936, 21952, 21968, 21984, 22000, 21760, 21776, 21792, 21808, 21824, 21840, 21856, 21872, 21888, 21904, 21920, 21936,
        21952, 21968, 21984, 22000, 22016, 22032, 22048, 22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,
        0,     0,     0,     22016, 22032, 22048, 22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,
        0,     0,     22016, 22032, 22048, 22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,
        0,     22016, 22032, 22048, 22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,
        22016, 22032, 22048, 22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016,
        22032, 22048, 22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016, 22032,
        22048, 22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016, 22032, 22048,
        22064, 22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016, 22032, 22048, 22064,
        22080, 22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016, 22032, 22048, 22064, 22080,
        22096, 22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016, 22032, 22048, 22064, 22080, 22096,
        22112, 22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016, 22032, 22048, 22064, 22080, 22096, 22112,
        22128, 22144, 0,     0,     0,     0,     0,     0,     0,     22016, 22032, 22048, 22064, 22080, 22096, 22112, 22128,
        22144, 0,     0,     0,     0,     0,     0,     0,
    },
    {
        // Joint group 7 - No padding -> block-16 + block-16
        22160, 22176, 22192, 22208, 22224, 22240, 22256, 22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160,
        22176, 22192, 22208, 22224, 22240, 22256, 22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176,
        22192, 22208, 22224, 22240, 22256, 22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192,
        22208, 22224, 22240, 22256, 22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208,
        22224, 22240, 22256, 22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224,
        22240, 22256, 22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224, 22240,
        22256, 22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224, 22240, 22256,
        22272, 22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224, 22240, 22256, 22272,
        22288, 22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224, 22240, 22256, 22272, 22288,
        22304, 22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224, 22240, 22256, 22272, 22288, 22304,
        22320, 22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224, 22240, 22256, 22272, 22288, 22304, 22320,
        22336, 22352, 22368, 22384, 22400, 22160, 22176, 22192, 22208, 22224, 22240, 22256, 22272, 22288, 22304, 22320, 22336,
        22352, 22368, 22384, 22400, 22416, 22432, 22448, 22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568,
        22576, 22584, 22592, 22416, 22432, 22448, 22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576,
        22584, 22592, 22416, 22432, 22448, 22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584,
        22592, 22416, 22432, 22448, 22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592,
        22416, 22432, 22448, 22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416,
        22432, 22448, 22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416, 22432,
        22448, 22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416, 22432, 22448,
        22464, 22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416, 22432, 22448, 22464,
        22480, 22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416, 22432, 22448, 22464, 22480,
        22496, 22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416, 22432, 22448, 22464, 22480, 22496,
        22512, 22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416, 22432, 22448, 22464, 22480, 22496, 22512,
        22528, 22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592, 22416, 22432, 22448, 22464, 22480, 22496, 22512, 22528,
        22536, 22544, 22552, 22560, 22568, 22576, 22584, 22592,
    }};

const AlignedMatrix<std::uint16_t> inputIndices = {{// Joint group 0
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 1
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 2
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 3
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 4
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 5
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 6
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0},
                                                   {// Joint group 7
                                                    12,
                                                    11,
                                                    10,
                                                    9,
                                                    8,
                                                    7,
                                                    6,
                                                    5,
                                                    4,
                                                    3,
                                                    2,
                                                    1,
                                                    0}};

const Vector<Vector<AlignedVector<std::uint16_t>>> outputIndices = {
    { // Quaternion outputs
     {// Joint group 0
      0,
      1,
      2,
      3,
      4,
      0,
      0,
      0},
     {// Joint group 1
      10,
      11,
      12,
      13,
      14,
      15,
      17,
      18},
     {// Joint group 2
      30,
      31,
      32,
      33,
      34,
      35,
      37,
      38,
      39,
      0,
      0,
      0,
      0,
      0,
      0,
      0},
     {// Joint group 3
      40,
      41,
      42,
      43,
      44,
      45,
      47,
      48,
      49,
      50,
      51,
      52,
      53,
      54,
      55,
      57},
     {// Joint group 4
      60, 61, 62, 63, 64, 65, 67, 68, 69, 70, 71, 72, 73, 74, 75, 77, 78, 0, 0, 0, 0, 0, 0, 0},
     {// Joint group 5
      80, 81, 82, 83, 84, 85, 87, 88, 89, 90, 91, 92, 93, 94, 95, 97, 98, 99, 100, 101, 102, 103, 104, 105},
     {// Joint group 6
      110, 111, 112, 113, 114, 115, 117, 118, 119, 120, 121, 122, 123, 124, 125, 127,
      128, 129, 130, 131, 132, 133, 134, 135, 137, 0,   0,   0,   0,   0,   0,   0},
     {// Joint group 7
      140, 141, 142, 143, 144, 145, 147, 148, 149, 150, 151, 152, 153, 154, 155, 157,
      158, 159, 160, 161, 162, 163, 164, 165, 167, 168, 169, 170, 171, 172, 173, 174}},
    { // Euler-angle outputs
     {// Joint group 0
      0,
      1,
      2,
      3,
      4,
      0,
      0,
      0},
     {// Joint group 1
      9,
      10,
      11,
      12,
      13,
      14,
      15,
      16},
     {// Joint group 2
      27,
      28,
      29,
      30,
      31,
      32,
      33,
      34,
      35,
      0,
      0,
      0,
      0,
      0,
      0,
      0},
     {// Joint group 3
      36,
      37,
      38,
      39,
      40,
      41,
      42,
      43,
      44,
      45,
      46,
      47,
      48,
      49,
      50,
      51},
     {// Joint group 4
      54, 55, 56, 57, 58, 59, 60, 61, 62, 63, 64, 65, 66, 67, 68, 69, 70, 0, 0, 0, 0, 0, 0, 0},
     {// Joint group 5
      72, 73, 74, 75, 76, 77, 78, 79, 80, 81, 82, 83, 84, 85, 86, 87, 88, 89, 90, 91, 92, 93, 94, 95},
     {// Joint group 6
      99,  100, 101, 102, 103, 104, 105, 106, 107, 108, 109, 110, 111, 112, 113, 114,
      115, 116, 117, 118, 119, 120, 121, 122, 123, 0,   0,   0,   0,   0,   0,   0},
     {// Joint group 7
      126, 127, 128, 129, 130, 131, 132, 133, 134, 135, 136, 137, 138, 139, 140, 141,
      142, 143, 144, 145, 146, 147, 148, 149, 150, 151, 152, 153, 154, 155, 156, 157}}};

const Vector<Vector<AlignedVector<std::uint16_t>>> outputRotationIndices = {{ // Quaternion outputs
                                                                             {// Joint group 0
                                                                              3},
                                                                             {// Joint group 1
                                                                              13},
                                                                             {// Joint group 2
                                                                              33},
                                                                             {// Joint group 3
                                                                              43,
                                                                              53},
                                                                             {// Joint group 4
                                                                              63,
                                                                              73},
                                                                             {// Joint group 5
                                                                              83,
                                                                              93,
                                                                              103},
                                                                             {// Joint group 6
                                                                              113,
                                                                              123,
                                                                              133},
                                                                             {// Joint group 7
                                                                              143,
                                                                              153,
                                                                              163,
                                                                              173}},
                                                                            {// Euler-angle outputs
                                                                             {
                                                                                 // Joint group 0
                                                                             },
                                                                             {
                                                                                 // Joint group 1
                                                                             },
                                                                             {
                                                                                 // Joint group 2
                                                                             },
                                                                             {
                                                                                 // Joint group 3
                                                                             },
                                                                             {
                                                                                 // Joint group 4
                                                                             },
                                                                             {
                                                                                 // Joint group 5
                                                                             },
                                                                             {
                                                                                 // Joint group 6
                                                                             },
                                                                             {
                                                                                 // Joint group 7
                                                                             }}};

const Vector<Vector<AlignedVector<std::uint16_t>>> outputRotationLODs = {{ // Quaternion outputs
                                                                          {// Joint group 0
                                                                           1,
                                                                           1,
                                                                           0,
                                                                           0},
                                                                          {// Joint group 1
                                                                           1,
                                                                           1,
                                                                           0,
                                                                           0},
                                                                          {// Joint group 2
                                                                           1,
                                                                           1,
                                                                           1,
                                                                           0},
                                                                          {// Joint group 3
                                                                           2,
                                                                           2,
                                                                           1,
                                                                           1},
                                                                          {// Joint group 4
                                                                           2,
                                                                           2,
                                                                           2,
                                                                           1},
                                                                          {// Joint group 5
                                                                           3,
                                                                           2,
                                                                           2,
                                                                           1},
                                                                          {// Joint group 6
                                                                           3,
                                                                           2,
                                                                           1,
                                                                           1},
                                                                          {// Joint group 7
                                                                           4,
                                                                           3,
                                                                           2,
                                                                           1}},
                                                                         {// Euler-angle outputs
                                                                          {
                                                                              // Joint group 0
                                                                          },
                                                                          {
                                                                              // Joint group 1
                                                                          },
                                                                          {
                                                                              // Joint group 2
                                                                          },
                                                                          {
                                                                              // Joint group 3
                                                                          },
                                                                          {
                                                                              // Joint group 4
                                                                          },
                                                                          {
                                                                              // Joint group 5
                                                                          },
                                                                          {
                                                                              // Joint group 6
                                                                          },
                                                                          {
                                                                              // Joint group 7
                                                                          }}};

const Vector<Vector<bpcm::JointGroup>> jointGroups = {
    // {valueOffset, inputOffset, outputOffset, lodOffset, outputRotationIndicesOffset, outputRotationLODsOffset, valueSize,
    // colCount, rowCount}
    { // Quaternion outputs
     {// Joint group 0
      0,
      0,
      0,
      0,
      0,
      0,
      104,
      13,
      8},
     {// Joint group 1
      104,
      13,
      8,
      4,
      1,
      4,
      104,
      13,
      8},
     {// Joint group 2
      208,
      26,
      16,
      8,
      2,
      8,
      208,
      13,
      16},
     {// Joint group 3
      416,
      39,
      32,
      12,
      3,
      12,
      208,
      13,
      16},
     {// Joint group 4
      624,
      52,
      48,
      16,
      5,
      16,
      312,
      13,
      24},
     {// Joint group 5
      936,
      65,
      72,
      20,
      7,
      20,
      312,
      13,
      24},
     {// Joint group 6
      1248,
      78,
      96,
      24,
      10,
      24,
      416,
      13,
      32},
     {// Joint group 7
      1664,
      91,
      128,
      28,
      13,
      28,
      416,
      13,
      32}},
    { // Euler-angle outputs
     {// Joint group 0
      0,
      0,
      0,
      0,
      0,
      0,
      104,
      13,
      8},
     {// Joint group 1
      104,
      13,
      8,
      4,
      0,
      0,
      104,
      13,
      8},
     {// Joint group 2
      208,
      26,
      16,
      8,
      0,
      0,
      208,
      13,
      16},
     {// Joint group 3
      416,
      39,
      32,
      12,
      0,
      0,
      208,
      13,
      16},
     {// Joint group 4
      624,
      52,
      48,
      16,
      0,
      0,
      312,
      13,
      24},
     {// Joint group 5
      936,
      65,
      72,
      20,
      0,
      0,
      312,
      13,
      24},
     {// Joint group 6
      1248,
      78,
      96,
      24,
      0,
      0,
      416,
      13,
      32},
     {// Joint group 7
      1664,
      91,
      128,
      28,
      0,
      0,
      416,
      13,
      32}}};

const Matrix<LODRegion> lodRegions = {
    // {size, sizeAlignedToLastFullBlock, sizeAlignedToSecondLastFullBlock}
    {
        // Joint group 0
        {{13, 12, 8}, {5, 0, 0}},  // LOD-0
        {{13, 12, 8}, {4, 0, 0}},  // LOD-1
        {{13, 12, 8}, {2, 0, 0}},  // LOD-2
        {{13, 12, 8}, {1, 0, 0}},  // LOD-3
    },
    {
        // Joint group 1
        {{13, 12, 8}, {8, 0, 0}},  // LOD-0
        {{13, 12, 8}, {5, 0, 0}},  // LOD-1
        {{13, 12, 8}, {3, 0, 0}},  // LOD-2
        {{13, 12, 8}, {1, 0, 0}},  // LOD-3
    },
    {
        // Joint group 2
        {{13, 12, 8}, {9, 16, 0}},  // LOD-0
        {{13, 12, 8}, {7, 16, 0}},  // LOD-1
        {{13, 12, 8}, {4, 16, 0}},  // LOD-2
        {{13, 12, 8}, {2, 16, 0}},  // LOD-3
    },
    {
        // Joint group 3
        {{13, 12, 8}, {16, 16, 16}},  // LOD-0
        {{13, 12, 8}, {13, 16, 0}},   // LOD-1
        {{13, 12, 8}, {8, 16, 0}},    // LOD-2
        {{13, 12, 8}, {4, 16, 0}},    // LOD-3
    },
    {
        // Joint group 4
        {{13, 12, 8}, {17, 16, 16}},  // LOD-0
        {{13, 12, 8}, {16, 16, 16}},  // LOD-1
        {{13, 12, 8}, {15, 16, 0}},   // LOD-2
        {{13, 12, 8}, {12, 16, 0}},   // LOD-3
    },
    {
        // Joint group 5
        {{13, 12, 8}, {24, 16, 16}},  // LOD-0
        {{13, 12, 8}, {16, 16, 16}},  // LOD-1
        {{13, 12, 8}, {15, 16, 0}},   // LOD-2
        {{13, 12, 8}, {8, 16, 0}},    // LOD-3
    },
    {
        // Joint group 6
        {{13, 12, 8}, {25, 32, 16}},  // LOD-0
        {{13, 12, 8}, {17, 32, 16}},  // LOD-1
        {{13, 12, 8}, {12, 16, 0}},   // LOD-2
        {{13, 12, 8}, {7, 16, 0}},    // LOD-3
    },
    {
        // Joint group 7
        {{13, 12, 8}, {32, 32, 32}},  // LOD-0
        {{13, 12, 8}, {25, 32, 16}},  // LOD-1
        {{13, 12, 8}, {17, 32, 16}},  // LOD-2
        {{13, 12, 8}, {9, 16, 0}},    // LOD-3
    }};

}  // namespace optimized

namespace input {

// Calculation input values
const Vector<float> values = {1.0f, 2.0f, 3.0f, 4.0f, 0.0f, 6.0f, 7.0f, 8.0f, 9.0f, 0.0f, 11.0f, 12.0f, 13.0f};

}  // namespace input

namespace output {

// Expected output results for each LOD
const Vector<Matrix<float>> valuesPerLOD =
    {
        { // Quaternion outputs
         {// LOD-0
          76.0,     152.0f,   228.0f,   -0.4623392f, 0.1533222f,  -0.0815229f, 0.8695336f,  0.0f,    0.0f,    0.0f,
          456.0f,   532.0f,   608.0f,   0.121975f,   0.4186294f,  0.7018941f,  0.5632195f,  912.0f,  988.0f,  0.0f,
          0.0f,     0.0f,     0.0f,     0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          1064.0f,  1140.0f,  1216.0f,  -0.7828576f, -0.1347763f, 0.5724537f,  0.2031407f,  1520.0f, 1596.0f, 1672.0f,
          1748.0f,  1824.0f,  1900.0f,  -0.5724537f, -0.1347763f, 0.7828576f,  0.2031407f,  2204.0f, 2280.0f, 2356.0f,
          2432.0f,  2508.0f,  2584.0f,  -0.4226313f, 0.1072908f,  0.8000127f,  0.4121301f,  2888.0f, 0.0f,    0.0f,
          2964.0f,  3040.0f,  3116.0f,  0.0688182f,  -0.4470575f, -0.6379199f, -0.6232671f, 3420.0f, 3496.0f, 3572.0f,
          3648.0f,  3724.0f,  3800.0f,  0.5823712f,  -0.330285f,  -0.4532453f, -0.5884932f, 4104.0f, 4180.0f, 0.0f,
          4256.0f,  4332.0f,  4408.0f,  0.5042307f,  0.0737294f,  -0.821218f,  -0.2567421f, 4712.0f, 4788.0f, 4864.0f,
          4940.0f,  5016.0f,  5092.0f,  0.4187725f,  -0.2083559f, -0.7320557f, -0.4952896f, 5396.0f, 5472.0f, 5548.0f,
          5624.0f,  5700.0f,  5776.0f,  0.5460311f,  -0.4298623f, -0.3175257f, -0.6451712f, 0.0f,    0.0f,    0.0f,
          6080.0f,  6156.0f,  6232.0f,  -0.648358f,  -0.1669619f, 0.7219668f,  0.1746985f,  6536.0f, 6612.0f, 6688.0f,
          6764.0f,  6840.0f,  6916.0f,  -0.4522159f, 0.0098581f,  0.8288482f,  0.3292632f,  7220.0f, 7296.0f, 7372.0f,
          7448.0f,  7524.0f,  7600.0f,  -0.4406819f, 0.3023303f,  0.625722f,   0.5682146f,  7904.0f, 0.0f,    0.0f,
          7980.0f,  8056.0f,  8132.0f,  0.4318467f,  -0.3992196f, -0.5039951f, -0.6325512f, 8436.0f, 8512.0f, 8588.0f,
          8664.0f,  8740.0f,  8816.0f,  0.7817417f,  -0.1411047f, -0.4184162f, -0.4403377f, 9120.0f, 9196.0f, 9272.0f,
          9348.0f,  9424.0f,  9500.0f,  0.7985949f,  0.1173789f,  -0.5483992f, -0.2184645f, 9804.0f, 9880.0f, 9956.0f,
          10032.0f, 10108.0f, 10184.0f, -0.6156615f, 0.0f,        0.7880108f,  0.0f},
         {// LOD-1
          76.0f,   152.0f,  228.0f,  0.4694716f,  0.0f,        0.0f,        -0.8829476f, 0.0f,    0.0f,    0.0f,
          456.0f,  532.0f,  608.0f,  -0.290381f,  0.3252805f,  -0.10569f,   0.8937008f,  0.0f,    0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          1064.0f, 1140.0f, 1216.0f, -0.7828576f, -0.1347763f, 0.5724537f,  0.2031407f,  1520.0f, 0.0f,    0.0f,
          1748.0f, 1824.0f, 1900.0f, -0.5724537f, -0.1347763f, 0.7828576f,  0.2031407f,  2204.0f, 2280.0f, 2356.0f,
          2432.0f, 2508.0f, 2584.0f, -0.9396926f, 0.0f,        0.0f,        -0.3420201f, 0.0f,    0.0f,    0.0f,
          2964.0f, 3040.0f, 3116.0f, 0.0688182f,  -0.4470575f, -0.6379199f, -0.6232671f, 3420.0f, 3496.0f, 3572.0f,
          3648.0f, 3724.0f, 3800.0f, 0.5823712f,  -0.330285f,  -0.4532453f, -0.5884932f, 4104.0f, 0.0f,    0.0f,
          4256.0f, 4332.0f, 4408.0f, 0.5042307f,  0.0737294f,  -0.821218f,  -0.2567421f, 4712.0f, 4788.0f, 4864.0f,
          4940.0f, 5016.0f, 5092.0f, 0.4187725f,  -0.2083559f, -0.7320557f, -0.4952896f, 5396.0f, 0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          6080.0f, 6156.0f, 6232.0f, -0.648358f,  -0.1669619f, 0.7219668f,  0.1746985f,  6536.0f, 6612.0f, 6688.0f,
          6764.0f, 6840.0f, 6916.0f, -0.4522159f, 0.0098581f,  0.8288482f,  0.3292632f,  7220.0f, 7296.0f, 0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          7980.0f, 8056.0f, 8132.0f, 0.4318467f,  -0.3992196f, -0.5039951f, -0.6325512f, 8436.0f, 8512.0f, 8588.0f,
          8664.0f, 8740.0f, 8816.0f, 0.7817417f,  -0.1411047f, -0.4184162f, -0.4403377f, 9120.0f, 9196.0f, 9272.0f,
          9348.0f, 9424.0f, 9500.0f, 0.7985949f,  0.1173789f,  -0.5483992f, -0.2184645f, 9804.0f, 0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f},
         {// LOD-2
          76.0f,   152.0f,  0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          456.0f,  532.0f,  608.0f,  0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          1064.0f, 1140.0f, 1216.0f, -0.9612617f, 0.0f,        0.0f,        0.2756374f,  0.0f,    0.0f,    0.0,
          1748.0f, 1824.0f, 1900.0f, -0.5724537f, -0.1347763f, 0.7828576f,  0.2031407f,  2204.0f, 2280.0f, 0.0,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          2964.0f, 3040.0f, 3116.0f, 0.0688182f,  -0.4470575f, -0.6379199f, -0.6232671f, 3420.0f, 3496.0f, 3572.0,
          3648.0f, 3724.0f, 3800.0f, 0.5823712f,  -0.330285f,  -0.4532453f, -0.5884932f, 0.0f,    0.0f,    0.0,
          4256.0,  4332.0f, 4408.0f, 0.5042307f,  0.0737294f,  -0.821218f,  -0.2567421f, 4712.0f, 4788.0f, 4864.0,
          4940.0f, 5016.0f, 5092.0f, 0.4187725f,  -0.2083559f, -0.7320557f, -0.4952896f, 0.0f,    0.0f,    0.0,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          6080.0f, 6156.0f, 6232.0f, -0.648358f,  -0.1669619f, 0.7219668f,  0.1746985f,  6536.0f, 6612.0f, 6688.0,
          6764.0f, 6840.0f, 6916.0f, 0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          7980.0f, 8056.0f, 8132.0f, 0.4318467f,  -0.3992196f, -0.5039951f, -0.6325512f, 8436.0f, 8512.0f, 8588.0,
          8664.0f, 8740.0f, 8816.0f, 0.7817417f,  -0.1411047f, -0.4184162f, -0.4403377f, 9120.0f, 9196.0f, 0.0,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f},
         {// LOD-3
          76.0f,   0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          456.0f,  0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          1064.0f, 1140.0f, 0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          1748.0f, 1824.0f, 1900.0f, -0.9993908f, 0.0f,        0.0f,        -0.0348995f, 0.0f,    0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          2964.0f, 3040.0f, 3116.0f, 0.0688182f,  -0.4470575f, -0.6379199f, -0.6232671f, 3420.0f, 3496.0f, 3572.0f,
          3648.0f, 3724.0f, 3800.0f, 0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          4256.0f, 4332.0f, 4408.0f, 0.5042307f,  0.0737294f,  -0.821218f,  -0.2567421f, 4712.0f, 4788.0f, 0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          6080.0f, 6156.0f, 6232.0f, -0.648358f,  -0.1669619f, 0.7219668f,  0.1746985f,  6536.0f, 0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          7980.0f, 8056.0f, 8132.0f, 0.4318467f,  -0.3992196f, -0.5039951f, -0.6325512f, 8436.0f, 8512.0f, 8588.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f,        0.0f,    0.0f,    0.0f,
          0.0f,    0.0f,    0.0f,    0.0f,        0.0f,        0.0f,        1.0f}},
        {// Euler-angle outputs
         {
             // LOD-0
             76.0f,   152.0f,  228.0f,  304.0f,   380.0f,  // Joint group 0
             0.0f,    0.0f,    0.0f,    0.0f,     456.0f,   532.0f,   608.0f,   684.0f,  760.0f,  836.0f,
             912.0f,  988.0f,  // Joint group 1
             0.0f,    0.0f,    0.0f,    0.0f,     0.0f,     0.0f,     0.0f,     0.0f,    0.0f,    0.0f,
             1064.0f, 1140.0f, 1216.0f, 1292.0f,  1368.0f,  1444.0f,  1520.0f,  1596.0f, 1672.0f,           // Joint group 2
             1748.0f, 1824.0f, 1900.0f, 1976.0f,  2052.0f,  2128.0f,  2204.0f,  2280.0f,                    // Joint group 3
             2356.0f, 2432.0f, 2508.0f, 2584.0f,  2660.0f,  2736.0f,  2812.0f,  2888.0f,                    // Joint group 3
             0.0f,    0.0f,    2964.0f, 3040.0f,  3116.0f,  3192.0f,  3268.0f,  3344.0f, 3420.0f, 3496.0f,  // Joint group 4
             3572.0f, 3648.0f, 3724.0f, 3800.0f,  3876.0f,  3952.0f,  4028.0f,  4104.0f, 4180.0f,           // Joint group 4
             0.0f,    4256.0f, 4332.0f, 4408.0f,  4484.0f,  4560.0f,  4636.0f,  4712.0f, 4788.0f,           // Joint group 5
             4864.0f, 4940.0f, 5016.0f, 5092.0f,  5168.0f,  5244.0f,  5320.0f,  5396.0f,                    // Joint group 5
             5472.0f, 5548.0f, 5624.0f, 5700.0f,  5776.0f,  5852.0f,  5928.0f,  6004.0f,                    // Joint group 5
             0.0f,    0.0f,    0.0f,    6080.0f,  6156.0f,  6232.0f,  6308.0f,  6384.0f, 6460.0f, 6536.0f,
             6612.0f,                                                                                       // Joint group 6
             6688.0f, 6764.0f, 6840.0f, 6916.0f,  6992.0f,  7068.0f,  7144.0f,  7220.0f,                    // Joint group 6
             7296.0f, 7372.0f, 7448.0f, 7524.0f,  7600.0f,  7676.0f,  7752.0f,  7828.0f, 7904.0f,           // Joint group 6
             0.0f,    0.0f,    7980.0f, 8056.0f,  8132.0f,  8208.0f,  8284.0f,  8360.0f, 8436.0f, 8512.0f,  // Joint group 7
             8588.0f, 8664.0f, 8740.0f, 8816.0f,  8892.0f,  8968.0f,  9044.0f,  9120.0f,                    // Joint group 7
             9196.0f, 9272.0f, 9348.0f, 9424.0f,  9500.0f,  9576.0f,  9652.0f,  9728.0f,                    // Joint group 7
             9804.0f, 9880.0f, 9956.0f, 10032.0f, 10108.0f, 10184.0f, 10260.0f, 10336.0f                    // Joint group 7
         },
         {
             // LOD-1
             76.0f,   152.0f,  228.0f,  304.0f,  0.0f,  // Joint group 0
             0.0f,    0.0f,    0.0f,    0.0f,    456.0f,  532.0f,  608.0f,  684.0f,  760.0f,  0.0f,    0.0f,    0.0f,  // Joint
                                                                                                                       // group 1
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    1064.0f, 1140.0f, 1216.0f,
             1292.0f, 1368.0f, 1444.0f, 1520.0f, 0.0f,    0.0f,                                                  // Joint group 2
             1748.0f, 1824.0f, 1900.0f, 1976.0f, 2052.0f, 2128.0f, 2204.0f, 2280.0f,                             // Joint group 3
             2356.0f, 2432.0f, 2508.0f, 2584.0f, 2660.0f, 0.0f,    0.0f,    0.0f,                                // Joint group 3
             0.0f,    0.0f,    2964.0f, 3040.0f, 3116.0f, 3192.0f, 3268.0f, 3344.0f, 3420.0f, 3496.0f,           // Joint group 4
             3572.0f, 3648.0f, 3724.0f, 3800.0f, 3876.0f, 3952.0f, 4028.0f, 4104.0f, 0.0f,                       // Joint group 4
             0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 4636.0f, 4712.0f, 4788.0f,                    // Joint group 5
             4864.0f, 4940.0f, 5016.0f, 5092.0f, 5168.0f, 5244.0f, 5320.0f, 5396.0f,                             // Joint group 5
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                // Joint group 5
             0.0f,    0.0f,    0.0f,    6080.0f, 6156.0f, 6232.0f, 6308.0f, 6384.0f, 6460.0f, 6536.0f, 6612.0f,  // Joint group 6
             6688.0f, 6764.0f, 6840.0f, 6916.0f, 6992.0f, 7068.0f, 7144.0f, 7220.0f,                             // Joint group 6
             7296.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                       // Joint group 6
             0.0f,    0.0f,    7980.0f, 8056.0f, 8132.0f, 8208.0f, 8284.0f, 8360.0f, 8436.0f, 8512.0f,           // Joint group 7
             8588.0f, 8664.0f, 8740.0f, 8816.0f, 8892.0f, 8968.0f, 9044.0f, 9120.0f,                             // Joint group 7
             9196.0f, 9272.0f, 9348.0f, 9424.0f, 9500.0f, 9576.0f, 9652.0f, 9728.0f,                             // Joint group 7
             9804.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f                                 // Joint group 7
         },
         {
             // LOD-2
             76.0f,   152.0f,  0.0f,    0.0f,    0.0f,  // Joint group 0
             0.0f,    0.0f,    0.0f,    0.0f,    456.0f,  532.0f,  608.0f,  0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint
                                                                                                                       // group 1
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    1064.0f, 1140.0f, 1216.0f,
             1292.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                                  // Joint group 2
             1748.0f, 1824.0f, 1900.0f, 1976.0f, 2052.0f, 2128.0f, 2204.0f, 2280.0f,                             // Joint group 3
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                // Joint group 3
             0.0f,    0.0f,    2964.0f, 3040.0f, 3116.0f, 3192.0f, 3268.0f, 3344.0f, 3420.0f, 3496.0f,           // Joint group 4
             3572.0f, 3648.0f, 3724.0f, 3800.0f, 3876.0f, 3952.0f, 4028.0f, 0.0f,    0.0f,                       // Joint group 4
             0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 4636.0f, 4712.0f, 4788.0f,                    // Joint group 5
             4864.0f, 4940.0f, 5016.0f, 5092.0f, 5168.0f, 5244.0f, 5320.0f, 0.0f,                                // Joint group 5
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                // Joint group 5
             0.0f,    0.0f,    0.0f,    6080.0f, 6156.0f, 6232.0f, 6308.0f, 6384.0f, 6460.0f, 6536.0f, 6612.0f,  // Joint group 6
             6688.0f, 6764.0f, 6840.0f, 6916.0f, 0.0f,    0.0f,    0.0f,    0.0f,                                // Joint group 6
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                       // Joint group 6
             0.0f,    0.0f,    7980.0f, 8056.0f, 8132.0f, 8208.0f, 8284.0f, 8360.0f, 8436.0f, 8512.0f,           // Joint group 7
             8588.0f, 8664.0f, 8740.0f, 8816.0f, 8892.0f, 8968.0f, 9044.0f, 9120.0f,                             // Joint group 7
             9196.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                // Joint group 7
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f                                 // Joint group 7
         },
         {
             // LOD-3
             76.0f,   0.0f,    0.0f,    0.0f,    0.0f,  // Joint group 0
             0.0f,    0.0f,    0.0f,    0.0f,    456.0f,  0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,  // Joint
                                                                                                                       // group 1
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    1064.0f, 1140.0f, 0.0f,
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                                               // Joint group 2
             1748.0f, 1824.0f, 1900.0f, 1976.0f, 0.0f,    0.0f,    0.0f,    0.0f,                             // Joint group 3
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                             // Joint group 3
             0.0f,    0.0f,    2964.0f, 3040.0f, 3116.0f, 3192.0f, 3268.0f, 3344.0f, 3420.0f, 3496.0f,        // Joint group 4
             3572.0f, 3648.0f, 3724.0f, 3800.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                    // Joint group 4
             0.0f,    4256.0f, 4332.0f, 4408.0f, 4484.0f, 4560.0f, 4636.0f, 4712.0f, 4788.0f,                 // Joint group 5
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                             // Joint group 5
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                             // Joint group 5
             0.0f,    0.0f,    0.0f,    6080.0f, 6156.0f, 6232.0f, 6308.0f, 6384.0f, 6460.0f, 6536.0f, 0.0f,  // Joint group 6
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                             // Joint group 6
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                    // Joint group 6
             0.0f,    0.0f,    7980.0f, 8056.0f, 8132.0f, 8208.0f, 8284.0f, 8360.0f, 8436.0f, 8512.0f,        // Joint group 7
             8588.0f, 0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                             // Joint group 7
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,                             // Joint group 7
             0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f,    0.0f                              // Joint group 7
         }}};

}  // namespace output

// clang-format on

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

CanonicalReader::~CanonicalReader() = default;

template<typename TValue>
struct OptimizedValues;

template<>
struct OptimizedValues<float> {
    static const AlignedMatrix<float>& get() {
        return optimized::floatValues;
    }
};

template<>
struct OptimizedValues<std::uint16_t> {
    static const AlignedMatrix<std::uint16_t>& get() {
        return optimized::halfFloatValues;
    }
};

template<typename TValue>
bpcm::Evaluator OptimizedStorage<TValue>::create(StrategyPtr strategy,
                                                 std::size_t rotationSelectorIndex,
                                                 rl4::RotationType rotationType,
                                                 MemoryResource* memRes) {
    bpcm::JointStorage storage{memRes};
    const auto& values = OptimizedValues<TValue>::get();
    for (std::uint16_t i = 0u; i < static_cast<std::uint16_t>(unoptimized::values.size()); ++i) {
        const auto offset = storage.values.size<TValue>();
        storage.values.resize<TValue>(offset + values[i].size());
        extd::copy(ConstArrayView<TValue>(values[i]),
                   ArrayView<TValue>(storage.values.data<TValue>() + offset, values[i].size()));
        storage.inputIndices.insert(storage.inputIndices.end(),
                                    optimized::inputIndices[i].begin(),
                                    optimized::inputIndices[i].end());
        storage.outputIndices.insert(storage.outputIndices.end(),
                                     optimized::outputIndices[rotationSelectorIndex][i].begin(),
                                     optimized::outputIndices[rotationSelectorIndex][i].end());
        storage.outputRotationIndices.insert(storage.outputRotationIndices.end(),
                                             optimized::outputRotationIndices[rotationSelectorIndex][i].begin(),
                                             optimized::outputRotationIndices[rotationSelectorIndex][i].end());
        storage.outputRotationLODs.insert(storage.outputRotationLODs.end(),
                                          optimized::outputRotationLODs[rotationSelectorIndex][i].begin(),
                                          optimized::outputRotationLODs[rotationSelectorIndex][i].end());
        storage.lodRegions.insert(storage.lodRegions.end(), optimized::lodRegions[i].begin(), optimized::lodRegions[i].end());
        storage.jointGroups.push_back(optimized::jointGroups[rotationSelectorIndex][i]);
    }

    const auto lastJointGroupIndex = optimized::outputIndices[rotationSelectorIndex].size() - 1ul;
    const auto numAttrsPerJoint =
        static_cast<std::uint16_t>(static_cast<std::uint8_t>(rl4::TranslationType::Vector) +
                                   static_cast<std::uint8_t>(rotationType) + static_cast<std::uint8_t>(rl4::ScaleType::Vector));
    const auto maxOutputIndex = extd::maxOf(optimized::outputIndices[rotationSelectorIndex][lastJointGroupIndex]);
    const auto outputCount = extd::roundUp(maxOutputIndex, numAttrsPerJoint);
    auto instanceFactory = [outputCount, rotationType](MemoryResource* instanceMemRes) {
        return pma::UniqueInstance<CPUJointsOutputInstance, JointsOutputInstance>::with(instanceMemRes)
            .create(outputCount, rl4::TranslationType::Vector, rotationType, rl4::ScaleType::Vector, instanceMemRes);
    };
    auto jointGroups = rl4::bpcm::StorageSnapshot<TValue, void, void>()(storage, memRes);
    return bpcm::Evaluator{std::move(storage), std::move(jointGroups), std::move(strategy), instanceFactory, memRes};
}

template<typename TValue>
bpcm::Evaluator OptimizedStorage<TValue>::create(StrategyPtr strategy,
                                                 std::size_t rotationSelectorIndex,
                                                 rl4::RotationType rotationType,
                                                 std::uint16_t jointGroupIndex,
                                                 MemoryResource* memRes) {
    bpcm::JointStorage storage{memRes};
    const auto& values = OptimizedValues<TValue>::get();
    storage.values.resize<TValue>(values[jointGroupIndex].size());
    extd::copy(ConstArrayView<TValue>(values[jointGroupIndex]),
               ArrayView<TValue>(storage.values.data<TValue>(), storage.values.size<TValue>()));
    storage.inputIndices.assign(optimized::inputIndices[jointGroupIndex].begin(), optimized::inputIndices[jointGroupIndex].end());
    storage.outputIndices.assign(optimized::outputIndices[rotationSelectorIndex][jointGroupIndex].begin(),
                                 optimized::outputIndices[rotationSelectorIndex][jointGroupIndex].end());
    storage.outputRotationIndices.assign(optimized::outputRotationIndices[rotationSelectorIndex][jointGroupIndex].begin(),
                                         optimized::outputRotationIndices[rotationSelectorIndex][jointGroupIndex].end());
    storage.outputRotationLODs.assign(optimized::outputRotationLODs[rotationSelectorIndex][jointGroupIndex].begin(),
                                      optimized::outputRotationLODs[rotationSelectorIndex][jointGroupIndex].end());
    storage.lodRegions.assign(optimized::lodRegions[jointGroupIndex].begin(), optimized::lodRegions[jointGroupIndex].end());
    storage.jointGroups.push_back(bpcm::JointGroup{0u,  // values offset
                                                   0u,  // inputIndicesOffset
                                                   0u,  // outputIndicesOffset
                                                   0u,  // lodsOffset
                                                   0u,  // outputRotationIndicesOffset
                                                   0u,  // outputRotationLODsOffset
                                                   static_cast<std::uint32_t>(storage.values.size<TValue>()),
                                                   optimized::jointGroups[rotationSelectorIndex][jointGroupIndex].colCount,
                                                   optimized::jointGroups[rotationSelectorIndex][jointGroupIndex].rowCount});

    const auto lastJointGroupIndex = optimized::outputIndices[rotationSelectorIndex].size() - 1ul;
    const auto numAttrsPerJoint =
        static_cast<std::uint16_t>(static_cast<std::uint8_t>(rl4::TranslationType::Vector) +
                                   static_cast<std::uint8_t>(rotationType) + static_cast<std::uint8_t>(rl4::ScaleType::Vector));
    const auto maxOutputIndex = extd::maxOf(optimized::outputIndices[rotationSelectorIndex][lastJointGroupIndex]);
    const auto outputCount = extd::roundUp(maxOutputIndex, numAttrsPerJoint);
    auto instanceFactory = [outputCount, rotationType](MemoryResource* instanceMemRes) {
        return pma::UniqueInstance<CPUJointsOutputInstance, JointsOutputInstance>::with(instanceMemRes)
            .create(outputCount, rl4::TranslationType::Vector, rotationType, rl4::ScaleType::Vector, instanceMemRes);
    };
    auto jointGroups = rl4::bpcm::StorageSnapshot<TValue, void, void>()(storage, memRes);
    return bpcm::Evaluator{std::move(storage), std::move(jointGroups), std::move(strategy), instanceFactory, memRes};
}

template struct OptimizedStorage<StorageValueType>;

}  // namespace block8
