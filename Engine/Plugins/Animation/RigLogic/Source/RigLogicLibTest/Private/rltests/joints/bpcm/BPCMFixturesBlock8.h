// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/dna/FakeReader.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/JointGroup.h"
#include "riglogic/riglogic/Configuration.h"
#include "riglogic/types/Extent.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace block8 {

using namespace rl4;

namespace unoptimized {

extern const tdm::rot_sign rotationSigns;
extern const std::uint16_t lodCount;
extern const std::uint16_t rawControlCount;
extern const Extent dimensions;
extern const Matrix<float> values;
extern const Matrix<std::uint16_t> inputIndices;
extern const Matrix<std::uint16_t> outputIndices;
extern const Matrix<std::uint16_t> lods;

}  // namespace unoptimized

namespace optimized {

extern const Extent dimensions;
extern const AlignedMatrix<float> floatValues;
extern const AlignedMatrix<std::uint16_t> halfFloatValues;
extern const AlignedMatrix<std::uint16_t> inputIndices;
extern const Vector<Vector<AlignedVector<std::uint16_t>>> outputIndices;
extern const Vector<Vector<AlignedVector<std::uint16_t>>> outputRotationIndices;
extern const Vector<Vector<AlignedVector<std::uint16_t>>> outputRotationLODs;
extern const Vector<Vector<bpcm::JointGroup>> jointGroups;
extern const Matrix<LODRegion> lodRegions;

}  // namespace optimized

namespace input {

// Calculation input values
extern const Vector<float> values;

}  // namespace input

namespace output {

// Calculation output values
extern const Vector<Matrix<float>> valuesPerLOD;

}  // namespace output

class CanonicalReader : public dna::FakeReader {
public:
    ~CanonicalReader();

    std::uint16_t getLODCount() const override {
        return unoptimized::lodCount;
    }

    std::uint16_t getRawControlCount() const override {
        return unoptimized::rawControlCount;
    }

    std::uint16_t getJointRowCount() const override {
        return static_cast<std::uint16_t>(unoptimized::dimensions.rows);
    }

    std::uint16_t getJointColumnCount() const override {
        return static_cast<std::uint16_t>(unoptimized::dimensions.cols);
    }

    std::uint16_t getJointGroupCount() const override {
        return static_cast<std::uint16_t>(unoptimized::values.size());
    }

    ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{unoptimized::lods[jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{unoptimized::inputIndices[jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{unoptimized::outputIndices[jointGroupIndex]};
    }

    ConstArrayView<float> getJointGroupValues(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<float>{unoptimized::values[jointGroupIndex]};
    }
};

template<typename TValue>
struct OptimizedStorage {
    using StrategyPtr = std::unique_ptr<bpcm::JointGroupLinearCalculationStrategy,
                                        std::function<void(bpcm::JointGroupLinearCalculationStrategy*)>>;

    static bpcm::Evaluator create(StrategyPtr strategy,
                                  std::size_t rotationSelectorIndex,
                                  rl4::RotationType rotationType,
                                  MemoryResource* memRes);
    static bpcm::Evaluator create(StrategyPtr strategy,
                                  std::size_t rotationSelectorIndex,
                                  rl4::RotationType rotationType,
                                  std::uint16_t jointGroupIndex,
                                  MemoryResource* memRes);
};

}  // namespace block8
