// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "riglogic/joints/cpu/bpcm/RotationAdapters.h"
#include "riglogic/riglogic/Configuration.h"

#include <cstddef>
#include <cstdint>

struct OutputScope {
    std::uint16_t lod;
    std::size_t offset;
    std::size_t size;
};

struct StrategyTestParams {
    std::uint16_t lod;
};

template<std::uint16_t LOD>
struct TStrategyTestParams {

    static constexpr std::uint16_t lod() {
        return LOD;
    }
};

template<rl4::CalculationType CalcType>
struct TCalculationType {

    static constexpr rl4::CalculationType get() {
        return CalcType;
    }
};

template<class TRotationAdapter>
struct BPCMRotationOutputTypeSelector;

template<>
struct BPCMRotationOutputTypeSelector<rl4::bpcm::NoopAdapter> {
    static constexpr std::size_t value() {
        return 1ul;
    }

    static constexpr rl4::RotationType rotation() {
        return rl4::RotationType::EulerAngles;
    }
};

template<typename T>
struct BPCMRotationOutputTypeSelector<rl4::bpcm::EulerAnglesToQuaternions<T, tdm::rot_seq::zyx>> {
    static constexpr std::size_t value() {
        return 0ul;
    }

    static constexpr rl4::RotationType rotation() {
        return rl4::RotationType::Quaternions;
    }
};
