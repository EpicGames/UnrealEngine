// Copyright Epic Games, Inc. All Rights Reserved.

#include "riglogic/system/simd/Detect.h"

#include "rltests/Defs.h"
#include "rltests/joints/bpcm/Assertions.h"
#include "rltests/joints/bpcm/BPCMFixturesBlock4.h"
#include "rltests/joints/bpcm/Helpers.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/JointBehaviorFilter.h"
#include "riglogic/joints/JointsBuilder.h"
#include "riglogic/joints/cpu/CPUJointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/BPCMCalculationStrategy.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/RotationAdapters.h"
#include "riglogic/riglogic/RigLogic.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/SIMD.h"

namespace {

template<typename TTestTypes>
class Block4JointStorageBuilderTest : public ::testing::Test {
protected:
    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value == 0ul, void>::type buildStorage() {
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value != 0ul, void>::type buildStorage() {
        using TValue = typename std::tuple_element<0, TestTypes>::type;
        using TFVec = typename std::tuple_element<1, TestTypes>::type;
        using TCalculationType = typename std::tuple_element<2, TestTypes>::type;
        using TRotationAdapter = typename std::tuple_element<3, TestTypes>::type;

        rl4::Configuration config{};
        auto meta = rl4::RigMetadata::create(config, &reader, &memRes, rl4::InitializationMethod::Create);
        config.calculationType = TCalculationType::get();
        config.rotationType = BPCMRotationOutputTypeSelector<TRotationAdapter>::rotation();
        auto builder = rl4::JointsBuilder::create(config, meta.get(), &memRes);

        rl4::JointBehaviorFilter filter{&reader, &memRes};
        filter.include(dna::TranslationRepresentation::Vector);
        filter.include(dna::RotationRepresentation::EulerAngles);
        filter.include(dna::ScaleRepresentation::Vector);

        builder->computeStorageRequirements(filter);
        builder->allocateStorage(filter);
        builder->fillStorage(filter);
        auto joints = builder->build();
        auto jointsImpl = static_cast<rl4::CPUJointsEvaluator*>(joints.get());
        auto bpcmJointsImpl = static_cast<rl4::bpcm::Evaluator*>(rl4::CPUJointsEvaluator::Accessor::getBPCMEvaluator(jointsImpl));

        const auto rotationSelectorIndex = BPCMRotationOutputTypeSelector<TRotationAdapter>::value();
        const auto rotationType = BPCMRotationOutputTypeSelector<TRotationAdapter>::rotation();
        using StrategyType = rl4::bpcm::VectorizedJointGroupLinearCalculationStrategy<TValue, TFVec, TRotationAdapter>;
        auto strategy = pma::UniqueInstance<StrategyType, rl4::bpcm::JointGroupLinearCalculationStrategy>::with(&memRes).create(
            TRotationAdapter{block4::unoptimized::rotationSigns});
        auto expected =
            block4::OptimizedStorage<TValue>::create(std::move(strategy), rotationSelectorIndex, rotationType, &memRes);

        rl4::bpcm::Evaluator::Accessor::assertRawDataEqual<TValue>(*bpcmJointsImpl, expected);
        rl4::bpcm::Evaluator::Accessor::assertJointGroupsEqual<TValue>(*bpcmJointsImpl, expected);
        rl4::bpcm::Evaluator::Accessor::assertLODsEqual<TValue>(*bpcmJointsImpl, expected);
    }

protected:
    pma::AlignedMemoryResource memRes;
    block4::CanonicalReader reader;
};

}  // namespace

using Block4StorageValueTypeList = ::testing::Types<
#if defined(RL_BUILD_WITH_SSE)
    std::tuple<StorageValueType, trimd::sse::F128, TCalculationType<rl4::CalculationType::SSE>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::sse::F128,
               TCalculationType<rl4::CalculationType::SSE>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
#endif  // RL_BUILD_WITH_AVX || RL_BUILD_WITH_SSE
#if defined(RL_BUILD_WITH_NEON)
    std::tuple<StorageValueType, trimd::neon::F128, TCalculationType<rl4::CalculationType::NEON>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::neon::F128,
               TCalculationType<rl4::CalculationType::NEON>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
#endif  // RL_BUILD_WITH_NEON
#if !defined(RL_BUILD_WITH_HALF_FLOATS)
    std::tuple<StorageValueType, trimd::scalar::F128, TCalculationType<rl4::CalculationType::Scalar>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F128,
               TCalculationType<rl4::CalculationType::Scalar>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
#endif  // RL_BUILD_WITH_HALF_FLOATS
    std::tuple<>>;

TYPED_TEST_SUITE(Block4JointStorageBuilderTest, Block4StorageValueTypeList, );

TYPED_TEST(Block4JointStorageBuilderTest, LayoutOptimization) {
    this->buildStorage();
}
