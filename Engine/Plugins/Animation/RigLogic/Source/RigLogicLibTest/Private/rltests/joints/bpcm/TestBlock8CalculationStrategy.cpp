// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

#include "riglogic/system/simd/Detect.h"

#include "rltests/Defs.h"
#include "rltests/controls/ControlFixtures.h"
#include "rltests/joints/bpcm/BPCMFixturesBlock8.h"
#include "rltests/joints/bpcm/Helpers.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/bpcm/BPCMCalculationStrategy.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"
#include "riglogic/joints/cpu/bpcm/RotationAdapters.h"
#include "riglogic/system/simd/SIMD.h"

namespace {

template<typename TTestTypes>
class Block8JointCalculationStrategyTest : public ::testing::TestWithParam<StrategyTestParams> {
protected:
    void SetUp() override {
        Block8JointCalculationStrategyTest::SetUpImpl();
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value != 0ul, void>::type SetUpImpl() {
        using T = typename std::tuple_element<0, TestTypes>::type;
        using TFVec = typename std::tuple_element<1, TestTypes>::type;
        using TStrategyTestParams = typename std::tuple_element<2, TestTypes>::type;
        using TRotationAdapter = typename std::tuple_element<3, TestTypes>::type;
        params.lod = TStrategyTestParams::lod();

        using CalculationStrategyBase = rl4::bpcm::JointGroupLinearCalculationStrategy;
        using CalculationStrategy = rl4::bpcm::VectorizedJointGroupLinearCalculationStrategy<T, TFVec, TRotationAdapter>;
        strategy = pma::UniqueInstance<CalculationStrategy, CalculationStrategyBase>::with(&memRes).create(
            TRotationAdapter{block8::unoptimized::rotationSigns});

        rotationSelectorIndex = BPCMRotationOutputTypeSelector<TRotationAdapter>::value();
        rotationType = BPCMRotationOutputTypeSelector<TRotationAdapter>::rotation();
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value == 0ul, void>::type SetUpImpl() {
        GTEST_SKIP();
    }

    template<typename TArray>
    void execute(const rl4::bpcm::Evaluator& joints, const TArray& expected, OutputScope scope) {
        auto outputInstance = joints.createInstance(&memRes);
        outputInstance->resetOutputBuffer();
        auto outputBuffer = outputInstance->getOutputBuffer();

        auto inputInstanceFactory =
            ControlsFactory::getInstanceFactory(0, static_cast<std::uint16_t>(block8::input::values.size()), 0, 0, 0);
        rl4::Vector<rl4::ControlInitializer> initialValues;
        auto inputInstance = inputInstanceFactory(initialValues, &memRes);
        auto inputBuffer = inputInstance->getInputBuffer();
        std::copy(block8::input::values.begin(), block8::input::values.end(), inputBuffer.begin());

        rl4::ConstArrayView<float> expectedView{expected[scope.lod].data() + scope.offset, scope.size};
        rl4::ConstArrayView<float> outputView{outputBuffer.data() + scope.offset, scope.size};
        joints.calculate(inputInstance.get(), outputInstance.get(), scope.lod);
        ASSERT_ELEMENTS_NEAR(outputView, expectedView, expectedView.size(), 0.002f);
    }

protected:
    pma::AlignedMemoryResource memRes;
    block8::OptimizedStorage<StorageValueType>::StrategyPtr strategy;
    StrategyTestParams params;
    std::size_t rotationSelectorIndex;
    rl4::RotationType rotationType;
};

}  // namespace

using Block8JointCalculationTypeList = ::testing::Types<
#if defined(RL_BUILD_WITH_AVX)
    std::tuple<StorageValueType, trimd::avx::F256, TStrategyTestParams<0>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::avx::F256,
               TStrategyTestParams<0>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::avx::F256, TStrategyTestParams<1>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::avx::F256,
               TStrategyTestParams<1>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::avx::F256, TStrategyTestParams<2>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::avx::F256,
               TStrategyTestParams<2>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::avx::F256, TStrategyTestParams<3>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::avx::F256,
               TStrategyTestParams<3>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
#endif  // RL_BUILD_WITH_AVX
#if defined(RL_BUILD_WITH_AVX) || defined(RL_BUILD_WITH_SSE)
    std::tuple<StorageValueType, trimd::sse::F256, TStrategyTestParams<0>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::sse::F256,
               TStrategyTestParams<0>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::sse::F256, TStrategyTestParams<1>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::sse::F256,
               TStrategyTestParams<1>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::sse::F256, TStrategyTestParams<2>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::sse::F256,
               TStrategyTestParams<2>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::sse::F256, TStrategyTestParams<3>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::sse::F256,
               TStrategyTestParams<3>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
#endif  // RL_BUILD_WITH_AVX || RL_BUILD_WITH_SSE
#if defined(RL_BUILD_WITH_NEON)
    std::tuple<StorageValueType, trimd::neon::F256, TStrategyTestParams<0>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::neon::F256,
               TStrategyTestParams<0>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::neon::F256, TStrategyTestParams<1>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::neon::F256,
               TStrategyTestParams<1>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::neon::F256, TStrategyTestParams<2>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::neon::F256,
               TStrategyTestParams<2>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::neon::F256, TStrategyTestParams<3>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::neon::F256,
               TStrategyTestParams<3>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
#endif  // RL_BUILD_WITH_NEON
#if !defined(RL_BUILD_WITH_HALF_FLOATS)
    std::tuple<StorageValueType, trimd::scalar::F256, TStrategyTestParams<0>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               TStrategyTestParams<0>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::scalar::F256, TStrategyTestParams<1>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               TStrategyTestParams<1>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::scalar::F256, TStrategyTestParams<2>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               TStrategyTestParams<2>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::scalar::F256, TStrategyTestParams<3>, rl4::bpcm::NoopAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               TStrategyTestParams<3>,
               rl4::bpcm::EulerAnglesToQuaternions<tdm::fdeg, tdm::rot_seq::zyx>>,
#endif  // RL_BUILD_WITH_HALF_FLOATS
    std::tuple<>>;

TYPED_TEST_SUITE(Block8JointCalculationStrategyTest, Block8JointCalculationTypeList, );

TYPED_TEST(Block8JointCalculationStrategyTest, Block8Padded) {
    const std::uint16_t jointGroupIndex = 0u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, Block8Exact) {
    const std::uint16_t jointGroupIndex = 1u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, Block16Padded) {
    const std::uint16_t jointGroupIndex = 2u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, Block16Exact) {
    const std::uint16_t jointGroupIndex = 3u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, Block24Padded) {
    const std::uint16_t jointGroupIndex = 4u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, Block24Exact) {
    const std::uint16_t jointGroupIndex = 5u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, Block32Padded) {
    const std::uint16_t jointGroupIndex = 6u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, Block32Exact) {
    const std::uint16_t jointGroupIndex = 7u;
    const auto& outputIndices = block8::optimized::outputIndices[this->rotationSelectorIndex][jointGroupIndex];
    const OutputScope scope{this->params.lod, outputIndices[0], block8::unoptimized::outputIndices[jointGroupIndex].size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     jointGroupIndex,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}

TYPED_TEST(Block8JointCalculationStrategyTest, MultipleBlocks) {
    const OutputScope scope{this->params.lod, 0ul, block8::output::valuesPerLOD[this->rotationSelectorIndex].front().size()};
    auto joints = block8::OptimizedStorage<StorageValueType>::create(std::move(this->strategy),
                                                                     this->rotationSelectorIndex,
                                                                     this->rotationType,
                                                                     &this->memRes);
    this->execute(joints, block8::output::valuesPerLOD[this->rotationSelectorIndex], scope);
}
