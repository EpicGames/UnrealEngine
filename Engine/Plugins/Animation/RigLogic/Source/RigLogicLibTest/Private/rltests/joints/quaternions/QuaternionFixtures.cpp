// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/joints/quaternions/QuaternionFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/quaternions/QuaternionJointsEvaluator.h"
#include "riglogic/types/Extent.h"

namespace rltests {

namespace qs {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif

namespace unoptimized {

const tdm::rot_sign rotationSigns{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
const std::uint16_t lodCount = 4u;
const Extent dimensions{198ul, 13ul};

const Matrix<float> values = {{
                                  // Joint group 0 - 8 quaternions for 2 expressions
                                  // e0     e1
                                  30.0f, 20.0f,  // rx
                                  15.0f, 30.0f,  // ry
                                  20.0f, 15.0f,  // rz
                                  10.0f, 40.0f,  // rx
                                  20.0f, 10.0f,  // ry
                                  40.0f, 20.0f,  // rz
                                  25.0f, 45.0f,  // rx
                                  45.0f, 25.0f,  // ry
                                  35.0f, 35.0f,  // rz
                                  5.0f,  35.0f,  // rx
                                  35.0f, 15.0f,  // ry
                                  15.0f, 5.0f,   // rz
                                  5.0f,  10.0f,  // rx
                                  5.0f,  10.0f,  // ry
                                  5.0f,  10.0f,  // rz
                                  15.0f, 20.0f,  // rx
                                  15.0f, 20.0f,  // ry
                                  15.0f, 20.0f,  // rz
                                  25.0f, 30.0f,  // rx
                                  25.0f, 30.0f,  // ry
                                  25.0f, 30.0f,  // rz
                                  35.0f, 40.0f,  // rx
                                  35.0f, 40.0f,  // ry
                                  35.0f, 40.0f   // rz
                              },
                              {
                                  // Joint group 1 - 4 quaternions for 2 expressions
                                  // e0     e1
                                  30.0f, 20.0f,  // rx
                                  15.0f, 30.0f,  // ry
                                  20.0f, 15.0f,  // rz
                                  10.0f, 40.0f,  // rx
                                  20.0f, 10.0f,  // ry
                                  40.0f, 20.0f,  // rz
                                  25.0f, 45.0f,  // rx
                                  45.0f, 25.0f,  // ry
                                  35.0f, 35.0f,  // rz
                                  5.0f,  35.0f,  // rx
                                  35.0f, 15.0f,  // ry
                                  15.0f, 5.0f    // rz
                              },
                              {
                                  // Joint group 2 - 7 quaternions for 2 expressions
                                  // e0     e1
                                  30.0f, 20.0f,  // rx
                                  15.0f, 30.0f,  // ry
                                  20.0f, 15.0f,  // rz
                                  10.0f, 40.0f,  // rx
                                  20.0f, 10.0f,  // ry
                                  40.0f, 20.0f,  // rz
                                  25.0f, 45.0f,  // rx
                                  45.0f, 25.0f,  // ry
                                  35.0f, 35.0f,  // rz
                                  5.0f,  35.0f,  // rx
                                  35.0f, 15.0f,  // ry
                                  15.0f, 5.0f,   // rz
                                  5.0f,  10.0f,  // rx
                                  5.0f,  10.0f,  // ry
                                  5.0f,  10.0f,  // rz
                                  15.0f, 20.0f,  // rx
                                  15.0f, 20.0f,  // ry
                                  15.0f, 20.0f,  // rz
                                  25.0f, 30.0f,  // rx
                                  25.0f, 30.0f,  // ry
                                  25.0f, 30.0f   // rz
                              },
                              {
                                  // Joint group 3 - 3 quaternions for 2 expressions
                                  // e0     e1
                                  30.0f,
                                  20.0f,  // rx
                                  15.0f,
                                  30.0f,  // ry
                                  20.0f,
                                  15.0f,  // rz
                                  10.0f,
                                  40.0f,  // rx
                                  20.0f,
                                  10.0f,  // ry
                                  40.0f,
                                  20.0f,  // rz
                                  25.0f,
                                  45.0f,  // rx
                                  45.0f,
                                  25.0f,  // ry
                                  35.0f,
                                  35.0f  // rz
                              },
                              {
                                  // Joint group 4 - no quaternions
                                  // e0     e1
                                  30.0f,
                                  20.0f,  // rx
                                  15.0f,
                                  30.0f  // ry
                              },
                              {
                                  // Joint group 5 - empty
                              },
                              {
                                  // Joint group 6 - 2 quaternions for 2 expressions
                                  // e0     e1
                                  30.0f,
                                  20.0f,  // rx
                                  15.0f,
                                  30.0f,  // ry
                                  20.0f,
                                  15.0f,  // rz
                                  10.0f,
                                  40.0f,  // rx
                                  20.0f,
                                  10.0f,  // ry
                                  40.0f,
                                  20.0f  // rz
                              },
                              {
                                  // Joint group 7 - test for joint group filter
                                  // LOD-3
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,  // tx
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  0.0f,
                                  8.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  0.0f,
                                  8.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  0.0f,
                                  8.0f,  // rz
                                         // LOD-2
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  7.0f,
                                  0.0f,  // rz
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  4.0f,
                                  5.0f,
                                  0.0f,
                                  7.0f,
                                  0.0f,  // rz
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,  // sy
                                         // LOD-1
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,  // tz
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rz
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rz
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rz
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rz
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  5.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rz
                                         // LOD-0
                                  3.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // rx
                                  3.0f,
                                  1.0f,
                                  2.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  6.0f,
                                  7.0f,
                                  0.0f,  // ry
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,  // sy
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,
                                  0.0f,  // tx
                              }};
const Matrix<std::uint16_t> inputIndices = {{// Joint group 0
                                             8,
                                             6},
                                            {// Joint group 1
                                             8,
                                             6},
                                            {// Joint group 2
                                             8,
                                             6},
                                            {// Joint group 3
                                             8,
                                             6},
                                            {// Joint group 4
                                             8,
                                             6},
                                            {
                                                // Joint group 5
                                            },
                                            {// Joint group 6
                                             8,
                                             6},
                                            {// Joint group 7
                                             1,
                                             2,
                                             3,
                                             4,
                                             10,
                                             5,
                                             6,
                                             7,
                                             8,
                                             9}};
const Matrix<std::uint16_t> outputIndices = {
    {// Joint group 0
     3, 4, 5, 12, 13, 14, 21, 22, 23, 30, 31, 32, 39, 40, 41, 48, 49, 50, 57, 58, 59, 66, 67, 68},
    {// Joint group 1
     75,
     76,
     77,
     84,
     85,
     86,
     93,
     94,
     95,
     102,
     103,
     104},
    {// Joint group 2
     111, 112, 113, 120, 121, 122, 129, 130, 131, 138, 139, 140, 147, 148, 149, 156, 157, 158, 165, 166, 167},
    {// Joint group 3
     174,
     175,
     176,
     183,
     184,
     185,
     192,
     193,
     194},
    {// Joint group 4
     200,
     201},
    {
        // Joint group 5
    },
    {// Joint group 6
     219,
     220,
     221,
     210,
     211,
     212},
    {
        // Joint group 7
        225, 228, 229, 230,                                                              // LOD-3
        237, 238, 239, 246, 247, 248, 250,                                               // LOD-2
        254, 255, 256, 257, 264, 265, 266, 273, 274, 275, 282, 283, 284, 291, 292, 293,  // LOD-1
        300, 301, 304, 306                                                               // LOD-0
    }};
const Matrix<std::uint16_t> lods = {{// Joint group 0
                                     24,
                                     12,
                                     12,
                                     0},
                                    {// Joint group 1
                                     12,
                                     6,
                                     6,
                                     0},
                                    {// Joint group 2
                                     21,
                                     9,
                                     9,
                                     0},
                                    {// Joint group 3
                                     9,
                                     3,
                                     3,
                                     0},
                                    {// Joint group 4
                                     2,
                                     2,
                                     0,
                                     0},
                                    {// Joint group 5
                                     0,
                                     0,
                                     0,
                                     0},
                                    {// Joint group 6
                                     6,
                                     3,
                                     0,
                                     0},
                                    {// Joint group 7
                                     31,
                                     27,
                                     11,
                                     4}};
const Vector<Extent> subMatrices = {
    {24, 2},   // Joint group 0
    {12, 2},   // Joint group 1
    {21, 2},   // Joint group 2
    {9, 2},    // Joint group 3
    {2, 2},    // Joint group 4
    {0, 0},    // Joint group 5
    {6, 2},    // Joint group 6
    {31, 10},  // Joint group 7
};
const Vector<dna::RotationRepresentation> jointRotationRepresentations = {
    // Joint group 0
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    // Joint group 1
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    // Joint group 2
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    // Joint group 3
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    // Joint group 4
    dna::RotationRepresentation::EulerAngles,
    // Joint group 5 - nothing
    // Joint group 6
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    // Joint group 7
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion,
    dna::RotationRepresentation::Quaternion};

}  // namespace unoptimized

namespace optimized {

const AlignedMatrix<float> floatValues = {
    {
        // Joint group 0
        /*
            x,          y,          z,          w,
            0.2745997f, 0.0796042f, 0.1995657f, 0.9372469f,  // q0, e0
            0.1398205f, 0.1331988f, 0.3497641f, 0.9167188f,  // q1, e0
            0.3030566f, 0.2961900f, 0.3502249f, 0.8353267f,  // q2, e0
            0.0804573f, 0.2924195f, 0.1373709f, 0.9429457f,  // q3, e0
            0.0454372f, 0.0416356f, 0.0454372f, 0.9970644f,  // q4, e0
            0.1451937f, 0.1114111f, 0.1451937f, 0.9723297f,  // q5, e0
            0.2520359f, 0.1605646f, 0.2520359f, 0.9204210f,  // q6, e0
            0.3597537f, 0.1872759f, 0.3597537f, 0.8402871f,  // q7, e0
            0.1995657f, 0.2308131f, 0.1687222f, 0.9372469f,  // q0, e1
            0.3497641f, 0.0214902f, 0.1919111f, 0.9167188f,  // q1, e1
            0.4164508f, 0.0783618f, 0.3502249f, 0.8353267f,  // q2, e1
            0.3032794f, 0.1113622f, 0.0804573f, 0.9429457f,  // q3, e1
            0.0940609f, 0.0789265f, 0.0940609f, 0.9879654f,  // q4, e1
            0.1981076f, 0.1387165f, 0.1981076f, 0.9498760f,  // q5, e1
            0.3061862f, 0.1767767f, 0.3061862f, 0.8838835f,  // q6, e1
            0.4119345f, 0.1920882f, 0.4119345f, 0.7897607f,  // q7, e1
        */
        0.2745997f, 0.1398205f, 0.3030566f, 0.0804573f, 0.0454372f, 0.1451937f, 0.2520359f, 0.3597537f,  // Xs[q0-q7]
        0.0796042f, 0.1331988f, 0.2961900f, 0.2924195f, 0.0416356f, 0.1114111f, 0.1605646f, 0.1872759f,  // Ys[q0-q7]
        0.1995657f, 0.3497641f, 0.3502249f, 0.1373709f, 0.0454372f, 0.1451937f, 0.2520359f, 0.3597537f,  // Zs[q0-q7]
        0.9372469f, 0.9167188f, 0.8353267f, 0.9429457f, 0.9970644f, 0.9723297f, 0.9204210f, 0.8402871f,  // Ws[q0-q7]
        0.1995657f, 0.3497641f, 0.4164508f, 0.3032794f, 0.0940609f, 0.1981076f, 0.3061862f, 0.4119345f,  // Xs[q0-q7]
        0.2308131f, 0.0214902f, 0.0783618f, 0.1113622f, 0.0789265f, 0.1387165f, 0.1767767f, 0.1920882f,  // Ys[q0-q7]
        0.1687222f, 0.1919111f, 0.3502249f, 0.0804573f, 0.0940609f, 0.1981076f, 0.3061862f, 0.4119345f,  // Zs[q0-q7]
        0.9372469f, 0.9167188f, 0.8353267f, 0.9429457f, 0.9879654f, 0.9498760f, 0.8838835f, 0.7897607f,  // Ws[q0-q7]
    },
    {
        // Joint group 1
        /*
            x,          y,          z,          w,
            0.2745997f, 0.0796042f, 0.1995657f, 0.9372469f,  // q0, e0
            0.1398205f, 0.1331988f, 0.3497641f, 0.9167188f,  // q1, e0
            0.3030566f, 0.2961900f, 0.3502249f, 0.8353267f,  // q2, e0
            0.0804573f, 0.2924195f, 0.1373709f, 0.9429457f,  // q3, e0
            0.1995657f, 0.2308131f, 0.1687222f, 0.9372469f,  // q0, e1
            0.3497641f, 0.0214902f, 0.1919111f, 0.9167188f,  // q1, e1
            0.4164508f, 0.0783618f, 0.3502249f, 0.8353267f,  // q2, e1
            0.3032794f, 0.1113622f, 0.0804573f, 0.9429457f,  // q3, e1
        */
        0.2745997f, 0.1398205f, 0.3030566f, 0.0804573f,  // Xs[q0-q3]
        0.0796042f, 0.1331988f, 0.2961900f, 0.2924195f,  // Ys[q0-q3]
        0.1995657f, 0.3497641f, 0.3502249f, 0.1373709f,  // Zs[q0-q3]
        0.9372469f, 0.9167188f, 0.8353267f, 0.9429457f,  // Ws[q0-q3]
        0.1995657f, 0.3497641f, 0.4164508f, 0.3032794f,  // Xs[q0-q3]
        0.2308131f, 0.0214902f, 0.0783618f, 0.1113622f,  // Ys[q0-q3]
        0.1687222f, 0.1919111f, 0.3502249f, 0.0804573f,  // Zs[q0-q3]
        0.9372469f, 0.9167188f, 0.8353267f, 0.9429457f,  // Ws[q0-q3]
    },
    {
        // Joint group 2
        /*
            x,          y,          z,          w,
            0.2745997f, 0.0796042f, 0.1995657f, 0.9372469f,  // q0, e0
            0.1398205f, 0.1331988f, 0.3497641f, 0.9167188f,  // q1, e0
            0.3030566f, 0.2961900f, 0.3502249f, 0.8353267f,  // q2, e0
            0.0804573f, 0.2924195f, 0.1373709f, 0.9429457f,  // q3, e0
            0.0454372f, 0.0416356f, 0.0454372f, 0.9970644f,  // q4, e0
            0.1451937f, 0.1114111f, 0.1451937f, 0.9723297f,  // q5, e0
            0.2520359f, 0.1605646f, 0.2520359f, 0.9204210f,  // q6, e0
            0.1995657f, 0.2308131f, 0.1687222f, 0.9372469f,  // q0, e1
            0.3497641f, 0.0214902f, 0.1919111f, 0.9167188f,  // q1, e1
            0.4164508f, 0.0783618f, 0.3502249f, 0.8353267f,  // q2, e1
            0.3032794f, 0.1113622f, 0.0804573f, 0.9429457f,  // q3, e1
            0.0940609f, 0.0789265f, 0.0940609f, 0.9879654f,  // q4, e1
            0.1981076f, 0.1387165f, 0.1981076f, 0.9498760f,  // q5, e1
            0.3061862f, 0.1767767f, 0.3061862f, 0.8838835f,  // q6, e1
        */
        0.2745997f, 0.1398205f, 0.3030566f, 0.0804573f, 0.0454372f, 0.1451937f, 0.2520359f, 0.0f,  // Xs[q0-q6]
        0.0796042f, 0.1331988f, 0.2961900f, 0.2924195f, 0.0416356f, 0.1114111f, 0.1605646f, 0.0f,  // Ys[q0-q6]
        0.1995657f, 0.3497641f, 0.3502249f, 0.1373709f, 0.0454372f, 0.1451937f, 0.2520359f, 0.0f,  // Zs[q0-q6]
        0.9372469f, 0.9167188f, 0.8353267f, 0.9429457f, 0.9970644f, 0.9723297f, 0.9204210f, 0.0f,  // Ws[q0-q6]
        0.1995657f, 0.3497641f, 0.4164508f, 0.3032794f, 0.0940609f, 0.1981076f, 0.3061862f, 0.0f,  // Xs[q0-q6]
        0.2308131f, 0.0214902f, 0.0783618f, 0.1113622f, 0.0789265f, 0.1387165f, 0.1767767f, 0.0f,  // Ys[q0-q6]
        0.1687222f, 0.1919111f, 0.3502249f, 0.0804573f, 0.0940609f, 0.1981076f, 0.3061862f, 0.0f,  // Zs[q0-q6]
        0.9372469f, 0.9167188f, 0.8353267f, 0.9429457f, 0.9879654f, 0.9498760f, 0.8838835f, 0.0f,  // Ws[q0-q6]
    },
    {
        // Joint group 3
        /*
            x,          y,          z,          w,
            0.2745997f, 0.0796042f, 0.1995657f, 0.9372469f,  // q0, e0
            0.1398205f, 0.1331988f, 0.3497641f, 0.9167188f,  // q1, e0
            0.3030566f, 0.2961900f, 0.3502249f, 0.8353267f,  // q2, e0
            0.1995657f, 0.2308131f, 0.1687222f, 0.9372469f,  // q0, e1
            0.3497641f, 0.0214902f, 0.1919111f, 0.9167188f,  // q1, e1
            0.4164508f, 0.0783618f, 0.3502249f, 0.8353267f,  // q2, e1
        */
        0.2745997f, 0.1398205f, 0.3030566f, 0.0f,  // Xs[q0-q2]
        0.0796042f, 0.1331988f, 0.2961900f, 0.0f,  // Ys[q0-q2]
        0.1995657f, 0.3497641f, 0.3502249f, 0.0f,  // Zs[q0-q2]
        0.9372469f, 0.9167188f, 0.8353267f, 0.0f,  // Ws[q0-q2]
        0.1995657f, 0.3497641f, 0.4164508f, 0.0f,  // Xs[q0-q2]
        0.2308131f, 0.0214902f, 0.0783618f, 0.0f,  // Ys[q0-q2]
        0.1687222f, 0.1919111f, 0.3502249f, 0.0f,  // Zs[q0-q2]
        0.9372469f, 0.9167188f, 0.8353267f, 0.0f,  // Ws[q0-q2]
    },
    {
        // Joint group 4
    },
    {
        // Joint group 5
    },
    {
        // Joint group 6
        /*
            x,          y,          z,          w,
            0.2745997f, 0.0796042f, 0.1995657f, 0.9372469f,  // q0, e0
            0.1398205f, 0.1331988f, 0.3497641f, 0.9167188f,  // q1, e0
            0.1995657f, 0.2308131f, 0.1687222f, 0.9372469f,  // q0, e1
            0.3497641f, 0.0214902f, 0.1919111f, 0.9167188f,  // q1, e1
        */
        0.2745997f, 0.1398205f, 0.0f, 0.0f,  // Xs[q0-q1]
        0.0796042f, 0.1331988f, 0.0f, 0.0f,  // Ys[q0-q1]
        0.1995657f, 0.3497641f, 0.0f, 0.0f,  // Zs[q0-q1]
        0.9372469f, 0.9167188f, 0.0f, 0.0f,  // Ws[q0-q1]
        0.1995657f, 0.3497641f, 0.0f, 0.0f,  // Xs[q0-q1]
        0.2308131f, 0.0214902f, 0.0f, 0.0f,  // Ys[q0-q1]
        0.1687222f, 0.1919111f, 0.0f, 0.0f,  // Zs[q0-q1]
        0.9372469f, 0.9167188f, 0.0f, 0.0f,  // Ws[q0-q1]
    },
    {
        // Joint group 7
        /*
           x,          y,          z,          w,
           0.0742712f, 0.0645629f, 0.0742712f, 0.9923705f,  // q0, e0
           0.0f,       0.0f,       0.0f,       1.0f,        // q1, e0
           0.0f,       0.0f,       0.0f,       1.0f,        // q2, e0
           0.0f,       0.0f,       0.0f,       1.0f,        // q3, e0
           0.0f,       0.0f,       0.0f,       1.0f,        // q4, e0
           0.0f,       0.0f,       0.0f,       1.0f,        // q5, e0
           0.0f,       0.0f,       0.0f,       1.0f,        // q6, e0
           0.0f,       0.0f,       0.0f,       1.0f,        // q7, e0
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q0, e1
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q1, e1
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q2, e1
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q3, e1
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q4, e1
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q5, e1
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q6, e1
           0.008802f,  0.0086497f, 0.008802f,  0.9998851f,  // q7, e1
           0.0454372f, 0.0416356f, 0.0454372f, 0.9970644f,  // q0, e2
           0.0454372f, 0.0416356f, 0.0454372f, 0.9970644f,  // q1, e2
           0.0454372f, 0.0416356f, 0.0454372f, 0.9970644f,  // q2, e2
           0.0435779f, 0.0435779f, 0.0019027f, 0.9980973f,  // q3, e2
           0.0435779f, 0.0435779f, 0.0019027f, 0.9980973f,  // q4, e2
           0.0435779f, 0.0435779f, 0.0019027f, 0.9980973f,  // q5, e2
           0.0435779f, 0.0435779f, 0.0019027f, 0.9980973f,  // q6, e2
           0.0435779f, 0.0435779f, 0.0019027f, 0.9980973f,  // q7, e2
           0.0360742f, 0.0336398f, 0.0360742f, 0.9981311f,  // q0, e3
           0.0360742f, 0.0336398f, 0.0360742f, 0.9981311f,  // q1, e3
           0.0360742f, 0.0336398f, 0.0360742f, 0.9981311f,  // q2, e3
           0.0f,       0.0f,       0.0f,       1.0f,        // q3, e3
           0.0f,       0.0f,       0.0f,       1.0f,        // q4, e3
           0.0f,       0.0f,       0.0f,       1.0f,        // q5, e3
           0.0f,       0.0f,       0.0f,       1.0f,        // q6, e3
           0.0f,       0.0f,       0.0f,       1.0f,        // q7, e3
           0.0f,       0.0f,       0.0f,       1.0f,        // q0, e4
           0.064541f,  0.057101f,  0.064541f,  0.9941873f,  // q1, e4
           0.064541f,  0.057101f,  0.064541f,  0.9941873f,  // q2, e4
           0.064541f,  0.057101f,  0.064541f,  0.9941873f,  // q3, e4
           0.064541f,  0.057101f,  0.064541f,  0.9941873f,  // q4, e4
           0.064541f,  0.057101f,  0.064541f,  0.9941873f,  // q5, e4
           0.064541f,  0.057101f,  0.064541f,  0.9941873f,  // q6, e4
           0.064541f,  0.057101f,  0.064541f,  0.9941873f,  // q7, e4
           0.0f,       0.0f,       0.0f,       1.0f,        // q0, e5
           0.0f,       0.0f,       0.0f,       1.0f,        // q1, e5
           0.0f,       0.0f,       0.0f,       1.0f,        // q2, e5
           0.0177516f, 0.0171426f, 0.0177516f, 0.9995378f,  // q3, e5
           0.0177516f, 0.0171426f, 0.0177516f, 0.9995378f,  // q4, e5
           0.0177516f, 0.0171426f, 0.0177516f, 0.9995378f,  // q5, e5
           0.0177516f, 0.0171426f, 0.0177516f, 0.9995378f,  // q6, e5
           0.0177516f, 0.0171426f, 0.0177516f, 0.9995378f,  // q7, e5
           0.0f,       0.0f,       0.0f,       1.0f,        // q0, e6
           0.0f,       0.0f,       0.0f,       1.0f,        // q1, e6
           0.0f,       0.0f,       0.0f,       1.0f,        // q2, e6
           0.0549279f, 0.0494573f, 0.0549279f, 0.9957509f,  // q3, e6
           0.0549279f, 0.0494573f, 0.0549279f, 0.9957509f,  // q4, e6
           0.0549279f, 0.0494573f, 0.0549279f, 0.9957509f,  // q5, e6
           0.0549279f, 0.0494573f, 0.0549279f, 0.9957509f,  // q6, e6
           0.0549279f, 0.0494573f, 0.0549279f, 0.9957509f,  // q7, e6
           0.0f,       0.0f,       0.0f,       1.0f,        // q0, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q1, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q2, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q3, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q4, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q5, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q6, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q7, e7
           0.0f,       0.0f,       0.0f,       1.0f,        // q8, e0
           0.0087262f, 0.0087262f, 0.0000762f, 0.9999238f,  // q8, e1
           0.0f,       0.0f,       0.0f,       1.0f,        // q8, e2
           0.0f,       0.0f,       0.0f,       1.0f,        // q8, e3
           0.0609347f, 0.0609347f, 0.0037269f, 0.9962731f,  // q8, e4
           0.0174497f, 0.0174497f, 0.0003046f, 0.9996954f,  // q8, e5
           0.0522642f, 0.0522642f, 0.0027391f, 0.9972609f,  // q8, e6
           0.026168f,  0.026168f,  0.0006852f, 0.9993148f,  // q8, e7
        */
        0.0742712f, 0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Xs[q0-q7]
        0.0645629f, 0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Ys[q0-q7]
        0.0742712f, 0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Zs[q0-q7]
        0.9923705f, 1.0f,       1.0f,       1.0f,       1.0f,       1.0f,       1.0f,       1.0f,        // Ws[q0-q7]
        0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,   // Xs[q0-q7]
        0.0086497f, 0.0086497f, 0.0086497f, 0.0086497f, 0.0086497f, 0.0086497f, 0.0086497f, 0.0086497f,  // Ys[q0-q7]
        0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,  0.008802f,   // Zs[q0-q7]
        0.9998851f, 0.9998851f, 0.9998851f, 0.9998851f, 0.9998851f, 0.9998851f, 0.9998851f, 0.9998851f,  // Ws[q0-q7]
        0.0454372f, 0.0454372f, 0.0454372f, 0.0435779f, 0.0435779f, 0.0435779f, 0.0435779f, 0.0435779f,  // Xs[q0-q7]
        0.0416356f, 0.0416356f, 0.0416356f, 0.0435779f, 0.0435779f, 0.0435779f, 0.0435779f, 0.0435779f,  // Ys[q0-q7]
        0.0454372f, 0.0454372f, 0.0454372f, 0.0019027f, 0.0019027f, 0.0019027f, 0.0019027f, 0.0019027f,  // Zs[q0-q7]
        0.9970644f, 0.9970644f, 0.9970644f, 0.9980973f, 0.9980973f, 0.9980973f, 0.9980973f, 0.9980973f,  // Ws[q0-q7]
        0.0360742f, 0.0360742f, 0.0360742f, 0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Xs[q0-q7]
        0.0336398f, 0.0336398f, 0.0336398f, 0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Ys[q0-q7]
        0.0360742f, 0.0360742f, 0.0360742f, 0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Zs[q0-q7]
        0.9981311f, 0.9981311f, 0.9981311f, 1.0f,       1.0f,       1.0f,       1.0f,       1.0f,        // Ws[q0-q7]
        0.0f,       0.064541f,  0.064541f,  0.064541f,  0.064541f,  0.064541f,  0.064541f,  0.064541f,   // Xs[q0-q7]
        0.0f,       0.057101f,  0.057101f,  0.057101f,  0.057101f,  0.057101f,  0.057101f,  0.057101f,   // Ys[q0-q7]
        0.0f,       0.064541f,  0.064541f,  0.064541f,  0.064541f,  0.064541f,  0.064541f,  0.064541f,   // Zs[q0-q7]
        1.0f,       0.9941873f, 0.9941873f, 0.9941873f, 0.9941873f, 0.9941873f, 0.9941873f, 0.9941873f,  // Ws[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0177516f, 0.0177516f, 0.0177516f, 0.0177516f, 0.0177516f,  // Xs[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0171426f, 0.0171426f, 0.0171426f, 0.0171426f, 0.0171426f,  // Ys[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0177516f, 0.0177516f, 0.0177516f, 0.0177516f, 0.0177516f,  // Zs[q0-q7]
        1.0f,       1.0f,       1.0f,       0.9995378f, 0.9995378f, 0.9995378f, 0.9995378f, 0.9995378f,  // Ws[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0549279f, 0.0549279f, 0.0549279f, 0.0549279f, 0.0549279f,  // Xs[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0494573f, 0.0494573f, 0.0494573f, 0.0494573f, 0.0494573f,  // Ys[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0549279f, 0.0549279f, 0.0549279f, 0.0549279f, 0.0549279f,  // Zs[q0-q7]
        1.0f,       1.0f,       1.0f,       0.9957509f, 0.9957509f, 0.9957509f, 0.9957509f, 0.9957509f,  // Ws[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Xs[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Ys[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,       0.0f,        // Zs[q0-q7]
        1.0f,       1.0f,       1.0f,       1.0f,       1.0f,       1.0f,       1.0f,       1.0f,        // Ws[q0-q7]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        1.0f,       0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
        0.0087262f, 0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.0087262f, 0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0000762f, 0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        0.9999238f, 0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        1.0f,       0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0f,       0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        1.0f,       0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
        0.0609347f, 0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.0609347f, 0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0037269f, 0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        0.9962731f, 0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
        0.0174497f, 0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.0174497f, 0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0003046f, 0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        0.9996954f, 0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
        0.0522642f, 0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.0522642f, 0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0027391f, 0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        0.9972609f, 0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
        0.026168f,  0.0f,       0.0f,       0.0f,                                                        // Xs[q0]
        0.026168f,  0.0f,       0.0f,       0.0f,                                                        // Ys[q0]
        0.0006852f, 0.0f,       0.0f,       0.0f,                                                        // Zs[q0]
        0.9993148f, 0.0f,       0.0f,       0.0f,                                                        // Ws[q0]
    }};

const AlignedMatrix<std::uint16_t> halfFloatValues = {
    {// Joint group 0
     13413, 12409, 13529, 11558, 10705, 12453, 13320, 13762, 11544, 12355, 13501, 13486, 10580, 12065, 12579, 12798,
     12899, 13721, 13723, 12389, 10705, 12453, 13320, 13762, 15231, 15189, 15023, 15243, 15354, 15303, 15197, 15033,
     12899, 13721, 13994, 13530, 11781, 12887, 13542, 13975, 13155, 9600,  11524, 12065, 11533, 12400, 12712, 12838,
     12646, 12836, 13723, 11558, 11781, 12887, 13542, 13975, 15231, 15189, 15023, 15243, 15335, 15257, 15122, 14929},
    {// Joint group 1
     13413, 12409, 13529, 11558, 11544, 12355, 13501, 13486, 12899, 13721, 13723, 12389, 15231, 15189, 15023, 15243,
     12899, 13721, 13994, 13530, 13155, 9600,  11524, 12065, 12646, 12836, 13723, 11558, 15231, 15189, 15023, 15243},
    {// Joint group 2
     13413, 12409, 13529, 11558, 10705, 12453, 13320, 0, 11544, 12355, 13501, 13486, 10580, 12065, 12579, 0,
     12899, 13721, 13723, 12389, 10705, 12453, 13320, 0, 15231, 15189, 15023, 15243, 15354, 15303, 15197, 0,
     12899, 13721, 13994, 13530, 11781, 12887, 13542, 0, 13155, 9600,  11524, 12065, 11533, 12400, 12712, 0,
     12646, 12836, 13723, 11558, 11781, 12887, 13542, 0, 15231, 15189, 15023, 15243, 15335, 15257, 15122, 0},
    {// Joint group 3
     13413, 12409, 13529, 0, 11544, 12355, 13501, 0, 12899, 13721, 13723, 0, 15231, 15189, 15023, 0,
     12899, 13721, 13994, 0, 13155, 9600,  11524, 0, 12646, 12836, 13723, 0, 15231, 15189, 15023, 0},
    {
        // Joint group 4
    },
    {
        // Joint group 5
    },
    {// Joint group 6
     13413, 12409, 0, 0, 11544, 12355, 0, 0, 12899, 13721, 0, 0, 15231, 15189, 0, 0,
     12899, 13721, 0, 0, 13155, 9600,  0, 0, 12646, 12836, 0, 0, 15231, 15189, 0, 0},
    {
        // Joint group 7
        11457, 0,     0,     0,     0,     0,     0,     0,     11298, 0,     0,     0,     0,     0,     0,     0,     11457,
        0,     0,     0,     0,     0,     0,     0,     15344, 15360, 15360, 15360, 15360, 15360, 15360, 15360, 8322,  8322,
        8322,  8322,  8322,  8322,  8322,  8322,  8302,  8302,  8302,  8302,  8302,  8302,  8302,  8302,  8322,  8322,  8322,
        8322,  8322,  8322,  8322,  8322,  15360, 15360, 15360, 15360, 15360, 15360, 15360, 15360, 10705, 10705, 10705, 10644,
        10644, 10644, 10644, 10644, 10580, 10580, 10580, 10644, 10644, 10644, 10644, 10644, 10705, 10705, 10705, 6091,  6091,
        6091,  6091,  6091,  15354, 15354, 15354, 15356, 15356, 15356, 15356, 15356, 10398, 10398, 10398, 0,     0,     0,
        0,     0,     10318, 10318, 10318, 0,     0,     0,     0,     0,     10398, 10398, 10398, 0,     0,     0,     0,
        0,     15356, 15356, 15356, 15360, 15360, 15360, 15360, 15360, 0,     11297, 11297, 11297, 11297, 11297, 11297, 11297,
        0,     11087, 11087, 11087, 11087, 11087, 11087, 11087, 0,     11297, 11297, 11297, 11297, 11297, 11297, 11297, 15360,
        15348, 15348, 15348, 15348, 15348, 15348, 15348, 0,     0,     0,     9355,  9355,  9355,  9355,  9355,  0,     0,
        0,     9315,  9315,  9315,  9315,  9315,  0,     0,     0,     9355,  9355,  9355,  9355,  9355,  15360, 15360, 15360,
        15359, 15359, 15359, 15359, 15359, 0,     0,     0,     11016, 11016, 11016, 11016, 11016, 0,     0,     0,     10837,
        10837, 10837, 10837, 10837, 0,     0,     0,     11016, 11016, 11016, 11016, 11016, 15360, 15360, 15360, 15351, 15351,
        15351, 15351, 15351, 0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,
        0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     15360, 15360, 15360, 15360, 15360, 15360, 15360,
        15360, 0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     15360, 0,     0,     0,
        8312,  0,     0,     0,     8312,  0,     0,     0,     1278,  0,     0,     0,     15360, 0,     0,     0,     0,
        0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     15360, 0,     0,     0,     0,     0,
        0,     0,     0,     0,     0,     0,     0,     0,     0,     0,     15360, 0,     0,     0,     11213, 0,     0,
        0,     11213, 0,     0,     0,     7074,  0,     0,     0,     15352, 0,     0,     0,     9336,  0,     0,     0,
        9336,  0,     0,     0,     3326,  0,     0,     0,     15359, 0,     0,     0,     10929, 0,     0,     0,     10929,
        0,     0,     0,     6556,  0,     0,     0,     15354, 0,     0,     0,     9907,  0,     0,     0,     9907,  0,
        0,     0,     4509,  0,     0,     0,     15359, 0,     0,     0,
    }};

const Vector<Extent> subMatrices = {
    {32, 2},  // Joint group 0
    {16, 2},  // Joint group 1
    {32, 2},  // Joint group 2
    {16, 2},  // Joint group 3
    {0, 0},   // Joint group 4
    {0, 0},   // Joint group 5
    {16, 2},  // Joint group 6
    {36, 8},  // Joint group 7
};

const Matrix<std::uint16_t> inputIndices = {{// Joint group 0
                                             8,
                                             6},
                                            {// Joint group 1
                                             8,
                                             6},
                                            {// Joint group 2
                                             8,
                                             6},
                                            {// Joint group 3
                                             8,
                                             6},
                                            {
                                                // Joint group 4
                                            },
                                            {
                                                // Joint group 5
                                            },
                                            {// Joint group 6
                                             8,
                                             6},
                                            {// Joint group 7
                                             9,
                                             2,
                                             6,
                                             5,
                                             8,
                                             3,
                                             7,
                                             1}};

const Matrix<Vector<std::uint16_t>> outputIndices = {
    {
        // Quaternion outputs
        {// Joint group 0
         3,  4,  5,  6,  13, 14, 15, 16, 23, 24, 25, 26, 33, 34, 35, 36,
         43, 44, 45, 46, 53, 54, 55, 56, 63, 64, 65, 66, 73, 74, 75, 76},
        {// Joint group 1
         83,
         84,
         85,
         86,
         93,
         94,
         95,
         96,
         103,
         104,
         105,
         106,
         113,
         114,
         115,
         116},
        {// Joint group 2
         123, 124, 125, 126, 133, 134, 135, 136, 143, 144, 145, 146, 153, 154,
         155, 156, 163, 164, 165, 166, 173, 174, 175, 176, 183, 184, 185, 186},
        {// Joint group 3
         193,
         194,
         195,
         196,
         203,
         204,
         205,
         206,
         213,
         214,
         215,
         216},
        {
            // Joint group 4
        },
        {
            // Joint group 5
        },
        {// Joint group 6
         243,
         244,
         245,
         246,
         233,
         234,
         235,
         236},
        {// Joint group 7
         253, 254, 255, 256, 263, 264, 265, 266, 273, 274, 275, 276, 283, 284, 285, 286, 293, 294,
         295, 296, 303, 304, 305, 306, 313, 314, 315, 316, 323, 324, 325, 326, 333, 334, 335, 336},
    },
    {
        // Euler-angle outputs
        {// Joint group 0
         3, 4, 5, 0, 12, 13, 14, 0, 21, 22, 23, 0, 30, 31, 32, 0, 39, 40, 41, 0, 48, 49, 50, 0, 57, 58, 59, 0, 66, 67, 68, 0},
        {// Joint group 1
         75,
         76,
         77,
         0,
         84,
         85,
         86,
         0,
         93,
         94,
         95,
         0,
         102,
         103,
         104,
         0},
        {// Joint group 2
         111, 112, 113, 0,   120, 121, 122, 0,   129, 130, 131, 0,   138, 139,
         140, 0,   147, 148, 149, 0,   156, 157, 158, 0,   165, 166, 167, 0},
        {// Joint group 3
         174,
         175,
         176,
         0,
         183,
         184,
         185,
         0,
         192,
         193,
         194,
         0},
        {
            // Joint group 4
        },
        {
            // Joint group 5
        },
        {// Joint group 6
         219,
         220,
         221,
         0,
         210,
         211,
         212,
         0},
        {// Joint group 7
         228, 229, 230, 0,   237, 238, 239, 0,   246, 247, 248, 0,   255, 256, 257, 0,   264, 265,
         266, 0,   273, 274, 275, 0,   282, 283, 284, 0,   291, 292, 293, 0,   300, 301, 302, 0},
    }};

const Matrix<LODRegion> lodRegions = {
    // {unaligned, aligned to last block-8, aligned to second last block-8}
    {
        // Joint group 0
        {{2, 0, 0}, {32, 32, 32}},  // LOD-0
        {{2, 0, 0}, {16, 32, 0}},   // LOD-1
        {{2, 0, 0}, {16, 32, 0}},   // LOD-2
        {{0, 0, 0}, {0, 0, 0}},     // LOD-3
    },
    {
        // Joint group 1
        {{2, 0, 0}, {16, 0, 0}},  // LOD-0
        {{2, 0, 0}, {8, 0, 0}},   // LOD-1
        {{2, 0, 0}, {8, 0, 0}},   // LOD-2
        {{0, 0, 0}, {0, 0, 0}},   // LOD-3
    },
    {
        // Joint group 2
        {{2, 0, 0}, {28, 32, 0}},  // LOD-0
        {{2, 0, 0}, {12, 32, 0}},  // LOD-1
        {{2, 0, 0}, {12, 32, 0}},  // LOD-2
        {{0, 0, 0}, {0, 0, 0}},    // LOD-3
    },
    {
        // Joint group 3
        {{2, 0, 0}, {12, 0, 0}},  // LOD-0
        {{2, 0, 0}, {4, 0, 0}},   // LOD-1
        {{2, 0, 0}, {4, 0, 0}},   // LOD-2
        {{0, 0, 0}, {0, 0, 0}},   // LOD-3
    },
    {
        // Joint group 4
        {{0, 0, 0}, {0, 0, 0}},  // LOD-0
        {{0, 0, 0}, {0, 0, 0}},  // LOD-1
        {{0, 0, 0}, {0, 0, 0}},  // LOD-2
        {{0, 0, 0}, {0, 0, 0}},  // LOD-3
    },
    {
        // Joint group 5
        {{0, 0, 0}, {0, 0, 0}},  // LOD-0
        {{0, 0, 0}, {0, 0, 0}},  // LOD-1
        {{0, 0, 0}, {0, 0, 0}},  // LOD-2
        {{0, 0, 0}, {0, 0, 0}},  // LOD-3
    },
    {
        // Joint group 6
        {{2, 0, 0}, {8, 0, 0}},  // LOD-0
        {{2, 0, 0}, {4, 0, 0}},  // LOD-1
        {{0, 0, 0}, {0, 0, 0}},  // LOD-2
        {{0, 0, 0}, {0, 0, 0}},  // LOD-3
    },
    {
        // Joint group 7
        {{8, 8, 8}, {36, 32, 32}},  // LOD-0
        {{7, 4, 0}, {32, 32, 32}},  // LOD-1
        {{5, 4, 0}, {12, 32, 0}},   // LOD-2
        {{4, 4, 0}, {4, 32, 0}},    // LOD-3
    },
};

}  // namespace optimized

namespace input {

// Calculation input values
const rl4::Vector<float> values = {0.1f, 0.2f, 0.3f, 0.4f, 0.0f, 0.6f, 0.7f, 0.8f, 0.9f, 0.0f, 0.15f, 0.125f, 0.35f};

}  // namespace input

namespace output {

// Expected output results for each LOD
const rl4::Vector<rl4::Matrix<float>> valuesPerLODPerConfig = {{ // Quaternion outputs
                                                                {// LOD-0
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.270364f,
                                                                 0.309758f,
                                                                 0.226229f,
                                                                 0.883048f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.106307f,
                                                                 0.0925174f,
                                                                 0.106724f,
                                                                 0.984251f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.263534f,
                                                                 0.193966f,
                                                                 0.265999f,
                                                                 0.906737f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.417713f,
                                                                 0.255751f,
                                                                 0.42371f,
                                                                 0.761957f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.550442f,
                                                                 0.275108f,
                                                                 0.561209f,
                                                                 0.553511f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.270364f,
                                                                 0.309758f,
                                                                 0.226229f,
                                                                 0.883048f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.270364f,
                                                                 0.309758f,
                                                                 0.226229f,
                                                                 0.883048f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.106307f,
                                                                 0.0925174f,
                                                                 0.106724f,
                                                                 0.984251f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.263534f,
                                                                 0.193966f,
                                                                 0.265999f,
                                                                 0.906737f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.417713f,
                                                                 0.255751f,
                                                                 0.42371f,
                                                                 0.761957f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0560275689f,
                                                                 0.0518481769f,
                                                                 0.0560221002f,
                                                                 0.994890451f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.113435827f,
                                                                 0.102763742f,
                                                                 0.113693371f,
                                                                 0.981072128f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.113435827f,
                                                                 0.102763742f,
                                                                 0.113693371f,
                                                                 0.981072128f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.111141264f,
                                                                 0.111082502f,
                                                                 0.00580846099f,
                                                                 0.986989498f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f},
                                                                {// LOD-1
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.270364f,
                                                                 0.309758f,
                                                                 0.226229f,
                                                                 0.883048f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0560275689f,
                                                                 0.0518481769f,
                                                                 0.0560221002f,
                                                                 0.994890451f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.113435827f,
                                                                 0.102763742f,
                                                                 0.113693371f,
                                                                 0.981072128f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.113435827f,
                                                                 0.102763742f,
                                                                 0.113693371f,
                                                                 0.981072128f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.138010800f,
                                                                 0.132913858f,
                                                                 0.112703480f,
                                                                 0.974272072f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f},
                                                                {// LOD-2
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.270364f,
                                                                 0.309758f,
                                                                 0.226229f,
                                                                 0.883048f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.342184f,
                                                                 0.068351f,
                                                                 0.459671f,
                                                                 0.816664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.464608f,
                                                                 0.27099f,
                                                                 0.579172f,
                                                                 0.612588f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.396116f,
                                                                 0.229363f,
                                                                 0.258099f,
                                                                 0.850805f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0560275689f,
                                                                 0.0518481769f,
                                                                 0.0560221002f,
                                                                 0.994890451f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.113435827f,
                                                                 0.102763742f,
                                                                 0.113693371f,
                                                                 0.981072128f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.113435827f,
                                                                 0.102763742f,
                                                                 0.113693371f,
                                                                 0.981072128f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f},
                                                                {// LOD-3
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0560275689f,
                                                                 0.0518481769f,
                                                                 0.0560221002f,
                                                                 0.994890451f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f}},
                                                               { // Euler-angle outputs
                                                                {// LOD-0
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.471332f,
                                                                 0.733388f,
                                                                 0.317664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.194852f,
                                                                 0.206271f,
                                                                 0.195788f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.444944f,
                                                                 0.514331f,
                                                                 0.45186f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.679014f,
                                                                 0.838617f,
                                                                 0.702706f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.891595f,
                                                                 1.17419f,
                                                                 0.968903f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.471332f,
                                                                 0.733388f,
                                                                 0.317664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.471332f,
                                                                 0.733388f,
                                                                 0.317664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.194852f,
                                                                 0.206271f,
                                                                 0.195788f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.444944f,
                                                                 0.514331f,
                                                                 0.45186f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.679014f,
                                                                 0.838617f,
                                                                 0.702706f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.106514752f,
                                                                 0.109663844f,
                                                                 0.106503174f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.206037536f,
                                                                 0.229438782f,
                                                                 0.206623316f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.206037536f,
                                                                 0.229438782f,
                                                                 0.206623316f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.225527078f,
                                                                 0.222394347f,
                                                                 -0.0135606825f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f},
                                                                {// LOD-1
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.471332f,
                                                                 0.733388f,
                                                                 0.317664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.106514752f,
                                                                 0.109663844f,
                                                                 0.106503174f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.206037536f,
                                                                 0.229438782f,
                                                                 0.206623316f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.206037536f,
                                                                 0.229438782f,
                                                                 0.206623316f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.252397001f,
                                                                 0.294328332f,
                                                                 0.192341432f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f},
                                                                {// LOD-2
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.471332f,
                                                                 0.733388f,
                                                                 0.317664f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.580409f,
                                                                 0.440315f,
                                                                 0.891871f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.544738f,
                                                                 1.05558f,
                                                                 1.19192f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.763116f,
                                                                 0.636969f,
                                                                 0.326027f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.106514752f,
                                                                 0.109663844f,
                                                                 0.106503174f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.206037536f,
                                                                 0.229438782f,
                                                                 0.206623316f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.206037536f,
                                                                 0.229438782f,
                                                                 0.206623316f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f},
                                                                {// LOD-3
                                                                 // Joint group 0 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 1 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 2 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 3 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 4 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 5 outputs
                                                                 // Joint group 6 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 // Joint group 7 outputs
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.106514752f,
                                                                 0.109663844f,
                                                                 0.106503174f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f,
                                                                 0.0f}}};

}  // namespace output

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

QuaternionReader::~QuaternionReader() = default;

}  // namespace qs

}  // namespace rltests
