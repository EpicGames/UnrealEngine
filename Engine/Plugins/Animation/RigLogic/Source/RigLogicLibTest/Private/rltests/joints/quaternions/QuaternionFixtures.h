// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/dna/FakeReader.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/quaternions/JointGroup.h"
#include "riglogic/types/Extent.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rltests {

namespace qs {

using namespace rl4;

namespace unoptimized {

extern const tdm::rot_sign rotationSigns;
extern const std::uint16_t lodCount;
extern const Extent dimensions;
extern const Matrix<float> values;
extern const Matrix<std::uint16_t> inputIndices;
extern const Matrix<std::uint16_t> outputIndices;
extern const Matrix<std::uint16_t> lods;
extern const Vector<Extent> subMatrices;
extern const Vector<dna::RotationRepresentation> jointRotationRepresentations;

}  // namespace unoptimized

namespace optimized {

extern const AlignedMatrix<float> floatValues;
extern const AlignedMatrix<std::uint16_t> halfFloatValues;
extern const Vector<Extent> subMatrices;
extern const Matrix<std::uint16_t> inputIndices;
extern const Matrix<Vector<std::uint16_t>> outputIndices;
extern const Matrix<LODRegion> lodRegions;

template<typename TValue>
struct Values;

template<>
struct Values<float> {
    static const AlignedMatrix<float>& get() {
        return floatValues;
    }
};

template<>
struct Values<std::uint16_t> {
    static const AlignedMatrix<std::uint16_t>& get() {
        return halfFloatValues;
    }
};

}  // namespace optimized

namespace input {

// Calculation input values
extern const Vector<float> values;

}  // namespace input

namespace output {

// Calculation output values
extern const Vector<Matrix<float>> valuesPerLODPerConfig;

}  // namespace output

class QuaternionReader : public dna::FakeReader {
public:
    ~QuaternionReader();

    dna::RotationSign getRotationSign() const override {
        return {dna::RotationDirection::positive, dna::RotationDirection::positive, dna::RotationDirection::positive};
    }

    dna::RotationSequence getRotationSequence() const override {
        return dna::RotationSequence::zyx;
    }

    std::uint16_t getLODCount() const override {
        return unoptimized::lodCount;
    }

    std::uint16_t getJointRowCount() const override {
        return static_cast<std::uint16_t>(unoptimized::dimensions.rows);
    }

    std::uint16_t getJointColumnCount() const override {
        return static_cast<std::uint16_t>(unoptimized::dimensions.cols);
    }

    std::uint16_t getJointGroupCount() const override {
        return static_cast<std::uint16_t>(unoptimized::subMatrices.size());
    }

    ConstArrayView<std::uint16_t> getJointGroupLODs(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{unoptimized::lods[jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupInputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{unoptimized::inputIndices[jointGroupIndex]};
    }

    ConstArrayView<std::uint16_t> getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<std::uint16_t>{unoptimized::outputIndices[jointGroupIndex]};
    }

    ConstArrayView<float> getJointGroupValues(std::uint16_t jointGroupIndex) const override {
        return ConstArrayView<float>{unoptimized::values[jointGroupIndex]};
    }

    dna::RotationRepresentation getJointRotationRepresentation(std::uint16_t jointIndex) const override {
        return unoptimized::jointRotationRepresentations[jointIndex];
    }
};

}  // namespace qs

}  // namespace rltests
