// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/controls/ControlFixtures.h"
#include "rltests/joints/Helpers.h"
#include "rltests/joints/quaternions/QuaternionFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/quaternions/QuaternionJointsEvaluator.h"
#include "riglogic/joints/cpu/quaternions/RotationAdapters.h"
#include "riglogic/system/simd/Detect.h"
#include "riglogic/system/simd/SIMD.h"

#include <tuple>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace {

template<typename TTestTypes>
class QuaternionEvaluatorTest : public ::testing::Test {
protected:
    void SetUp() override {
        QuaternionEvaluatorTest::SetUpImpl();
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value != 0ul, void>::type SetUpImpl() {
        using T = typename std::tuple_element<0, TestTypes>::type;
        using TF256 = typename std::tuple_element<1, TestTypes>::type;
        using TF128 = typename std::tuple_element<2, TestTypes>::type;
        using TRotationAdapter = typename std::tuple_element<3, TestTypes>::type;
        using CalculationStrategy = rl4::VectorizedJointGroupQuaternionCalculationStrategy<T, TF256, TF128, TRotationAdapter>;

        rotationSelectorIndex = RotationOutputTypeSelector<TRotationAdapter>::value();
        rotationType = RotationOutputTypeSelector<TRotationAdapter>::rotation();
        const auto& values = rltests::qs::optimized::Values<T>::get();
        rl4::Vector<rl4::JointGroup> jointGroups{values.size(), rl4::JointGroup{&memRes}, &memRes};
        for (std::size_t jgi = {}; jgi < jointGroups.size(); ++jgi) {
            auto& jointGroup = jointGroups[jgi];
            jointGroup.values.resize<T>(values[jgi].size());
            extd::copy(rl4::ConstArrayView<T>(values[jgi]),
                       rl4::ArrayView<T>(jointGroup.values.data<T>(), jointGroup.values.size<T>()));
            jointGroup.inputIndices = rltests::qs::optimized::inputIndices[jgi];
            jointGroup.outputIndices = rltests::qs::optimized::outputIndices[rotationSelectorIndex][jgi];
            jointGroup.lods = rltests::qs::optimized::lodRegions[jgi];
            jointGroup.colCount = rltests::qs::optimized::subMatrices[jgi].cols;
            jointGroup.rowCount = rltests::qs::optimized::subMatrices[jgi].rows;
        }
        auto factory = rl4::UniqueInstance<rl4::QuaternionJointsEvaluator, rl4::JointsEvaluator>::with(&memRes);
        auto strategy =
            rl4::UniqueInstance<CalculationStrategy, rl4::JointGroupQuaternionCalculationStrategy>::with(&memRes).create(
                TRotationAdapter{rltests::qs::unoptimized::rotationSigns});
        evaluator = factory.create(std::move(strategy), std::move(jointGroups), nullptr, &memRes);
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value == 0ul, void>::type SetUpImpl() {
        GTEST_SKIP();
    }

protected:
    pma::AlignedMemoryResource memRes;
    rltests::qs::QuaternionReader reader;
    rl4::JointsEvaluator::Pointer evaluator;
    std::size_t rotationSelectorIndex;
    rl4::RotationType rotationType;
};

}  // namespace

#if defined(RL_BUILD_WITH_AVX) && defined(RL_BUILD_WITH_SSE)
using QuaternionEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>
    #else
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               trimd::scalar::F128,
               rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_AVX)
using QuaternionEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>
    #else
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               trimd::scalar::F128,
               rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_SSE)
using QuaternionEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>
    #else
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               trimd::scalar::F128,
               rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_NEON)
using QuaternionEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<StorageValueType,
               trimd::neon::F256,
               trimd::neon::F128,
               rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::neon::F256, trimd::neon::F128, rl4::PassthroughAdapter>
    #else
    std::tuple<StorageValueType,
               trimd::neon::F256,
               trimd::neon::F128,
               rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::neon::F256, trimd::neon::F128, rl4::PassthroughAdapter>,
    std::tuple<StorageValueType,
               trimd::scalar::F256,
               trimd::scalar::F128,
               rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#else
    #ifndef RL_BUILD_WITH_HALF_FLOATS
using QuaternionEvaluatorTypeList =
    ::testing::Types<std::tuple<StorageValueType,
                                trimd::scalar::F256,
                                trimd::scalar::F128,
                                rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
                     std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>>;
    #else
using QuaternionEvaluatorTypeList = ::testing::Types<std::tuple<>>;
    #endif  // RL_BUILD_WITH_HALF_FLOATS
#endif

TYPED_TEST_SUITE(QuaternionEvaluatorTest, QuaternionEvaluatorTypeList, );

TYPED_TEST(QuaternionEvaluatorTest, EvaluatePerLOD) {
    const auto jointAttrCount =
        static_cast<std::uint16_t>(rltests::qs::output::valuesPerLODPerConfig[this->rotationSelectorIndex][0].size());
    rl4::CPUJointsOutputInstance outputInstance{jointAttrCount,
                                                rl4::TranslationType::Vector,
                                                this->rotationType,
                                                rl4::ScaleType::Vector,
                                                &this->memRes};
    auto outputBuffer = outputInstance.getOutputBuffer();
    auto inputInstanceFactory =
        ControlsFactory::getInstanceFactory(0, static_cast<std::uint16_t>(rltests::qs::input::values.size()), 0, 0, 0);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &this->memRes);
    auto inputBuffer = inputInstance->getInputBuffer();
    std::copy(rltests::qs::input::values.begin(), rltests::qs::input::values.end(), inputBuffer.begin());

    for (std::uint16_t lod = 0u; lod < rltests::qs::unoptimized::lodCount; ++lod) {
        std::fill(outputBuffer.begin(), outputBuffer.end(), 0.0f);
        this->evaluator->calculate(inputInstance.get(), &outputInstance, lod);
        const auto& expected = rltests::qs::output::valuesPerLODPerConfig[this->rotationSelectorIndex][lod];
#ifdef RL_BUILD_WITH_HALF_FLOATS
        static constexpr float threshold = 0.05f;
#else
        static constexpr float threshold = 0.002f;
#endif  // RL_BUILD_WITH_HALF_FLOATS
        ASSERT_ELEMENTS_NEAR(outputBuffer, expected, expected.size(), threshold);
    }
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
