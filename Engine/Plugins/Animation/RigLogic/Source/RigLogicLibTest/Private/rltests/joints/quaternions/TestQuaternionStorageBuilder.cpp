// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/joints/Helpers.h"
#include "rltests/joints/quaternions/QuaternionFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/quaternions/QuaternionJointsBuilder.h"
#include "riglogic/joints/cpu/quaternions/QuaternionJointsEvaluator.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Detect.h"

#include <tuple>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace rl4 {

template<typename T, typename TF256, typename TF128>
struct QuaternionJointGroupVerifier {

    void operator()(const Vector<JointGroup>& jointGroups, rl4::RotationType rotationType) {
        ASSERT_EQ(jointGroups.size(), rltests::qs::unoptimized::subMatrices.size());
        const auto& values = rltests::qs::optimized::Values<T>::get();
        const auto& inputIndices = rltests::qs::optimized::inputIndices;
        const auto rotationSelectorIndex = (rotationType == RotationType::EulerAngles) ? 1ul : 0ul;
        const auto& outputIndices = rltests::qs::optimized::outputIndices[rotationSelectorIndex];
        const auto& lods = rltests::qs::optimized::lodRegions;
        for (std::size_t jgi = {}; jgi < jointGroups.size(); ++jgi) {
            const auto& jointGroup = jointGroups[jgi];
            ASSERT_ELEMENTS_NEAR(jointGroup.values.data<T>(), values[jgi], values[jgi].size(), 0.0002f);
            ASSERT_ELEMENTS_EQ(jointGroup.inputIndices, inputIndices[jgi], inputIndices[jgi].size());
            ASSERT_ELEMENTS_EQ(jointGroup.outputIndices, outputIndices[jgi], outputIndices[jgi].size());
            ASSERT_EQ(jointGroup.lods.size(), lods[jgi].size());
            for (std::size_t lod = {}; lod < lods[jgi].size(); ++lod) {
                const auto& resultLOD = jointGroup.lods[lod];
                const auto& expectedLOD = lods[jgi][lod];
                ASSERT_EQ(resultLOD.inputLODs.size, expectedLOD.inputLODs.size);
                ASSERT_EQ(resultLOD.outputLODs.size, expectedLOD.outputLODs.size);
                ASSERT_EQ(resultLOD.outputLODs.sizePaddedToLastFullBlock, expectedLOD.outputLODs.sizePaddedToLastFullBlock);
                ASSERT_EQ(resultLOD.outputLODs.sizePaddedToSecondLastFullBlock,
                          expectedLOD.outputLODs.sizePaddedToSecondLastFullBlock);
            }
        }
    }
};

struct QuaternionJointsEvaluator::Accessor {

    static void assertRawDataEqual(const QuaternionJointsEvaluator& result,
                                   rl4::RotationType rotationType,
                                   const Configuration& config) {
        RuntimeTemplateInstantiator rti{&config};
        rti.invoke<QuaternionJointGroupVerifier, void>(result.jointGroups, rotationType);
    }
};

}  // namespace rl4

namespace {

struct QuaternionJointTestParam {
    rl4::CalculationType calculationType;
    rl4::RotationType rotationType;
};

class QuaternionJointStorageBuilderTest : public ::testing::TestWithParam<QuaternionJointTestParam> {
protected:
    pma::AlignedMemoryResource memRes;
    rltests::qs::QuaternionReader reader;
};

}  // namespace

TEST_P(QuaternionJointStorageBuilderTest, LayoutOptimization) {
    const auto params = GetParam();
    rl4::Configuration config{};
    // Earlier ZYX was XYZ, and XYZ is the default, so now that they are changed, it must be explicitly passed here.
    config.rotationType = params.rotationType;
    config.calculationType = params.calculationType;
    auto meta = rl4::RigMetadata::create(config, &reader, &memRes, rl4::InitializationMethod::Create);
    rl4::QuaternionJointsBuilder builder(config, meta.get(), &memRes);

    rl4::JointBehaviorFilter filter{&reader, &memRes};
    filter.include(dna::RotationRepresentation::Quaternion);

    builder.computeStorageRequirements(filter);
    builder.allocateStorage(filter);
    builder.fillStorage(filter);
    auto joints = builder.build();

    rl4::QuaternionJointsEvaluator::Accessor::assertRawDataEqual(*static_cast<rl4::QuaternionJointsEvaluator*>(joints.get()),
                                                                 params.rotationType,
                                                                 config);
}

#ifdef RL_BUILD_WITH_ZYX_ROTATION_ORDER
INSTANTIATE_TEST_SUITE_P(QuaternionJointStorageBuilderTestSuite,
                         QuaternionJointStorageBuilderTest,
                         ::testing::Values(QuaternionJointTestParam{rl4::CalculationType::AVX, rl4::RotationType::EulerAngles},
                                           QuaternionJointTestParam{rl4::CalculationType::AVX, rl4::RotationType::Quaternions},
                                           QuaternionJointTestParam{rl4::CalculationType::SSE, rl4::RotationType::EulerAngles},
                                           QuaternionJointTestParam{rl4::CalculationType::SSE, rl4::RotationType::Quaternions},
                                           QuaternionJointTestParam{rl4::CalculationType::NEON, rl4::RotationType::EulerAngles},
                                           QuaternionJointTestParam{rl4::CalculationType::NEON, rl4::RotationType::Quaternions},
                                           QuaternionJointTestParam{rl4::CalculationType::Scalar, rl4::RotationType::EulerAngles},
                                           QuaternionJointTestParam{rl4::CalculationType::Scalar,
                                                                    rl4::RotationType::Quaternions}));
#else
INSTANTIATE_TEST_SUITE_P(QuaternionJointStorageBuilderTestSuite,
                         QuaternionJointStorageBuilderTest,
                         ::testing::Values(QuaternionJointTestParam{rl4::CalculationType::AVX, rl4::RotationType::Quaternions},
                                           QuaternionJointTestParam{rl4::CalculationType::SSE, rl4::RotationType::Quaternions},
                                           QuaternionJointTestParam{rl4::CalculationType::NEON, rl4::RotationType::Quaternions},
                                           QuaternionJointTestParam{rl4::CalculationType::Scalar,
                                                                    rl4::RotationType::Quaternions}));
#endif

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
