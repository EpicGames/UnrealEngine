// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/controls/ControlFixtures.h"
#include "rltests/joints/Helpers.h"
#include "rltests/joints/twistswing/TwistSwingFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/quaternions/RotationAdapters.h"
#include "riglogic/joints/cpu/twistswing/TwistSwingJointsEvaluator.h"
#include "riglogic/system/simd/Detect.h"
#include "riglogic/system/simd/SIMD.h"

#include <tuple>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace {

template<typename TTestTypes>
class TwistSwingEvaluatorTest : public ::testing::Test {
protected:
    void SetUp() override {
        TwistSwingEvaluatorTest::SetUpImpl();
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value != 0ul, void>::type SetUpImpl() {
        using T = typename std::tuple_element<0, TestTypes>::type;
        using TF256 = typename std::tuple_element<1, TestTypes>::type;
        using TF128 = typename std::tuple_element<2, TestTypes>::type;
        using TRotationAdapter = typename std::tuple_element<3, TestTypes>::type;
        using TSJEvaluator = rl4::TwistSwingJointsEvaluator<T, TF256, TF128, TRotationAdapter>;

        rotationSelectorIndex = RotationOutputTypeSelector<TRotationAdapter>::value();
        rotationType = RotationOutputTypeSelector<TRotationAdapter>::rotation();
        rl4::Vector<rl4::TwistSwingSetup> setups{rltests::twsw::optimized::setupCount, rl4::TwistSwingSetup{&memRes}, &memRes};
        for (std::size_t si = {}; si < setups.size(); ++si) {
            auto& setup = setups[si];
            setup.twistTwistAxis = rltests::twsw::unoptimized::twistTwistAxes[si];
            setup.twistBlendWeights = rltests::twsw::optimized::twistBlendWeights[si];
            setup.twistOutputIndices = rltests::twsw::optimized::twistOutputIndices[rotationSelectorIndex][si];
            setup.twistInputIndices = rltests::twsw::optimized::twistInputIndices[si];
            setup.swingTwistAxis = rltests::twsw::unoptimized::swingTwistAxes[si];
            setup.swingBlendWeights = rltests::twsw::optimized::swingBlendWeights[si];
            setup.swingOutputIndices = rltests::twsw::optimized::swingOutputIndices[rotationSelectorIndex][si];
            setup.swingInputIndices = rltests::twsw::optimized::swingInputIndices[si];
        }
        auto factory = rl4::UniqueInstance<TSJEvaluator, rl4::JointsEvaluator>::with(&memRes);
        evaluator =
            factory.create(std::move(setups), TRotationAdapter{rltests::twsw::unoptimized::rotationSigns}, nullptr, &memRes);
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value == 0ul, void>::type SetUpImpl() {
        GTEST_SKIP();
    }

protected:
    pma::AlignedMemoryResource memRes;
    rltests::twsw::TwistSwingReader reader;
    rl4::JointsEvaluator::Pointer evaluator;
    std::size_t rotationSelectorIndex;
    rl4::RotationType rotationType;
};

}  // namespace

#if defined(RL_BUILD_WITH_AVX) && defined(RL_BUILD_WITH_SSE)
using TwistSwingEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>
    #else
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_AVX)
using TwistSwingEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>
    #else
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_SSE)
using TwistSwingEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>
    #else
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#else
    #ifndef RL_BUILD_WITH_HALF_FLOATS
using TwistSwingEvaluatorTypeList = ::testing::Types<
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>>;
    #else
using TwistSwingEvaluatorTypeList = ::testing::Types<std::tuple<>>;
    #endif  // RL_BUILD_WITH_HALF_FLOATS
#endif

TYPED_TEST_SUITE(TwistSwingEvaluatorTest, TwistSwingEvaluatorTypeList, );

TYPED_TEST(TwistSwingEvaluatorTest, EvaluatePerLOD) {
    const auto jointAttrCount =
        static_cast<std::uint16_t>(rltests::twsw::output::valuesPerLODPerConfig[this->rotationSelectorIndex][0].size());
    rl4::CPUJointsOutputInstance outputInstance{jointAttrCount,
                                                rl4::TranslationType::Vector,
                                                this->rotationType,
                                                rl4::ScaleType::Vector,
                                                &this->memRes};
    auto outputBuffer = outputInstance.getOutputBuffer();
    auto inputInstanceFactory =
        ControlsFactory::getInstanceFactory(0, static_cast<std::uint16_t>(rltests::twsw::input::values.size()), 0, 0, 0);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &this->memRes);
    auto inputBuffer = inputInstance->getInputBuffer();
    std::copy(rltests::twsw::input::values.begin(), rltests::twsw::input::values.end(), inputBuffer.begin());

    for (std::uint16_t lod = 0u; lod < rltests::twsw::unoptimized::lodCount; ++lod) {
        std::fill(outputBuffer.begin(), outputBuffer.end(), 0.0f);
        this->evaluator->calculate(inputInstance.get(), &outputInstance, lod);
        const auto& expected = rltests::twsw::output::valuesPerLODPerConfig[this->rotationSelectorIndex][lod];
#ifdef RL_BUILD_WITH_HALF_FLOATS
        static constexpr float threshold = 0.05f;
#else
        static constexpr float threshold = 0.002f;
#endif  // RL_BUILD_WITH_HALF_FLOATS
        ASSERT_ELEMENTS_NEAR(outputBuffer, expected, expected.size(), threshold);
    }
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
