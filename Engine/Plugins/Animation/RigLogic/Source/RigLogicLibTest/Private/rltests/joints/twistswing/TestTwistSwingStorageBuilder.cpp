// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/joints/Helpers.h"
#include "rltests/joints/twistswing/TwistSwingFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/twistswing/TwistSwingJointsBuilder.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Detect.h"

#include <tuple>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace rl4 {

template<typename T, typename TF256, typename TF128, class TRotationAdapter>
struct TwistSwingJointsEvaluator<T, TF256, TF128, TRotationAdapter>::Accessor {

    static void assertRawDataEqual(const TwistSwingJointsEvaluator<T, TF256, TF128, TRotationAdapter>& result) {
        const auto rotationSelectorIndex = RotationOutputTypeSelector<TRotationAdapter>::value();
        ASSERT_EQ(result.setups.size(), rltests::twsw::optimized::setupCount);
        for (std::size_t si = {}; si < result.setups.size(); ++si) {
            ASSERT_EQ(result.setups[si].swingTwistAxis, rltests::twsw::unoptimized::swingTwistAxes[si]);
            ASSERT_EQ(result.setups[si].swingInputIndices.size(), rltests::twsw::optimized::swingInputIndices[si].size());
            ASSERT_ELEMENTS_EQ(result.setups[si].swingInputIndices,
                               rltests::twsw::optimized::swingInputIndices[si],
                               rltests::twsw::optimized::swingInputIndices[si].size());
            ASSERT_EQ(result.setups[si].swingBlendWeights.size(), rltests::twsw::optimized::swingBlendWeights[si].size());
            ASSERT_ELEMENTS_EQ(result.setups[si].swingBlendWeights,
                               rltests::twsw::optimized::swingBlendWeights[si],
                               rltests::twsw::optimized::swingBlendWeights[si].size());
            ASSERT_EQ(result.setups[si].swingOutputIndices.size(),
                      rltests::twsw::optimized::swingOutputIndices[rotationSelectorIndex][si].size());
            ASSERT_ELEMENTS_EQ(result.setups[si].swingOutputIndices,
                               rltests::twsw::optimized::swingOutputIndices[rotationSelectorIndex][si],
                               rltests::twsw::optimized::swingOutputIndices[rotationSelectorIndex][si].size());

            ASSERT_EQ(result.setups[si].twistTwistAxis, rltests::twsw::unoptimized::twistTwistAxes[si]);
            ASSERT_EQ(result.setups[si].twistInputIndices.size(), rltests::twsw::optimized::twistInputIndices[si].size());
            ASSERT_ELEMENTS_EQ(result.setups[si].twistInputIndices,
                               rltests::twsw::optimized::twistInputIndices[si],
                               rltests::twsw::optimized::twistInputIndices[si].size());
            ASSERT_EQ(result.setups[si].twistBlendWeights.size(), rltests::twsw::optimized::twistBlendWeights[si].size());
            ASSERT_ELEMENTS_EQ(result.setups[si].twistBlendWeights,
                               rltests::twsw::optimized::twistBlendWeights[si],
                               rltests::twsw::optimized::twistBlendWeights[si].size());
            ASSERT_EQ(result.setups[si].twistOutputIndices.size(),
                      rltests::twsw::optimized::twistOutputIndices[rotationSelectorIndex][si].size());
            ASSERT_ELEMENTS_EQ(result.setups[si].twistOutputIndices,
                               rltests::twsw::optimized::twistOutputIndices[rotationSelectorIndex][si],
                               rltests::twsw::optimized::twistOutputIndices[rotationSelectorIndex][si].size());
        }
    }
};

}  // namespace rl4

namespace {

template<typename TTestTypes>
class TwistSwingJointStorageBuilderTest : public ::testing::Test {
protected:
    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value != 0ul, void>::type buildStorage() {
        using T = typename std::tuple_element<0, TTestTypes>::type;
        using TF256 = typename std::tuple_element<1, TTestTypes>::type;
        using TF128 = typename std::tuple_element<2, TTestTypes>::type;
        using TRotationAdapter = typename std::tuple_element<3, TTestTypes>::type;
        using TSJEvaluator = rl4::TwistSwingJointsEvaluator<T, TF256, TF128, TRotationAdapter>;

        rl4::Configuration config{};
        config.rotationType = RotationOutputTypeSelector<TRotationAdapter>::rotation();
        auto meta = rl4::RigMetadata::create(config, &reader, &memRes, rl4::InitializationMethod::Create);
        rl4::TwistSwingJointsBuilder<T, TF256, TF128> builder(config, meta.get(), &memRes);

        rl4::JointBehaviorFilter filter{&reader, &memRes};

        builder.computeStorageRequirements(filter);
        builder.allocateStorage(filter);
        builder.fillStorage(filter);
        auto joints = builder.build();
        rl4::TwistSwingJointsEvaluator<T, TF256, TF128, TRotationAdapter>::Accessor::assertRawDataEqual(
            *static_cast<TSJEvaluator*>(joints.get()));
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value == 0ul, void>::type buildStorage() {
    }

protected:
    pma::AlignedMemoryResource memRes;
    rltests::twsw::TwistSwingReader reader;
};

}  // namespace

#if defined(RL_BUILD_WITH_AVX) && defined(RL_BUILD_WITH_SSE)
using StorageValueTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>
    #else
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_AVX)
using StorageValueTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>
    #else
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::avx::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_SSE)
using StorageValueTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>
    #else
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#else
    #ifndef RL_BUILD_WITH_HALF_FLOATS
using StorageValueTypeList = ::testing::Types<
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::PassthroughAdapter>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128, rl4::QuaternionsToEulerAngles<tdm::frad, tdm::rot_seq::zyx>>>;
    #else
using StorageValueTypeList = ::testing::Types<std::tuple<>>;
    #endif  // RL_BUILD_WITH_HALF_FLOATS
#endif

TYPED_TEST_SUITE(TwistSwingJointStorageBuilderTest, StorageValueTypeList, );

TYPED_TEST(TwistSwingJointStorageBuilderTest, LayoutOptimization) {
    this->buildStorage();
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
