// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/joints/twistswing/TwistSwingFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/twistswing/TwistSwingJointsEvaluator.h"
#include "riglogic/types/Extent.h"

namespace rltests {

namespace twsw {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif

namespace unoptimized {

const tdm::rot_sign rotationSigns{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};

const std::uint16_t lodCount = 4u;

const pma::Matrix<float> swingBlendWeights = {{0.3f, 0.7f}, {0.5f, 0.5f}, {1.0f}};

const pma::Matrix<std::uint16_t> swingOutputJointIndices = {{2u, 3u}, {7u, 9u}, {8u}};

const pma::Matrix<std::uint16_t> swingInputControlIndices = {{0u, 1u, 2u, 3u}, {4u, 5u, 6u, 7u}, {8u, 9u, 10u, 11u}};

const pma::Vector<dna::TwistAxis> swingTwistAxes = {dna::TwistAxis::X, dna::TwistAxis::X, dna::TwistAxis::X};

const pma::Matrix<float> twistBlendWeights = {{0.2f, 0.8f}, {0.6f, 0.4f}, {1.0f}};

const pma::Matrix<std::uint16_t> twistOutputJointIndices = {{0u, 1u}, {4u, 6u}, {5u}};

const pma::Matrix<std::uint16_t> twistInputControlIndices = {{0u, 1u, 2u, 3u}, {4u, 5u, 6u, 7u}, {8u, 9u, 10u, 11u}};

const pma::Vector<dna::TwistAxis> twistTwistAxes = {dna::TwistAxis::X, dna::TwistAxis::X, dna::TwistAxis::X};

}  // namespace unoptimized

namespace optimized {

const std::uint16_t setupCount = 3u;

const pma::Matrix<float> swingBlendWeights = {{0.3f, 0.7f}, {0.5f, 0.5f}, {1.0f}};

const pma::Vector<pma::Matrix<std::uint16_t>> swingOutputIndices = {{// Quaternion outputs
                                                                     {23, 24, 25, 26, 33, 34, 35, 36},
                                                                     {73, 74, 75, 76, 93, 94, 95, 96},
                                                                     {83, 84, 85, 86}},
                                                                    {// Euler-angle outputs
                                                                     {21, 22, 23, 0, 30, 31, 32, 0},
                                                                     {66, 67, 68, 0, 84, 85, 86, 0},
                                                                     {75, 76, 77, 0}}};

const pma::Matrix<std::uint16_t> swingInputIndices = {{0u, 1u, 2u, 3u}, {4u, 5u, 6u, 7u}, {8u, 9u, 10u, 11u}};

const pma::Matrix<float> twistBlendWeights = {{0.2f, 0.8f}, {0.6f, 0.4f}, {1.0f}};

const pma::Vector<pma::Matrix<std::uint16_t>> twistOutputIndices = {{// Quaternion outputs
                                                                     {3, 4, 5, 6, 13, 14, 15, 16},
                                                                     {43, 44, 45, 46, 63, 64, 65, 66},
                                                                     {53, 54, 55, 56}},
                                                                    {// Euler-angle outputs
                                                                     {3, 4, 5, 0, 12, 13, 14, 0},
                                                                     {39, 40, 41, 0, 57, 58, 59, 0},
                                                                     {48, 49, 50, 0}}};

const pma::Matrix<std::uint16_t> twistInputIndices = {{0u, 1u, 2u, 3u}, {4u, 5u, 6u, 7u}, {8u, 9u, 10u, 11u}};

}  // namespace optimized

namespace input {

// Calculation input values
const rl4::Vector<float> values = {0.308434369465179f,
                                   0.3353868017170958f,
                                   0.14504533469235772f,
                                   0.8782629354871905f,
                                   0.24610924459196282f,
                                   0.19641565253465743f,
                                   0.40743077341317624f,
                                   0.8572346796774493f,
                                   0.24550619238838187f,
                                   0.5267005234522829f,
                                   0.3547646971819078f,
                                   0.7324310737041945f,
                                   0.0f};

}  // namespace input

namespace output {

// Expected output results for each LOD
const rl4::Vector<rl4::Matrix<float>> valuesPerLODPerConfig = {
    { // Quaternion outputs
     {// LOD-0
      0.0f, 0.0f, 0.0f, -0.26691f,  0.0f,       0.0f,        0.963722f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.067495f, 0.0f,       0.0f,        0.99772f,  0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.320053f, -0.2376f,   -0.102755f,  0.911347f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.329264f, -0.102785f, -0.0444518f, 0.937574f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.111598f, 0.0f,       0.0f,        0.993753f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        1.0f,      0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.166961f, 0.0f,       0.0f,        0.985964f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.317815f, 0.0f,       0.0f,        0.948153f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,      0.0f, 0.0f, 0.0f},
     {// LOD-1
      0.0f, 0.0f, 0.0f, -0.26691f,  0.0f,       0.0f,        0.963722f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.067495f, 0.0f,       0.0f,        0.99772f,  0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.320053f, -0.2376f,   -0.102755f,  0.911347f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.329264f, -0.102785f, -0.0444518f, 0.937574f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.111598f, 0.0f,       0.0f,        0.993753f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        1.0f,      0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.166961f, 0.0f,       0.0f,        0.985964f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.317815f, 0.0f,       0.0f,        0.948153f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,      0.0f, 0.0f, 0.0f},
     {// LOD-2
      0.0f, 0.0f, 0.0f, -0.26691f,  0.0f,       0.0f,        0.963722f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.067495f, 0.0f,       0.0f,        0.99772f,  0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.320053f, -0.2376f,   -0.102755f,  0.911347f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.329264f, -0.102785f, -0.0444518f, 0.937574f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.111598f, 0.0f,       0.0f,        0.993753f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        1.0f,      0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.166961f, 0.0f,       0.0f,        0.985964f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.317815f, 0.0f,       0.0f,        0.948153f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,      0.0f, 0.0f, 0.0f},
     {// LOD-3
      0.0f, 0.0f, 0.0f, -0.26691f,  0.0f,       0.0f,        0.963722f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.067495f, 0.0f,       0.0f,        0.99772f,  0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.320053f, -0.2376f,   -0.102755f,  0.911347f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.329264f, -0.102785f, -0.0444518f, 0.937574f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.111598f, 0.0f,       0.0f,        0.993753f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        1.0f,      0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.166961f, 0.0f,       0.0f,        0.985964f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.317815f, 0.0f,       0.0f,        0.948153f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, -0.268386f, -0.100976f, -0.209457f,  0.934827f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,      0.0f, 0.0f, 0.0f}},
    { // Euler-angle outputs
     {// LOD-0
      0.0f,       0.0f,        0.0f,       -0.540371f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       -0.135093f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       -0.747349f,
      -0.376101f, -0.373509f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       -0.688136f,  -0.164202f, -0.153704f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, -0.223662f, 0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        -0.335493f, 0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      -0.577202f, -0.0764333f, -0.46354f,  0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        -0.646848f, 0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       -0.577202f, -0.0764333f, -0.46354f,  0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f},
     {// LOD-1
      0.0f,       0.0f,        0.0f,       -0.540371f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       -0.135093f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       -0.747349f,
      -0.376101f, -0.373509f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       -0.688136f,  -0.164202f, -0.153704f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, -0.223662f, 0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        -0.335493f, 0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      -0.577202f, -0.0764333f, -0.46354f,  0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        -0.646848f, 0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       -0.577202f, -0.0764333f, -0.46354f,  0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f},
     {// LOD-2
      0.0f,       0.0f,        0.0f,       -0.540371f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       -0.135093f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       -0.747349f,
      -0.376101f, -0.373509f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       -0.688136f,  -0.164202f, -0.153704f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, -0.223662f, 0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        -0.335493f, 0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      -0.577202f, -0.0764333f, -0.46354f,  0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        -0.646848f, 0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       -0.577202f, -0.0764333f, -0.46354f,  0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f},
     {// LOD-3
      0.0f,       0.0f,        0.0f,       -0.540371f, 0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       -0.135093f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       -0.747349f,
      -0.376101f, -0.373509f,  0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       -0.688136f,  -0.164202f, -0.153704f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, -0.223662f, 0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      0.0f,       0.0f,        -0.335493f, 0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f,
      -0.577202f, -0.0764333f, -0.46354f,  0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        -0.646848f, 0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       -0.577202f, -0.0764333f, -0.46354f,  0.0f,
      0.0f,       0.0f,        0.0f,       0.0f,       0.0f, 0.0f, 0.0f,       0.0f,       0.0f,        0.0f,       0.0f}}};

}  // namespace output

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

TwistSwingReader::~TwistSwingReader() = default;

}  // namespace twsw

}  // namespace rltests
