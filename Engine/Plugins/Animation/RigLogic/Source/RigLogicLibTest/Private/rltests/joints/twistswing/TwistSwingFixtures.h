// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/dna/FakeReader.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/twistswing/TwistSwingSetup.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rltests {

namespace twsw {

using namespace rl4;

namespace unoptimized {

extern const tdm::rot_sign rotationSigns;
extern const std::uint16_t lodCount;
extern const pma::Matrix<float> swingBlendWeights;
extern const pma::Matrix<std::uint16_t> swingOutputJointIndices;
extern const pma::Matrix<std::uint16_t> swingInputControlIndices;
extern const pma::Vector<dna::TwistAxis> swingTwistAxes;
extern const pma::Matrix<float> twistBlendWeights;
extern const pma::Matrix<std::uint16_t> twistOutputJointIndices;
extern const pma::Matrix<std::uint16_t> twistInputControlIndices;
extern const pma::Vector<dna::TwistAxis> twistTwistAxes;

}  // namespace unoptimized

namespace optimized {

extern const std::uint16_t setupCount;
extern const pma::Matrix<float> swingBlendWeights;
extern const pma::Vector<pma::Matrix<std::uint16_t>> swingOutputIndices;
extern const pma::Matrix<std::uint16_t> swingInputIndices;
extern const pma::Matrix<float> twistBlendWeights;
extern const pma::Vector<pma::Matrix<std::uint16_t>> twistOutputIndices;
extern const pma::Matrix<std::uint16_t> twistInputIndices;

}  // namespace optimized

namespace input {

// Calculation input values
extern const Vector<float> values;

}  // namespace input

namespace output {

// Calculation output values
extern const Vector<Matrix<float>> valuesPerLODPerConfig;

}  // namespace output

class TwistSwingReader : public dna::FakeReader {
public:
    ~TwistSwingReader();

    std::uint16_t getTwistCount() const override {
        return static_cast<std::uint16_t>(unoptimized::twistBlendWeights.size());
    }

    dna::TwistAxis getTwistSetupTwistAxis(std::uint16_t twistIndex) const override {
        return unoptimized::twistTwistAxes[twistIndex];
    }

    ConstArrayView<std::uint16_t> getTwistInputControlIndices(std::uint16_t twistIndex) const override {
        return unoptimized::twistInputControlIndices[twistIndex];
    }

    ConstArrayView<std::uint16_t> getTwistOutputJointIndices(std::uint16_t twistIndex) const override {
        return unoptimized::twistOutputJointIndices[twistIndex];
    }

    ConstArrayView<float> getTwistBlendWeights(std::uint16_t twistIndex) const override {
        return unoptimized::twistBlendWeights[twistIndex];
    }

    std::uint16_t getSwingCount() const override {
        return static_cast<std::uint16_t>(unoptimized::swingBlendWeights.size());
    }

    dna::TwistAxis getSwingSetupTwistAxis(std::uint16_t swingIndex) const override {
        return unoptimized::swingTwistAxes[swingIndex];
    }

    ConstArrayView<std::uint16_t> getSwingInputControlIndices(std::uint16_t swingIndex) const override {
        return unoptimized::swingInputControlIndices[swingIndex];
    }

    ConstArrayView<std::uint16_t> getSwingOutputJointIndices(std::uint16_t swingIndex) const override {
        return unoptimized::swingOutputJointIndices[swingIndex];
    }

    ConstArrayView<float> getSwingBlendWeights(std::uint16_t swingIndex) const override {
        return unoptimized::swingBlendWeights[swingIndex];
    }
};

}  // namespace twsw

}  // namespace rltests
