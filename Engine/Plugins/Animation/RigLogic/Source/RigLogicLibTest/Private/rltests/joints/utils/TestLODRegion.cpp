// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"

#include "riglogic/joints/cpu/utils/LODRegion.h"

#include <cstdint>

namespace {

struct LODTestSetup {
    std::uint32_t rowCount;
    std::uint32_t lodEndRow;
    std::uint32_t blockHeight;
    std::uint32_t padTo;
    std::uint32_t lodEndRowPaddedToLastFullBlock;
    std::uint32_t lodEndRowPaddedToSecondLastFullBlock;
};

class LODRegionTest : public ::testing::TestWithParam<LODTestSetup> {};

}  // namespace

TEST(LODRegionTest, ConstructFromFixtures) {
    rl4::LODRegion lod{{4, 5, 6}, {1, 2, 3}};

    ASSERT_EQ(lod.inputLODs.size, 4u);
    ASSERT_EQ(lod.inputLODs.sizeAlignedTo4, 5u);
    ASSERT_EQ(lod.inputLODs.sizeAlignedTo8, 6u);

    ASSERT_EQ(lod.outputLODs.size, 1u);
    ASSERT_EQ(lod.outputLODs.sizePaddedToLastFullBlock, 2u);
    ASSERT_EQ(lod.outputLODs.sizePaddedToSecondLastFullBlock, 3u);
}

TEST_P(LODRegionTest, ConstructLODRegionFromVaryingParameters) {
    auto params = GetParam();
    rl4::LODRegion lod{{0, 0, 0}, {params.lodEndRow, params.rowCount, params.blockHeight, params.padTo}};
    ASSERT_EQ(lod.outputLODs.size, params.lodEndRow);
    ASSERT_EQ(lod.outputLODs.sizePaddedToLastFullBlock, params.lodEndRowPaddedToLastFullBlock);
    ASSERT_EQ(lod.outputLODs.sizePaddedToSecondLastFullBlock, params.lodEndRowPaddedToSecondLastFullBlock);
}

INSTANTIATE_TEST_SUITE_P(
    LODTestSuite,
    LODRegionTest,
    ::testing::Values(
        // Matrices passed to LODRegion have already been padded,
        // so the rowcount must be an integral multiple of 4
        // {{rows, cols}, lodEndRow, lodEndRowAlignedToLastFullBlock, lodEndRowAlignedToSecondLastFullBlock}
        // SSE test cases
        LODTestSetup{4, 2, 8, 4, 0, 0},      // Mask-off last 2 rows from 1st block-4 (handles block-4 loop)
        LODTestSetup{4, 4, 8, 4, 0, 0},      // No mask-off (handles block-4 loop)
        LODTestSetup{8, 2, 8, 4, 8, 0},      // Mask-off last 6 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{8, 4, 8, 4, 8, 0},      // Mask-off last 4 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{8, 6, 8, 4, 8, 0},      // Mask-off last 2 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{8, 8, 8, 4, 8, 8},      // No mask-off (handles block-8 loop without mask-off logic)
        LODTestSetup{12, 2, 8, 4, 8, 0},     // Mask-off last 6 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{12, 4, 8, 4, 8, 0},     // Mask-off last 4 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{12, 6, 8, 4, 8, 0},     // Mask-off last 2 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{12, 8, 8, 4, 8, 8},     // No mask-off (handles block-8 loop without mask-off logic)
        LODTestSetup{12, 10, 8, 4, 8, 8},    // Mask-off last 2 rows from block-4 (handles block-8 without mask-off, then block-4
                                             // loop)
        LODTestSetup{12, 12, 8, 4, 8, 8},    // No mask-off (handles block-8 without mask-off, then block-4 loop)
        LODTestSetup{32, 2, 8, 4, 8, 0},     // Mask-off last 6 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{32, 4, 8, 4, 8, 0},     // Mask-off last 4 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{32, 6, 8, 4, 8, 0},     // Mask-off last 2 rows from 1st block-8 (handles block-8 loop)
        LODTestSetup{32, 8, 8, 4, 8, 8},     // No mask-off (handles block-8 loop without mask-off logic)
        LODTestSetup{32, 9, 8, 4, 16, 8},    // Mask-off last 7 rows from 2nd block-8 (handles block-8 loop without mask-off, then
                                             // block-8 loop)
        LODTestSetup{32, 12, 8, 4, 16, 8},   // Mask-off last 4 rows from 2nd block-8 (handles block-8 loop without mask-off, then
                                             // block-8 loop)
        LODTestSetup{32, 16, 8, 4, 16, 16},  // No mask-off (handles block-8 loop without mask-off logic)
        // AVX test cases (pads to 8 so (rowCount % 4 == 0) is not possible)
        LODTestSetup{8, 2, 16, 8, 0, 0},
        LODTestSetup{8, 4, 16, 8, 0, 0},
        LODTestSetup{8, 6, 16, 8, 0, 0},
        LODTestSetup{8, 8, 16, 8, 0, 0},
        LODTestSetup{16, 2, 16, 8, 16, 0},
        LODTestSetup{16, 4, 16, 8, 16, 0},
        LODTestSetup{16, 6, 16, 8, 16, 0},
        LODTestSetup{16, 8, 16, 8, 16, 0},
        LODTestSetup{16, 10, 16, 8, 16, 0},
        LODTestSetup{16, 12, 16, 8, 16, 0},
        LODTestSetup{16, 14, 16, 8, 16, 0},
        LODTestSetup{16, 16, 16, 8, 16, 16},
        LODTestSetup{24, 2, 16, 8, 16, 0},
        LODTestSetup{24, 4, 16, 8, 16, 0},
        LODTestSetup{24, 6, 16, 8, 16, 0},
        LODTestSetup{24, 8, 16, 8, 16, 0},
        LODTestSetup{24, 10, 16, 8, 16, 0},
        LODTestSetup{24, 12, 16, 8, 16, 0},
        LODTestSetup{24, 14, 16, 8, 16, 0},
        LODTestSetup{24, 16, 16, 8, 16, 16},
        LODTestSetup{24, 18, 16, 8, 16, 16},
        LODTestSetup{24, 20, 16, 8, 16, 16},
        LODTestSetup{24, 22, 16, 8, 16, 16},
        LODTestSetup{24, 24, 16, 8, 16, 16},
        LODTestSetup{32, 2, 16, 8, 16, 0},
        LODTestSetup{32, 4, 16, 8, 16, 0},
        LODTestSetup{32, 6, 16, 8, 16, 0},
        LODTestSetup{32, 8, 16, 8, 16, 0},
        LODTestSetup{32, 10, 16, 8, 16, 0},
        LODTestSetup{32, 12, 16, 8, 16, 0},
        LODTestSetup{32, 14, 16, 8, 16, 0},
        LODTestSetup{32, 16, 16, 8, 16, 16},
        LODTestSetup{32, 18, 16, 8, 32, 16},
        LODTestSetup{32, 20, 16, 8, 32, 16},
        LODTestSetup{32, 22, 16, 8, 32, 16},
        LODTestSetup{32, 24, 16, 8, 32, 16},
        LODTestSetup{32, 26, 16, 8, 32, 16},
        LODTestSetup{32, 28, 16, 8, 32, 16},
        LODTestSetup{32, 30, 16, 8, 32, 16},
        LODTestSetup{32, 32, 16, 8, 32, 32}));
