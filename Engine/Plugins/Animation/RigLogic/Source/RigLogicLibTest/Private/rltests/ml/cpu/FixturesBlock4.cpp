// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/ml/cpu/FixturesBlock4.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/joints/cpu/CPUJointsOutputInstance.h"
#include "riglogic/joints/cpu/bpcm/BPCMJointsEvaluator.h"
#include "riglogic/types/Extent.h"

namespace rltests {

namespace ml {

namespace block4 {

using namespace rl4;

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif

namespace unoptimized {

const std::uint16_t rawControlCount = 13;  // rltests::ml::block4::input::values.size()
const std::uint16_t mlControlCount = 24;   // number of all parameters under Operation-3
const std::uint16_t lodCount = 4;
const Vector<Matrix<dna::MachineLearnedBehaviorOperationType>> mlOperationTypes = {
    { // Type-0
     {// Operation-Set-0
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather},
     {// Operation-Set-1
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP},
     {// Operation-Set-2
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter}},
    { // Type-1
     {// Operation-Set-0
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather,
      dna::MachineLearnedBehaviorOperationType::Gather},
     {// Operation-Set-1
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP},
     {// Operation-Set-2
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP},
     {// Operation-Set-3
      dna::MachineLearnedBehaviorOperationType::WeightedSum,
      dna::MachineLearnedBehaviorOperationType::WeightedSum},
     {// Operation-Set-4
      dna::MachineLearnedBehaviorOperationType::MLP,
      dna::MachineLearnedBehaviorOperationType::MLP},
     {// Operation-Set-5
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter,
      dna::MachineLearnedBehaviorOperationType::Scatter}}};
const Matrix<Matrix<std::uint32_t>> mlOperationParameters = {{  // Type-0
                                                              { // Operation Set 0
                                                               {// Operation 0
                                                                1u,
                                                                4u},
                                                               {// Operation 1
                                                                0u,
                                                                2u,
                                                                3u},
                                                               {// Operation 2
                                                                5u,
                                                                7u,
                                                                8u,
                                                                9u,
                                                                12u},
                                                               {// Operation 3
                                                                6u,
                                                                10u,
                                                                11u}},
                                                              {// Operation Set 1
                                                               {
                                                                   // Operation 0
                                                                   0  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   1  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 2
                                                                   2  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 3
                                                                   3  // Neural network index
                                                               }},
                                                              { // Operation Set 2
                                                               {// Operation 0
                                                                13u},
                                                               {// Operation 1
                                                                14u,
                                                                15u,
                                                                16u},
                                                               {// Operation 2
                                                                17u,
                                                                18u,
                                                                19u,
                                                                20u,
                                                                21u,
                                                                22u},
                                                               {// Operation 3
                                                                23u,
                                                                24u,
                                                                25u,
                                                                26u}}},
                                                             {  // Type-1
                                                              { // Operation Set 0
                                                               {// Operation 0
                                                                1u,
                                                                4u},
                                                               {// Operation 1
                                                                0u,
                                                                2u,
                                                                3u},
                                                               {// Operation 2
                                                                5u,
                                                                7u,
                                                                8u,
                                                                9u,
                                                                12u},
                                                               {// Operation 3
                                                                6u,
                                                                10u,
                                                                11u},
                                                               {// Operation 4
                                                                11u,
                                                                8u,
                                                                7u,
                                                                6u,
                                                                4u},
                                                               {// Operation 5
                                                                10u,
                                                                9u,
                                                                5u}},
                                                              {// Operation Set 1
                                                               {
                                                                   // Operation 0
                                                                   0  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   1  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 2
                                                                   2  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 3
                                                                   3  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 4
                                                                   2  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 5
                                                                   3  // Neural network index
                                                               }},
                                                              {// Operation Set 2
                                                               {
                                                                   // Operation 4
                                                                   2u  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 5
                                                                   3u  // Neural network index
                                                               }},
                                                              {// Operation Set 3
                                                               {
                                                                   // Operation 0
                                                                   6u,
                                                                   1065353216,
                                                                   1051372203,
                                                                   1051372203,
                                                                   1051372203  // (output count, 1.0, 0.333, 0.333, 0.333)
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   4u,
                                                                   1065353216,
                                                                   1051372203,
                                                                   1051372203,
                                                                   1051372203  // (output count, 1.0, 0.333, 0.333, 0.333)
                                                               }},
                                                              {// Operation Set 4
                                                               {
                                                                   // Operation 0
                                                                   2u  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   3u  // Neural network index
                                                               }},
                                                              { // Operation Set 5
                                                               {// Operation 0
                                                                13u},
                                                               {// Operation 1
                                                                14u,
                                                                15u,
                                                                16u},
                                                               {// Operation 2
                                                                17u,
                                                                18u,
                                                                19u,
                                                                20u,
                                                                21u,
                                                                22u},
                                                               {// Operation 3
                                                                23u,
                                                                24u,
                                                                25u,
                                                                26u},
                                                               {// Operation 4
                                                                27u,
                                                                28u,
                                                                29u,
                                                                30u,
                                                                31u,
                                                                32u},
                                                               {// Operation 5
                                                                33u,
                                                                34u,
                                                                35u,
                                                                36u}}}};
const Matrix<Matrix<std::uint16_t>> mlDependencyOperationSetIndices = {{ // Type-0
                                                                        {// Operation Set 0
                                                                         {
                                                                             // Operation 0
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                         },
                                                                         {
                                                                             // Operation 2
                                                                         },
                                                                         {
                                                                             // Operation 3
                                                                         }},
                                                                        {// Operation Set 1
                                                                         {
                                                                             // Operation 0
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 2
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 3
                                                                             0  // Operation Set 0
                                                                         }},
                                                                        {// Operation Set 2
                                                                         {
                                                                             // Operation 0
                                                                             1  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                             1  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 2
                                                                             1  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 3
                                                                             1  // Operation Set 1
                                                                         }}},
                                                                       { // Type-1
                                                                        {// Operation Set 0
                                                                         {
                                                                             // Operation 0
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                         },
                                                                         {
                                                                             // Operation 2
                                                                         },
                                                                         {
                                                                             // Operation 3
                                                                         },
                                                                         {
                                                                             // Operation 4
                                                                         },
                                                                         {
                                                                             // Operation 5
                                                                         }},
                                                                        {// Operation Set 1
                                                                         {
                                                                             // Operation 0
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 2
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 3
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 4
                                                                             0  // Operation Set 0
                                                                         },
                                                                         {
                                                                             // Operation 5
                                                                             0  // Operation Set 0
                                                                         }},
                                                                        {// Operation Set 2
                                                                         {
                                                                             // Operation 0
                                                                             1,  // Operation Set 1
                                                                             1   // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                             1  // Operation Set 1
                                                                         }},
                                                                        {// Operation Set 3 (weighted sum)
                                                                         {
                                                                             // Operation 0
                                                                             1,  // Operation Set 1
                                                                             1,  // Operation Set 1
                                                                             1,  // Operation Set 1
                                                                             2,  // Operation Set 2
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                             1,  // Operation Set 1
                                                                             1,  // Operation Set 1
                                                                             1,  // Operation Set 1
                                                                             2,  // Operation Set 2
                                                                         }},
                                                                        {// Operation Set 4
                                                                         {
                                                                             // Operation 0
                                                                             3  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                             3  // Operation Set 1
                                                                         }},
                                                                        {// Operation Set 5
                                                                         {
                                                                             // Operation 0
                                                                             1  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 1
                                                                             1  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 2
                                                                             1  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 3
                                                                             1  // Operation Set 1
                                                                         },
                                                                         {
                                                                             // Operation 4
                                                                             4  // Operation Set 3
                                                                         },
                                                                         {
                                                                             // Operation 5
                                                                             4  // Operation Set 3
                                                                         }}}};
const Matrix<Matrix<std::uint16_t>> mlDependencyOperationIndices = {
    { // Type-0
     {// Operation Set 0
      {
          // Operation 0
      },
      {
          // Operation 1
      },
      {
          // Operation 2
      },
      {
          // Operation 3
      }},
     {// Operation Set 1
      {
          // Operation 0
          0  // Operation 0
      },
      {
          // Operation 1
          1  // Operation 1
      },
      {
          // Operation 2
          2  // Operation 2
      },
      {
          // Operation 3
          3  // Operation 3
      }},
     {// Operation Set 2
      {
          // Operation 0
          0  // Operation 0
      },
      {
          // Operation 1
          1  // Operation 1
      },
      {
          // Operation 2
          2  // Operation 2
      },
      {
          // Operation 3
          3  // Operation 3
      }}},
    { // Type-1
     {// Operation Set 0
      {
          // Operation 0
      },
      {
          // Operation 1
      },
      {
          // Operation 2
      },
      {
          // Operation 3
      },
      {
          // Operation 4
      },
      {
          // Operation 5
      }},
     {// Operation Set 1
      {
          // Operation 0
          0  // Operation 0
      },
      {
          // Operation 1
          1  // Operation 1
      },
      {
          // Operation 2
          2  // Operation 2
      },
      {
          // Operation 3
          3  // Operation 3
      },
      {
          // Operation 4
          4  // Operation 4
      },
      {
          // Operation 5
          5  // Operation 5
      }},
     {// Operation Set 2
      {
          // Operation 4
          0,  // Operation 0
          3   // Operation 3
      },
      {
          // Operation 5
          1  // Operation 1
      }},
     {// Operation Set 3 (weighted sum)
      {
          // Operation 0
          4,  // Operation 4 (set 1 - represents parent in average)
          2,  // Operation 2 (set 1)
          4,  // Operation 4 (set 1)
          0   // Operation 0 (set 2)
      },
      {
          // Operation 1
          5,  // Operation 5 (set 1 - represents parent in average)
          3,  // Operation 3 (set 1)
          5,  // Operation 5 (set 1)
          1   // Operation 1 (set 2)
      }},
     {// Operation Set 4
      {
          // Operation 0
          0  // Operation 0
      },
      {
          // Operation 1
          1  // Operation 1
      }},
     {// Operation Set 5
      {
          // Operation 0
          0  // Operation 0
      },
      {
          // Operation 1
          1  // Operation 1
      },
      {
          // Operation 2
          2  // Operation 2
      },
      {
          // Operation 3
          3  // Operation 3
      },
      {
          // Operation 0
          0  // Operation 0
      },
      {
          // Operation 1
          1  // Operation 1
      }}}};
const Matrix<Matrix<std::uint16_t>> mlOperationIndicesPerLOD = {{ // Type-0
                                                                 {// Operation-Set-0
                                                                  {0u, 1u, 2u, 3u},
                                                                  {0u, 1u, 2u},
                                                                  {},
                                                                  {}},
                                                                 {// Operation-Set-1
                                                                  {0u, 1u, 2u, 3u},
                                                                  {0u, 1u, 2u},
                                                                  {},
                                                                  {}},
                                                                 {// Operation-Set-2
                                                                  {0u, 1u, 2u, 3u},
                                                                  {0u, 1u, 2u},
                                                                  {},
                                                                  {}}},
                                                                { // Type-1
                                                                 {// Operation-Set-0
                                                                  {0u, 1u, 2u, 3u, 4u, 5u},
                                                                  {0u, 1u, 2u, 3u},
                                                                  {},
                                                                  {}},
                                                                 {// Operation-Set-1
                                                                  {0u, 1u, 2u, 3u, 4u, 5u},
                                                                  {0u, 1u, 2u, 3u},
                                                                  {},
                                                                  {}},
                                                                 {// Operation-Set-2
                                                                  {0u, 1u},
                                                                  {0u, 1u},
                                                                  {},
                                                                  {}},
                                                                 {// Operation-Set-3
                                                                  {0u, 1u},
                                                                  {},
                                                                  {},
                                                                  {}},
                                                                 {// Operation-Set-4
                                                                  {0u, 1u},
                                                                  {},
                                                                  {},
                                                                  {}},
                                                                 {// Operation-Set-5
                                                                  {0u, 1u, 2u, 3u, 4u, 5u},
                                                                  {0u, 1u, 2u, 3u},
                                                                  {},
                                                                  {}}}};
const Matrix<dna::ActivationFunction> mlbNetActivationFunctions = {{
                                                                       // Mesh-0 Region-0
                                                                       dna::ActivationFunction::sigmoid,  // Layer 0
                                                                       dna::ActivationFunction::sigmoid,  // Layer 1
                                                                       dna::ActivationFunction::linear    // Layer 2
                                                                   },
                                                                   {
                                                                       // Mesh-0 Region-1
                                                                       dna::ActivationFunction::leakyrelu,  // Layer 0
                                                                       dna::ActivationFunction::leakyrelu,  // Layer 1
                                                                       dna::ActivationFunction::leakyrelu,  // Layer 2
                                                                       dna::ActivationFunction::linear      // Layer 3
                                                                   },
                                                                   {
                                                                       // Mesh-1 Region-0
                                                                       dna::ActivationFunction::relu,   // Layer 0
                                                                       dna::ActivationFunction::relu,   // Layer 1
                                                                       dna::ActivationFunction::relu,   // Layer 2
                                                                       dna::ActivationFunction::linear  // Layer 3
                                                                   },
                                                                   {
                                                                       // Mesh-1 Region-1
                                                                       dna::ActivationFunction::tanh,   // Layer 0
                                                                       dna::ActivationFunction::linear  // Layer 1
                                                                   }};
const Vector<Matrix<float>> mlbNetActivationFunctionParameters = {{
                                                                      // Mesh-0 Region-0
                                                                      {},  // Layer 0
                                                                      {},  // Layer 1
                                                                      {}   // Layer 2
                                                                  },
                                                                  {
                                                                      // Mesh-0 Region-1
                                                                      {0.5f},  // Layer 0
                                                                      {0.5f},  // Layer 1
                                                                      {0.5f},  // Layer 2
                                                                      {}       // Layer 3
                                                                  },
                                                                  {
                                                                      // Mesh-1 Region-0
                                                                      {},  // Layer 0
                                                                      {},  // Layer 1
                                                                      {},  // Layer 2
                                                                      {}   // Layer 3
                                                                  },
                                                                  {
                                                                      // Mesh-1 Region-1
                                                                      {},  // Layer 0
                                                                      {}   // Layer 1
                                                                  }};
const Vector<Matrix<float>> mlbNetWeights = {
    {// Mesh-0 Region-0
     {
         // Layer 0 - Pad to block-4 => block-4
         0.11f,
         0.11f,  // Unit-0
         0.11f,
         0.11f  // Unit-1
     },
     {
         // Layer 1 - No padding => block-4
         0.22f,
         0.22f,  // Unit-0
         0.33f,
         0.33f,  // Unit-1
         0.44f,
         0.44f,  // Unit-2
         0.55f,
         0.55f  // Unit-3
     },
     {
         // Layer 2 - Pad to block-4 => block-4
         0.66f,
         0.66f,
         0.66f,
         0.66f  // Unit-0
     }},
    {// Mesh-0 Region-1
     {
         // Layer 0 - Pad to block-8 => block-8
         0.66f, 0.66f, 0.66f,  // Unit-0
         0.77f, 0.77f, 0.77f,  // Unit-1
         0.88f, 0.88f, 0.88f,  // Unit-2
         0.99f, 0.99f, 0.99f,  // Unit-3
         1.0f,  1.0f,  1.0f,   // Unit-4
         1.11f, 1.11f, 1.11f,  // Unit-5
         1.22f, 1.22f, 1.22f   // Unit-6
     },
     {// Layer 1 - No padding => block-8
      1.33f, 1.33f, 1.33f, 1.33f, 1.33f, 1.33f, 1.33f, 1.44f, 1.44f, 1.44f, 1.44f, 1.44f, 1.44f, 1.44f,
      1.55f, 1.55f, 1.55f, 1.55f, 1.55f, 1.55f, 1.55f, 1.66f, 1.66f, 1.66f, 1.66f, 1.66f, 1.66f, 1.66f,
      1.77f, 1.77f, 1.77f, 1.77f, 1.77f, 1.77f, 1.77f, 1.88f, 1.88f, 1.88f, 1.88f, 1.88f, 1.88f, 1.88f,
      1.99f, 1.99f, 1.99f, 1.99f, 1.99f, 1.99f, 1.99f, 2.0f,  2.0f,  2.0f,  2.0f,  2.0f,  2.0f,  2.0f},
     {// Layer 2 - Pad to block-4 => block-8 + block-4
      -0.09f, -0.09f, -0.09f, -0.09f, -0.09f, -0.09f, -0.09f, -0.09f, -0.08f, -0.08f, -0.08f, -0.08f, -0.08f, -0.08f, -0.08f,
      -0.08f, -0.07f, -0.07f, -0.07f, -0.07f, -0.07f, -0.07f, -0.07f, -0.07f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f,
      -0.06f, -0.06f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f,
      -0.04f, -0.04f, -0.04f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.02f, -0.02f, -0.02f, -0.02f,
      -0.02f, -0.02f, -0.02f, -0.02f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f},
     {// Layer 3 - Pad to block-4 => block-4
      -0.30f, -0.30f, -0.30f, -0.30f, -0.30f, -0.30f, -0.30f, -0.30f, -0.30f, -0.31f, -0.31f, -0.31f, -0.31f, -0.31f,
      -0.31f, -0.31f, -0.31f, -0.31f, -0.32f, -0.32f, -0.32f, -0.32f, -0.32f, -0.32f, -0.32f, -0.32f, -0.32f}},
    { // Mesh-1 Region-0
     {// Layer 0 - No padding => block-8 + block-4
      0.30f, 0.30f, 0.30f, 0.30f, 0.30f, 0.31f, 0.31f, 0.31f, 0.31f, 0.31f, 0.32f, 0.32f, 0.32f, 0.32f, 0.32f,
      0.33f, 0.33f, 0.33f, 0.33f, 0.33f, 0.34f, 0.34f, 0.34f, 0.34f, 0.34f, 0.35f, 0.35f, 0.35f, 0.35f, 0.35f,
      0.36f, 0.36f, 0.36f, 0.36f, 0.36f, 0.37f, 0.37f, 0.37f, 0.37f, 0.37f, 0.38f, 0.38f, 0.38f, 0.38f, 0.38f,
      0.39f, 0.39f, 0.39f, 0.39f, 0.39f, 0.40f, 0.40f, 0.40f, 0.40f, 0.40f, 0.41f, 0.41f, 0.41f, 0.41f, 0.41f},
     {// Layer 1 - Pad to block-8 => block-8 + block-8
      0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.42f, 0.43f, 0.43f, 0.43f, 0.43f, 0.43f,
      0.43f, 0.43f, 0.43f, 0.43f, 0.43f, 0.43f, 0.43f, 0.44f, 0.44f, 0.44f, 0.44f, 0.44f, 0.44f, 0.44f, 0.44f, 0.44f, 0.44f,
      0.44f, 0.44f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.45f, 0.46f, 0.46f, 0.46f,
      0.46f, 0.46f, 0.46f, 0.46f, 0.46f, 0.46f, 0.46f, 0.46f, 0.46f, 0.47f, 0.47f, 0.47f, 0.47f, 0.47f, 0.47f, 0.47f, 0.47f,
      0.47f, 0.47f, 0.47f, 0.47f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.48f, 0.49f,
      0.49f, 0.49f, 0.49f, 0.49f, 0.49f, 0.49f, 0.49f, 0.49f, 0.49f, 0.49f, 0.49f, 0.50f, 0.50f, 0.50f, 0.50f, 0.50f, 0.50f,
      0.50f, 0.50f, 0.50f, 0.50f, 0.50f, 0.50f, 0.51f, 0.51f, 0.51f, 0.51f, 0.51f, 0.51f, 0.51f, 0.51f, 0.51f, 0.51f, 0.51f,
      0.51f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.52f, 0.53f, 0.53f, 0.53f, 0.53f,
      0.53f, 0.53f, 0.53f, 0.53f, 0.53f, 0.53f, 0.53f, 0.53f, 0.54f, 0.54f, 0.54f, 0.54f, 0.54f, 0.54f, 0.54f, 0.54f, 0.54f,
      0.54f, 0.54f, 0.54f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f, 0.55f},
     {// Layer 2 - No padding => block-8 + block-8
      0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.56f, 0.57f, 0.57f,
      0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.57f, 0.58f, 0.58f, 0.58f, 0.58f,
      0.58f, 0.58f, 0.58f, 0.58f, 0.58f, 0.58f, 0.58f, 0.58f, 0.58f, 0.58f, 0.59f, 0.59f, 0.59f, 0.59f, 0.59f, 0.59f,
      0.59f, 0.59f, 0.59f, 0.59f, 0.59f, 0.59f, 0.59f, 0.59f, 0.60f, 0.60f, 0.60f, 0.60f, 0.60f, 0.60f, 0.60f, 0.60f,
      0.60f, 0.60f, 0.60f, 0.60f, 0.60f, 0.60f, 0.61f, 0.61f, 0.61f, 0.61f, 0.61f, 0.61f, 0.61f, 0.61f, 0.61f, 0.61f,
      0.61f, 0.61f, 0.61f, 0.61f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f, 0.62f,
      0.62f, 0.62f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f, 0.63f,
      0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.64f, 0.65f, 0.65f,
      0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.65f, 0.66f, 0.66f, 0.66f, 0.66f,
      0.66f, 0.66f, 0.66f, 0.66f, 0.66f, 0.66f, 0.66f, 0.66f, 0.66f, 0.66f, 0.67f, 0.67f, 0.67f, 0.67f, 0.67f, 0.67f,
      0.67f, 0.67f, 0.67f, 0.67f, 0.67f, 0.67f, 0.67f, 0.67f, 0.68f, 0.68f, 0.68f, 0.68f, 0.68f, 0.68f, 0.68f, 0.68f,
      0.68f, 0.68f, 0.68f, 0.68f, 0.68f, 0.68f, 0.69f, 0.69f, 0.69f, 0.69f, 0.69f, 0.69f, 0.69f, 0.69f, 0.69f, 0.69f,
      0.69f, 0.69f, 0.69f, 0.69f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f, 0.70f,
      0.70f, 0.70f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f, 0.71f},
     {// Layer 3 - Pad to block-4 => block-8
      -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f, -0.01f,
      -0.01f, -0.01f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f, -0.02f,
      -0.02f, -0.02f, -0.02f, -0.02f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f,
      -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.03f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f,
      -0.04f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f, -0.04f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f,
      -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.05f, -0.06f, -0.06f, -0.06f, -0.06f,
      -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f, -0.06f}},
    { // Mesh-1 Region-1
     {// Layer 0 - Pad to block-4 => block-4
      1.0f,
      1.0f,
      1.0f,
      2.0f,
      2.0f,
      2.0f},
     {// Layer 1 - Pad to block-4 => block-4
      3.0f,
      3.0f,
      4.0f,
      4.0f,
      5.0f,
      5.0f,
      6.0f,
      6.0f}}};

const Vector<Matrix<float>> mlbNetBiases = {{ // Mesh-0 Region-0
                                             {// Layer 0
                                              0.1f,
                                              0.1f},
                                             {// Layer 1
                                              0.2f,
                                              0.2f,
                                              0.2f,
                                              0.2f},
                                             {
                                                 // Layer 2
                                                 0.3f,
                                             }},
                                            {// Mesh-0 Region-1
                                             {
                                                 // Layer 0
                                                 0.4f,
                                                 0.4f,
                                                 0.4f,
                                                 0.4f,
                                                 0.4f,
                                                 0.4f,
                                                 0.4f,
                                             },
                                             {// Layer 1
                                              0.5f,
                                              0.5f,
                                              0.5f,
                                              0.5f,
                                              0.5f,
                                              0.5f,
                                              0.5f,
                                              0.5f},
                                             {// Layer 2
                                              0.6f,
                                              0.6f,
                                              0.6f,
                                              0.6f,
                                              0.6f,
                                              0.6f,
                                              0.6f,
                                              0.6f,
                                              0.6f},
                                             {// Layer 3
                                              0.7f,
                                              0.7f,
                                              0.7f}},
                                            {// Mesh-1 Region-0
                                             {
                                                 // Layer 0
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                                 0.8f,
                                             },
                                             {// Layer 1
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f,
                                              0.9f},
                                             {// Layer 2
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f,
                                              1.1f},
                                             {// Layer 3
                                              1.2f,
                                              1.2f,
                                              1.2f,
                                              1.2f,
                                              1.2f,
                                              1.2f}},
                                            { // Mesh-1 Region-1
                                             {// Layer 0
                                              1.3f,
                                              1.3f},
                                             {// Layer 1
                                              1.4f,
                                              1.4f,
                                              1.4f,
                                              1.4f}}};

}  // namespace unoptimized

namespace optimized {

const Vector<Matrix<rl4::ml::cpu::OperationSetType>> mlOperationTypes = {{ // Type-0
                                                                          {// Operation-0
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP}},
                                                                         { // Type-1
                                                                          {// Operation-0
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP},
                                                                          {// Operation-1
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP},
                                                                          {// Operation-2
                                                                           rl4::ml::cpu::OperationSetType::WeightedSum64,
                                                                           rl4::ml::cpu::OperationSetType::WeightedSum64},
                                                                          {// Operation-3
                                                                           rl4::ml::cpu::OperationSetType::MLP,
                                                                           rl4::ml::cpu::OperationSetType::MLP}}};

const Matrix<Matrix<std::uint32_t>> mlOperationParameters = {{ // Type-0
                                                              {// Operation Set 1
                                                               {
                                                                   // Operation 0
                                                                   0  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   1  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 2
                                                                   2  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 3
                                                                   3  // Neural network index
                                                               }}},
                                                             { // Type-1
                                                              {// Operation Set 1
                                                               {
                                                                   // Operation 0
                                                                   0  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   1  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 2
                                                                   2  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 3
                                                                   3  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 4
                                                                   2  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 5
                                                                   3  // Neural network index
                                                               }},
                                                              {// Operation Set 2
                                                               {
                                                                   // Operation 4
                                                                   2u  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 5
                                                                   3u  // Neural network index
                                                               }},
                                                              {// Operation Set 3
                                                               {
                                                                   // Operation 0
                                                                   6u,
                                                                   1065353216,
                                                                   1065353216,
                                                                   1065353216  // (output count, 1.0, 1.0, 1.0)
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   4u,
                                                                   1065353216,
                                                                   1065353216,
                                                                   1065353216  // (output count, 1.0, 1.0, 1.0)
                                                               }},
                                                              {// Operation Set 4
                                                               {
                                                                   // Operation 0
                                                                   2u  // Neural network index
                                                               },
                                                               {
                                                                   // Operation 1
                                                                   3u  // Neural network index
                                                               }}}};

const Vector<AlignedMatrix<float>> mlbNetWeightsFloat = {
    { // Mesh-0 Region-0
     {// Layer-0
      0.11f,
      0.11f,
      0.0f,
      0.0f,
      0.11f,
      0.11f,
      0.0f,
      0.0f},
     {// Layer-1
      0.22f,
      0.33f,
      0.44f,
      0.55f,
      0.22f,
      0.33f,
      0.44f,
      0.55f},
     {// Layer-2
      0.66f,
      0.0f,
      0.0f,
      0.0f,
      0.66f,
      0.0f,
      0.0f,
      0.0f,
      0.66f,
      0.0f,
      0.0f,
      0.0f,
      0.66f,
      0.0f,
      0.0f,
      0.0f}},
    { // Mesh-0 Region-1
     {// Layer-0
      0.66f, 0.77f, 0.88f, 0.99f, 1.0f,  1.11f, 1.22f, 0.0f,  0.66f, 0.77f, 0.88f, 0.99f,
      1.0f,  1.11f, 1.22f, 0.0f,  0.66f, 0.77f, 0.88f, 0.99f, 1.0f,  1.11f, 1.22f, 0.0f},
     {// Layer-1
      1.33f, 1.44f, 1.55f, 1.66f, 1.77f, 1.88f, 1.99f, 2.0f,  1.33f, 1.44f, 1.55f, 1.66f, 1.77f, 1.88f,
      1.99f, 2.0f,  1.33f, 1.44f, 1.55f, 1.66f, 1.77f, 1.88f, 1.99f, 2.0f,  1.33f, 1.44f, 1.55f, 1.66f,
      1.77f, 1.88f, 1.99f, 2.0f,  1.33f, 1.44f, 1.55f, 1.66f, 1.77f, 1.88f, 1.99f, 2.0f,  1.33f, 1.44f,
      1.55f, 1.66f, 1.77f, 1.88f, 1.99f, 2.0f,  1.33f, 1.44f, 1.55f, 1.66f, 1.77f, 1.88f, 1.99f, 2.0f},
     {// Layer-2
      -0.09f, -0.08f, -0.07f, -0.06f, -0.05f, -0.04f, -0.03f, -0.02f, -0.09f, -0.08f, -0.07f, -0.06f, -0.05f, -0.04f,
      -0.03f, -0.02f, -0.09f, -0.08f, -0.07f, -0.06f, -0.05f, -0.04f, -0.03f, -0.02f, -0.09f, -0.08f, -0.07f, -0.06f,
      -0.05f, -0.04f, -0.03f, -0.02f, -0.09f, -0.08f, -0.07f, -0.06f, -0.05f, -0.04f, -0.03f, -0.02f, -0.09f, -0.08f,
      -0.07f, -0.06f, -0.05f, -0.04f, -0.03f, -0.02f, -0.09f, -0.08f, -0.07f, -0.06f, -0.05f, -0.04f, -0.03f, -0.02f,
      -0.09f, -0.08f, -0.07f, -0.06f, -0.05f, -0.04f, -0.03f, -0.02f, -0.01f, 0.0f,   0.0f,   0.0f,   -0.01f, 0.0f,
      0.0f,   0.0f,   -0.01f, 0.0f,   0.0f,   0.0f,   -0.01f, 0.0f,   0.0f,   0.0f,   -0.01f, 0.0f,   0.0f,   0.0f,
      -0.01f, 0.0f,   0.0f,   0.0f,   -0.01f, 0.0f,   0.0f,   0.0f,   -0.01f, 0.0f,   0.0f,   0.0f},
     {// Layer-3
      -0.30f, -0.31f, -0.32f, 0.0f, -0.30f, -0.31f, -0.32f, 0.0f, -0.30f, -0.31f, -0.32f, 0.0f,
      -0.30f, -0.31f, -0.32f, 0.0f, -0.30f, -0.31f, -0.32f, 0.0f, -0.30f, -0.31f, -0.32f, 0.0f,
      -0.30f, -0.31f, -0.32f, 0.0f, -0.30f, -0.31f, -0.32f, 0.0f, -0.30f, -0.31f, -0.32f, 0.0f}},
    { // Mesh-1 Region-0
     {// Layer-0
      0.30f, 0.31f, 0.32f, 0.33f, 0.34f, 0.35f, 0.36f, 0.37f, 0.30f, 0.31f, 0.32f, 0.33f, 0.34f, 0.35f, 0.36f,
      0.37f, 0.30f, 0.31f, 0.32f, 0.33f, 0.34f, 0.35f, 0.36f, 0.37f, 0.30f, 0.31f, 0.32f, 0.33f, 0.34f, 0.35f,
      0.36f, 0.37f, 0.30f, 0.31f, 0.32f, 0.33f, 0.34f, 0.35f, 0.36f, 0.37f, 0.38f, 0.39f, 0.40f, 0.41f, 0.38f,
      0.39f, 0.40f, 0.41f, 0.38f, 0.39f, 0.40f, 0.41f, 0.38f, 0.39f, 0.40f, 0.41f, 0.38f, 0.39f, 0.40f, 0.41f},
     {// Layer-1
      0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f, 0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f,
      0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f, 0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f,
      0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f, 0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f,
      0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f, 0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f,
      0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f, 0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f,
      0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f, 0.42f, 0.43f, 0.44f, 0.45f, 0.46f, 0.47f, 0.48f, 0.49f,
      0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,  0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,
      0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,  0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,
      0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,  0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,
      0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,  0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,
      0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,  0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,
      0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f,  0.50f, 0.51f, 0.52f, 0.53f, 0.54f, 0.55f, 0.0f,  0.0f},
     {
         // Layer-2
         0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f, 0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f,
         0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f, 0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f,
         0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f, 0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f,
         0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f, 0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f,
         0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f, 0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f,
         0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f, 0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f,
         0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f, 0.56f, 0.57f, 0.58f, 0.59f, 0.60f, 0.61f, 0.62f, 0.63f,
         0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f, 0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f,
         0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f, 0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f,
         0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f, 0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f,
         0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f, 0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f,
         0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f, 0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f,
         0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f, 0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f,
         0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f, 0.64f, 0.65f, 0.66f, 0.67f, 0.68f, 0.69f, 0.70f, 0.71f,
     },
     {// Layer-3
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f,
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f,
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f,
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f,
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f,
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f,
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f,
      -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f, -0.01f, -0.02f, -0.03f, -0.04f, -0.05f, -0.06f, 0.0f, 0.0f}},
    { // Mesh-1 Region-1
     {// Layer-0
      1.0f,
      2.0f,
      0.0f,
      0.0f,
      1.0f,
      2.0f,
      0.0f,
      0.0f,
      1.0f,
      2.0f,
      0.0f,
      0.0f},
     {// Layer-1
      3.0f,
      4.0f,
      5.0f,
      6.0f,
      3.0f,
      4.0f,
      5.0f,
      6.0f}}};

const Vector<AlignedMatrix<std::uint16_t>> mlbNetWeightsHalfFloat = {
    { // Mesh-0 Region-0
     {// Layer-0
      12042,
      12042,
      0,
      0,
      12042,
      12042,
      0,
      0},
     {// Layer-1
      13066,
      13640,
      14090,
      14438,
      13066,
      13640,
      14090,
      14438},
     {// Layer-2
      14664,
      0,
      0,
      0,
      14664,
      0,
      0,
      0,
      14664,
      0,
      0,
      0,
      14664,
      0,
      0,
      0}},
    { // Mesh-0 Region-1
     {// Layer-0
      14664, 14889, 15114, 15340, 15360, 15473, 15585, 0,     14664, 14889, 15114, 15340,
      15360, 15473, 15585, 0,     14664, 14889, 15114, 15340, 15360, 15473, 15585, 0},
     {// Layer-1
      15698, 15811, 15923, 16036, 16148, 16261, 16374, 16384, 15698, 15811, 15923, 16036, 16148, 16261,
      16374, 16384, 15698, 15811, 15923, 16036, 16148, 16261, 16374, 16384, 15698, 15811, 15923, 16036,
      16148, 16261, 16374, 16384, 15698, 15811, 15923, 16036, 16148, 16261, 16374, 16384, 15698, 15811,
      15923, 16036, 16148, 16261, 16374, 16384, 15698, 15811, 15923, 16036, 16148, 16261, 16374, 16384},
     {// Layer-2
      44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271, 44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271,
      44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271, 44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271,
      44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271, 44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271,
      44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271, 44483, 44319, 44155, 43950, 43622, 43295, 42926, 42271,
      41247, 0,     0,     0,     41247, 0,     0,     0,     41247, 0,     0,     0,     41247, 0,     0,     0,
      41247, 0,     0,     0,     41247, 0,     0,     0,     41247, 0,     0,     0,     41247, 0,     0,     0},
     {// Layer-3
      46285, 46326, 46367, 0, 46285, 46326, 46367, 0, 46285, 46326, 46367, 0, 46285, 46326, 46367, 0, 46285, 46326, 46367, 0,
      46285, 46326, 46367, 0, 46285, 46326, 46367, 0, 46285, 46326, 46367, 0, 46285, 46326, 46367, 0}},
    { // Mesh-1 Region-0
     {// Layer-0
      13517, 13558, 13599, 13640, 13681, 13722, 13763, 13804, 13517, 13558, 13599, 13640, 13681, 13722, 13763,
      13804, 13517, 13558, 13599, 13640, 13681, 13722, 13763, 13804, 13517, 13558, 13599, 13640, 13681, 13722,
      13763, 13804, 13517, 13558, 13599, 13640, 13681, 13722, 13763, 13804, 13844, 13885, 13926, 13967, 13844,
      13885, 13926, 13967, 13844, 13885, 13926, 13967, 13844, 13885, 13926, 13967, 13844, 13885, 13926, 13967},
     {// Layer-1
      14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295, 14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295,
      14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295, 14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295,
      14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295, 14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295,
      14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295, 14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295,
      14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295, 14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295,
      14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295, 14008, 14049, 14090, 14131, 14172, 14213, 14254, 14295,
      14336, 14356, 14377, 14397, 14418, 14438, 0,     0,     14336, 14356, 14377, 14397, 14418, 14438, 0,     0,
      14336, 14356, 14377, 14397, 14418, 14438, 0,     0,     14336, 14356, 14377, 14397, 14418, 14438, 0,     0,
      14336, 14356, 14377, 14397, 14418, 14438, 0,     0,     14336, 14356, 14377, 14397, 14418, 14438, 0,     0,
      14336, 14356, 14377, 14397, 14418, 14438, 0,     0,     14336, 14356, 14377, 14397, 14418, 14438, 0,     0,
      14336, 14356, 14377, 14397, 14418, 14438, 0,     0,     14336, 14356, 14377, 14397, 14418, 14438, 0,     0,
      14336, 14356, 14377, 14397, 14418, 14438, 0,     0,     14336, 14356, 14377, 14397, 14418, 14438, 0,     0},
     {// Layer-2
      14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602, 14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602,
      14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602, 14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602,
      14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602, 14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602,
      14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602, 14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602,
      14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602, 14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602,
      14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602, 14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602,
      14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602, 14459, 14479, 14500, 14520, 14541, 14561, 14582, 14602,
      14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766, 14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766,
      14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766, 14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766,
      14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766, 14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766,
      14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766, 14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766,
      14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766, 14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766,
      14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766, 14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766,
      14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766, 14623, 14643, 14664, 14684, 14705, 14725, 14746, 14766},
     {// Layer-3
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0,
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0,
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0,
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0,
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0,
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0,
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0,
      41247, 42271, 42926, 43295, 43622, 43950, 0, 0, 41247, 42271, 42926, 43295, 43622, 43950, 0, 0}},
    { // Mesh-1 Region-1
     {// Layer-0
      15360,
      16384,
      0,
      0,
      15360,
      16384,
      0,
      0,
      15360,
      16384,
      0,
      0},
     {// Layer-1
      16896,
      17408,
      17664,
      17920,
      16896,
      17408,
      17664,
      17920}}};

const Vector<AlignedMatrix<float>> mlbNetBiasesFloat = {{ // Mesh-0 Region-0
                                                         {// Layer 0
                                                          0.1f,
                                                          0.1f,
                                                          0.0f,
                                                          0.0f},
                                                         {// Layer 1
                                                          0.2f,
                                                          0.2f,
                                                          0.2f,
                                                          0.2f},
                                                         {// Layer 2
                                                          0.3f,
                                                          0.0f,
                                                          0.0f,
                                                          0.0f}},
                                                        { // Mesh-0 Region-1
                                                         {// Layer 0
                                                          0.4f,
                                                          0.4f,
                                                          0.4f,
                                                          0.4f,
                                                          0.4f,
                                                          0.4f,
                                                          0.4f,
                                                          0.0f},
                                                         {// Layer 1
                                                          0.5f,
                                                          0.5f,
                                                          0.5f,
                                                          0.5f,
                                                          0.5f,
                                                          0.5f,
                                                          0.5f,
                                                          0.5f},
                                                         {// Layer 2
                                                          0.6f,
                                                          0.6f,
                                                          0.6f,
                                                          0.6f,
                                                          0.6f,
                                                          0.6f,
                                                          0.6f,
                                                          0.6f,
                                                          0.6f,
                                                          0.0f,
                                                          0.0f,
                                                          0.0f},
                                                         {// Layer 3
                                                          0.7f,
                                                          0.7f,
                                                          0.7f,
                                                          0.0f}},
                                                        {// Mesh-1 Region-0
                                                         {
                                                             // Layer 0
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                             0.8f,
                                                         },
                                                         {// Layer 1
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.9f,
                                                          0.0f,
                                                          0.0f},
                                                         {// Layer 2
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f,
                                                          1.1f},
                                                         {// Layer 3
                                                          1.2f,
                                                          1.2f,
                                                          1.2f,
                                                          1.2f,
                                                          1.2f,
                                                          1.2f,
                                                          0.0f,
                                                          0.0f}},
                                                        { // Mesh-1 Region-1
                                                         {// Layer 0
                                                          1.3f,
                                                          1.3f,
                                                          0.0f,
                                                          0.0f},
                                                         {// Layer 1
                                                          1.4f,
                                                          1.4f,
                                                          1.4f,
                                                          1.4f}}};

const Vector<AlignedMatrix<std::uint16_t>> mlbNetBiasesHalfFloat = {{ // Mesh-0 Region-0
                                                                     {// Layer 0
                                                                      11878,
                                                                      11878,
                                                                      0,
                                                                      0},
                                                                     {// Layer 1
                                                                      12902,
                                                                      12902,
                                                                      12902,
                                                                      12902},
                                                                     {// Layer 2
                                                                      13517,
                                                                      0,
                                                                      0,
                                                                      0}},
                                                                    { // Mesh-0 Region-1
                                                                     {// Layer 0
                                                                      13926,
                                                                      13926,
                                                                      13926,
                                                                      13926,
                                                                      13926,
                                                                      13926,
                                                                      13926,
                                                                      0},
                                                                     {// Layer 1
                                                                      14336,
                                                                      14336,
                                                                      14336,
                                                                      14336,
                                                                      14336,
                                                                      14336,
                                                                      14336,
                                                                      14336},
                                                                     {// Layer 2
                                                                      14541,
                                                                      14541,
                                                                      14541,
                                                                      14541,
                                                                      14541,
                                                                      14541,
                                                                      14541,
                                                                      14541,
                                                                      14541,
                                                                      0,
                                                                      0,
                                                                      0},
                                                                     {// Layer 3
                                                                      14746,
                                                                      14746,
                                                                      14746,
                                                                      0}},
                                                                    {// Mesh-1 Region-0
                                                                     {
                                                                         // Layer 0
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                         14950,
                                                                     },
                                                                     {// Layer 1
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      15155,
                                                                      0,
                                                                      0},
                                                                     {// Layer 2
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462,
                                                                      15462},
                                                                     {// Layer 3
                                                                      15565,
                                                                      15565,
                                                                      15565,
                                                                      15565,
                                                                      15565,
                                                                      15565,
                                                                      0,
                                                                      0}},
                                                                    { // Mesh-1 Region-1
                                                                     {// Layer 0
                                                                      15667,
                                                                      15667,
                                                                      0,
                                                                      0},
                                                                     {// Layer 1
                                                                      15770,
                                                                      15770,
                                                                      15770,
                                                                      15770}}};

const Matrix<LODSpec<std::uint16_t>> lods = {{// ML Type 0
                                              {
                                                  // Operation-Set-0
                                                  { // Operation indices
                                                   {// LOD-0
                                                    0u,
                                                    1u,
                                                    2u,
                                                    3u},
                                                   {// LOD-1
                                                    0u,
                                                    1u,
                                                    2u},
                                                   {
                                                       // LOD-2
                                                   },
                                                   {
                                                       // LOD-3
                                                   }},
                                                  4u  // Operation count
                                              }},
                                             {// ML Type 1
                                              {
                                                  // Operation-Set-0
                                                  { // Operation indices
                                                   {// LOD-0
                                                    0u,
                                                    1u,
                                                    2u,
                                                    3u,
                                                    4u,
                                                    5u},
                                                   {// LOD-1
                                                    0u,
                                                    1u,
                                                    2u,
                                                    3u},
                                                   {
                                                       // LOD-2
                                                   },
                                                   {
                                                       // LOD-3
                                                   }},
                                                  6u  // Operation count
                                              },
                                              {
                                                  // Operation-Set-1
                                                  { // Operation indices
                                                   {// LOD-0
                                                    0u,
                                                    1u},
                                                   {// LOD-1
                                                    0u,
                                                    1u},
                                                   {
                                                       // LOD-2
                                                   },
                                                   {
                                                       // LOD-3
                                                   }},
                                                  2u  // Operation count
                                              },
                                              {
                                                  // Operation-Set-2
                                                  { // Operation indices
                                                   {// LOD-0
                                                    0u,
                                                    1u},
                                                   {
                                                       // LOD-1
                                                   },
                                                   {
                                                       // LOD-2
                                                   },
                                                   {
                                                       // LOD-3
                                                   }},
                                                  2u  // Operation count
                                              },
                                              {
                                                  // Operation-Set-3
                                                  { // Operation indices
                                                   {// LOD-0
                                                    0u,
                                                    1u},
                                                   {
                                                       // LOD-1
                                                   },
                                                   {
                                                       // LOD-2
                                                   },
                                                   {
                                                       // LOD-3
                                                   }},
                                                  2u  // Operation count
                                              }}};

template<>
const rl4::Vector<rl4::AlignedMatrix<float>>& Values<float>::weights() {
    return optimized::mlbNetWeightsFloat;
}

template<>
const rl4::Vector<rl4::AlignedMatrix<float>>& Values<float>::biases() {
    return optimized::mlbNetBiasesFloat;
}

template<>
const rl4::Vector<rl4::AlignedMatrix<std::uint16_t>>& Values<std::uint16_t>::weights() {
    return optimized::mlbNetWeightsHalfFloat;
}

template<>
const rl4::Vector<rl4::AlignedMatrix<std::uint16_t>>& Values<std::uint16_t>::biases() {
    return optimized::mlbNetBiasesHalfFloat;
}

}  // namespace optimized

namespace input {

// Calculation input values
const rl4::Vector<float> values = {0.1f, 0.2f, 0.3f, 0.4f, 0.0f, 0.6f, 0.7f, 0.8f, 0.9f, 0.0f, 0.11f, 0.12f, 0.13f};

}  // namespace input

namespace output {

// Expected output results for each LOD
const rl4::Matrix<float> valuesPerLOD = {{// LOD-0
                                          2.00812602f,  7.61041260f,  7.84075928f,   8.07110691f,  -14.0201883f, -29.2403755f,
                                          -44.4605598f, -59.6807518f, -74.9009399f,  -90.1211243f, 7.32063866f,  9.29418468f,
                                          11.2677307f,  13.2412767f,  -0.256159782f, -1.71231961f, -3.16847968f, -4.62463951f,
                                          -6.08080006f, -7.53695965f, 7.40000010f,   9.39999962f,  11.3999996f,  13.3999996f},
                                         {// LOD-1
                                          2.00812592326754f,
                                          7.6104124f,
                                          7.84075948f,
                                          8.07110656f,
                                          -14.0201885552f,
                                          -29.2403771104f,
                                          -44.4605656656f,
                                          -59.6807542208f,
                                          -74.900942776f,
                                          -90.1211313312f,
                                          7.3206385240577f,
                                          9.29418469874361f,
                                          11.2677308734295f,
                                          13.2412770481154f,
                                          0.0f,
                                          0.0f,
                                          0.0f,
                                          0.0f,
                                          0.0f,
                                          0.0f,
                                          0.0f,
                                          0.0f,
                                          0.0f,
                                          0.0f},
                                         {// LOD-2
                                          0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                          0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},
                                         {// LOD-3
                                          0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
                                          0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}};

}  // namespace output

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

CanonicalReader::~CanonicalReader() = default;

}  // namespace block4

}  // namespace ml

}  // namespace rltests
