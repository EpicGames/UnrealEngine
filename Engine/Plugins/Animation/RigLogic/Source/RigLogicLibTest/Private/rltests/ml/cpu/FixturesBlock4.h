// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/dna/FakeReader.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/Operation.h"
#include "riglogic/types/LODSpec.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rltests {

namespace ml {

namespace block4 {

using namespace rl4;

namespace unoptimized {

extern const std::uint16_t rawControlCount;
extern const std::uint16_t mlControlCount;
extern const std::uint16_t lodCount;
extern const Vector<Matrix<dna::MachineLearnedBehaviorOperationType>> mlOperationTypes;
extern const Matrix<Matrix<std::uint32_t>> mlOperationParameters;
extern const Matrix<Matrix<std::uint16_t>> mlDependencyOperationSetIndices;
extern const Matrix<Matrix<std::uint16_t>> mlDependencyOperationIndices;
extern const Matrix<Matrix<std::uint16_t>> mlOperationIndicesPerLOD;

// [neuralNetIndex][layerIndex]
extern const Matrix<dna::ActivationFunction> mlbNetActivationFunctions;
// [neuralNetIndex][layerIndex][paramIndex]
extern const Vector<Matrix<float>> mlbNetActivationFunctionParameters;
// [neuralNetIndex][layerIndex][weightIndex]
extern const Vector<Matrix<float>> mlbNetWeights;
// [neuralNetIndex][layerIndex][biasIndex]
extern const Vector<Matrix<float>> mlbNetBiases;

}  // namespace unoptimized

namespace optimized {

extern const Vector<Matrix<rl4::ml::cpu::OperationSetType>> mlOperationTypes;
extern const Matrix<Matrix<std::uint32_t>> mlOperationParameters;
extern const Vector<AlignedMatrix<float>> mlbNetWeightsFloat;
extern const Vector<AlignedMatrix<std::uint16_t>> mlbNetWeightsHalfFloat;
extern const Vector<AlignedMatrix<float>> mlbNetBiasesFloat;
extern const Vector<AlignedMatrix<std::uint16_t>> mlbNetBiasesHalfFloat;
extern const Matrix<LODSpec<std::uint16_t>> lods;

template<typename TValue>
struct Values {
    static const rl4::Vector<rl4::AlignedMatrix<TValue>>& weights();
    static const rl4::Vector<rl4::AlignedMatrix<TValue>>& biases();
};

}  // namespace optimized

namespace input {

// Calculation input values
extern const Vector<float> values;

}  // namespace input

namespace output {

// Calculation output values
extern const Matrix<float> valuesPerLOD;

}  // namespace output

class CanonicalReader : public dna::FakeReader {
public:
    ~CanonicalReader();

    std::uint16_t getRawControlCount() const override {
        return unoptimized::rawControlCount;
    }

    std::uint16_t getLODCount() const override {
        return unoptimized::lodCount;
    }

    std::uint16_t getMLTypeCount() const override {
        return static_cast<std::uint16_t>(unoptimized::mlOperationTypes.size());
    }

    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override {
        return static_cast<std::uint16_t>(unoptimized::mlOperationTypes[mlTypeIndex].size());
    }

    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override {
        return static_cast<std::uint16_t>(unoptimized::mlOperationTypes[mlTypeIndex][mlOperationSetIndex].size());
    }

    dna::MachineLearnedBehaviorOperationType getMLOperationType(std::uint16_t mlTypeIndex,
                                                                std::uint16_t mlOperationSetIndex,
                                                                std::uint16_t mlOperationIndex) const override {
        return unoptimized::mlOperationTypes[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint32_t> getMLOperationParameters(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t mlOperationIndex) const override {
        return unoptimized::mlOperationParameters[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                              std::uint16_t mlOperationSetIndex,
                                                                              std::uint16_t mlOperationIndex) const override {
        return unoptimized::mlDependencyOperationSetIndices[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                           std::uint16_t mlOperationSetIndex,
                                                                           std::uint16_t mlOperationIndex) const override {
        return unoptimized::mlDependencyOperationIndices[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex,
                                                              std::uint16_t lod) const override {
        return unoptimized::mlOperationIndicesPerLOD[mlTypeIndex][mlOperationSetIndex][lod];
    }

    std::uint16_t getNeuralNetworkLayerCount(std::uint16_t neuralNetIndex) const override {
        return static_cast<std::uint16_t>(unoptimized::mlbNetWeights[neuralNetIndex].size());
    }

    dna::ActivationFunction getNeuralNetworkLayerActivationFunction(std::uint16_t neuralNetIndex,
                                                                    std::uint16_t layerIndex) const override {
        return unoptimized::mlbNetActivationFunctions[neuralNetIndex][layerIndex];
    }

    ConstArrayView<float> getNeuralNetworkLayerActivationFunctionParameters(std::uint16_t neuralNetIndex,
                                                                            std::uint16_t layerIndex) const override {
        return unoptimized::mlbNetActivationFunctionParameters[neuralNetIndex][layerIndex];
    }

    ConstArrayView<float> getNeuralNetworkLayerBiases(std::uint16_t neuralNetIndex, std::uint16_t layerIndex) const override {
        return unoptimized::mlbNetBiases[neuralNetIndex][layerIndex];
    }

    ConstArrayView<float> getNeuralNetworkLayerWeights(std::uint16_t neuralNetIndex, std::uint16_t layerIndex) const override {
        return unoptimized::mlbNetWeights[neuralNetIndex][layerIndex];
    }
};

}  // namespace block4

}  // namespace ml

}  // namespace rltests
