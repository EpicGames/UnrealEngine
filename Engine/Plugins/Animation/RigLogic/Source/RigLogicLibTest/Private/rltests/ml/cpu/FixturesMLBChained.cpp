// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/ml/cpu/FixturesMLBChained.h"

#include "riglogic/TypeDefs.h"

namespace rltests {

namespace ml {

namespace chained {

using namespace rl4;

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif

// 8 raw input controls (indices 0..7 are gathered) + 8 ML output controls (indices 8..15).
const std::uint16_t rawControlCount = 8u;
const std::uint16_t mlControlCount = 8u;
const std::uint16_t lodCount = 2u;

// Unoptimized pipeline (5 op-sets):
//   Op-Set-0: Gather         – gathers raw controls [0..7]
//   Op-Set-1: MLP (1 layer)  – intermediate dep, no scatter; 1 layer = odd → exercises normalization fix
//   Op-Set-2: WeightedSum    – reads from Op-Set-1; outputCount=8 → exercises SIMD aligned-block path
//   Op-Set-3: MLP (2 layers) – reads from Op-Set-2, scatters to controls [8..15]
//   Op-Set-4: Scatter        – drives outputControlIndices on Op-Set-3
const Vector<Matrix<dna::MachineLearnedBehaviorOperationType>> mlOperationTypes = {{
    // Type-0
    {dna::MachineLearnedBehaviorOperationType::Gather},       // Op-Set-0
    {dna::MachineLearnedBehaviorOperationType::MLP},          // Op-Set-1
    {dna::MachineLearnedBehaviorOperationType::WeightedSum},  // Op-Set-2
    {dna::MachineLearnedBehaviorOperationType::MLP},          // Op-Set-3
    {dna::MachineLearnedBehaviorOperationType::Scatter}       // Op-Set-4
}};

// Gather params: control indices. MLP params: [neuralNetIndex]. WeightedSum params: [outputCount, weight_bits...].
// Scatter params: output control indices.
const Matrix<Matrix<std::uint32_t>> mlOperationParameters = {{
    // Type-0
    {{0u, 1u, 2u, 3u, 4u, 5u, 6u, 7u}},       // Op-Set-0 Gather: gathers controls [0..7]
    {{0u}},                                   // Op-Set-1 MLP:    neural net index 0
    {{8u, 1065353216u}},                      // Op-Set-2 WS:     outputCount=8, weight=1.0 (0x3F800000)
    {{1u}},                                   // Op-Set-3 MLP:    neural net index 1
    {{8u, 9u, 10u, 11u, 12u, 13u, 14u, 15u}}  // Op-Set-4 Scatter: output control indices [8..15]
}};

const Matrix<Matrix<std::uint16_t>> mlDependencyOperationSetIndices = {{
    // Type-0
    {{}},    // Op-Set-0 Gather:  no deps
    {{0u}},  // Op-Set-1 MLP:     dep on Op-Set-0
    {{1u}},  // Op-Set-2 WS:      dep on Op-Set-1
    {{2u}},  // Op-Set-3 MLP:     dep on Op-Set-2
    {{3u}}   // Op-Set-4 Scatter: dep on Op-Set-3
}};

const Matrix<Matrix<std::uint16_t>> mlDependencyOperationIndices = {{
    // Type-0
    {{}},    // Op-Set-0: no deps
    {{0u}},  // Op-Set-1: dep op 0
    {{0u}},  // Op-Set-2: dep op 0
    {{0u}},  // Op-Set-3: dep op 0
    {{0u}}   // Op-Set-4: dep op 0
}};

// Both LODs run the single operation in each set.
const Matrix<Matrix<std::uint16_t>> mlOperationIndicesPerLOD = {{
    // Type-0
    {{0u}, {0u}},  // Op-Set-0: LOD0={op0}, LOD1={op0}
    {{0u}, {0u}},  // Op-Set-1
    {{0u}, {0u}},  // Op-Set-2
    {{0u}, {0u}},  // Op-Set-3
    {{0u}, {0u}}   // Op-Set-4
}};

// NN0: 1 layer (odd), linear activation, 8 inputs -> 8 outputs.
//   Weights: diagonal × 2  =>  output[i] = 2 × input[i].
// NN1: 2 layers (even), linear activation, 8 inputs -> 8 outputs.
//   Weights: identity in both layers  =>  output = input passthrough.
const Matrix<dna::ActivationFunction> mlbNetActivationFunctions = {
    {dna::ActivationFunction::linear},                                  // NN0: 1 layer
    {dna::ActivationFunction::linear, dna::ActivationFunction::linear}  // NN1: 2 layers
};

const Vector<Matrix<float>> mlbNetActivationFunctionParameters = {
    {{}},     // NN0
    {{}, {}}  // NN1
};

const Vector<Matrix<float>> mlbNetWeights = {
    {// NN0 Layer 0: 8-in × 8-out, diagonal × 2  (row-major: row i = weights for output i)
     {2.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 2.0f}},
    {// NN1 Layer 0: 8-in × 8-out, identity
     {1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f},
     // NN1 Layer 1: 8-in × 8-out, identity
     {1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f,
      0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f}}};

const Vector<Matrix<float>> mlbNetBiases = {
    {{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}},  // NN0 Layer 0
    {{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},   // NN1 Layer 0
     {0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f}}   // NN1 Layer 1
};

namespace input {
// Raw control values placed at indices 0..7 (rawControlCount=8).
const Vector<float> values = {0.1f, 0.2f, 0.3f, 0.4f, 0.5f, 0.6f, 0.7f, 0.8f};
}  // namespace input

namespace output {
// Expected ML output for controls [8..15] at each LOD.
// Computation:
//   Gather:      [0.1, 0.2, 0.3, 0.4, 0.5, 0.6, 0.7, 0.8]
//   NN0 (×2):    [0.2, 0.4, 0.6, 0.8, 1.0, 1.2, 1.4, 1.6]
//   WS (w=1.0):  [0.2, 0.4, 0.6, 0.8, 1.0, 1.2, 1.4, 1.6]  (SIMD aligned-block path: outputCount=8)
//   NN1 (id×2):  [0.2, 0.4, 0.6, 0.8, 1.0, 1.2, 1.4, 1.6]
//   Scatter:      controls[8..15] = [0.2, 0.4, 0.6, 0.8, 1.0, 1.2, 1.4, 1.6]
const Matrix<float> valuesPerLOD = {
    {0.2f, 0.4f, 0.6f, 0.8f, 1.0f, 1.2f, 1.4f, 1.6f},  // LOD 0
    {0.2f, 0.4f, 0.6f, 0.8f, 1.0f, 1.2f, 1.4f, 1.6f}   // LOD 1
};
}  // namespace output

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

CanonicalReader::~CanonicalReader() = default;

}  // namespace chained

}  // namespace ml

}  // namespace rltests
