// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "rltests/Defs.h"
#include "rltests/dna/FakeReader.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/Operation.h"
#include "riglogic/types/LODSpec.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rltests {

namespace ml {

// Chained topology: Gather -> MLP(odd layers, intermediate dep) -> WeightedSum -> MLP(even layers, scatter)
// Purpose: exercise cross-op-set dep gather path and the odd-layer ping-pong normalization fix.
namespace chained {

using namespace rl4;

extern const std::uint16_t rawControlCount;
extern const std::uint16_t mlControlCount;
extern const std::uint16_t lodCount;

extern const Vector<Matrix<dna::MachineLearnedBehaviorOperationType>> mlOperationTypes;
extern const Matrix<Matrix<std::uint32_t>> mlOperationParameters;
extern const Matrix<Matrix<std::uint16_t>> mlDependencyOperationSetIndices;
extern const Matrix<Matrix<std::uint16_t>> mlDependencyOperationIndices;
extern const Matrix<Matrix<std::uint16_t>> mlOperationIndicesPerLOD;

extern const Matrix<dna::ActivationFunction> mlbNetActivationFunctions;
extern const Vector<Matrix<float>> mlbNetActivationFunctionParameters;
extern const Vector<Matrix<float>> mlbNetWeights;
extern const Vector<Matrix<float>> mlbNetBiases;

namespace input {
extern const Vector<float> values;
}  // namespace input

namespace output {
extern const Matrix<float> valuesPerLOD;
}  // namespace output

class CanonicalReader : public dna::FakeReader {
public:
    ~CanonicalReader();

    std::uint16_t getRawControlCount() const override {
        return rawControlCount;
    }

    std::uint16_t getLODCount() const override {
        return lodCount;
    }

    std::uint16_t getMLTypeCount() const override {
        return static_cast<std::uint16_t>(mlOperationTypes.size());
    }

    std::uint16_t getMLOperationSetCount(std::uint16_t mlTypeIndex) const override {
        return static_cast<std::uint16_t>(mlOperationTypes[mlTypeIndex].size());
    }

    std::uint16_t getMLOperationCount(std::uint16_t mlTypeIndex, std::uint16_t mlOperationSetIndex) const override {
        return static_cast<std::uint16_t>(mlOperationTypes[mlTypeIndex][mlOperationSetIndex].size());
    }

    dna::MachineLearnedBehaviorOperationType getMLOperationType(std::uint16_t mlTypeIndex,
                                                                std::uint16_t mlOperationSetIndex,
                                                                std::uint16_t mlOperationIndex) const override {
        return mlOperationTypes[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint32_t> getMLOperationParameters(std::uint16_t mlTypeIndex,
                                                           std::uint16_t mlOperationSetIndex,
                                                           std::uint16_t mlOperationIndex) const override {
        return mlOperationParameters[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationSetIndices(std::uint16_t mlTypeIndex,
                                                                              std::uint16_t mlOperationSetIndex,
                                                                              std::uint16_t mlOperationIndex) const override {
        return mlDependencyOperationSetIndices[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint16_t> getMLOperationDependencyOperationIndices(std::uint16_t mlTypeIndex,
                                                                           std::uint16_t mlOperationSetIndex,
                                                                           std::uint16_t mlOperationIndex) const override {
        return mlDependencyOperationIndices[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
    }

    ConstArrayView<std::uint16_t> getMLOperationIndicesForLOD(std::uint16_t mlTypeIndex,
                                                              std::uint16_t mlOperationSetIndex,
                                                              std::uint16_t lod) const override {
        return mlOperationIndicesPerLOD[mlTypeIndex][mlOperationSetIndex][lod];
    }

    std::uint16_t getNeuralNetworkLayerCount(std::uint16_t neuralNetIndex) const override {
        return static_cast<std::uint16_t>(mlbNetWeights[neuralNetIndex].size());
    }

    dna::ActivationFunction getNeuralNetworkLayerActivationFunction(std::uint16_t neuralNetIndex,
                                                                    std::uint16_t layerIndex) const override {
        return mlbNetActivationFunctions[neuralNetIndex][layerIndex];
    }

    ConstArrayView<float> getNeuralNetworkLayerActivationFunctionParameters(std::uint16_t neuralNetIndex,
                                                                            std::uint16_t layerIndex) const override {
        return mlbNetActivationFunctionParameters[neuralNetIndex][layerIndex];
    }

    ConstArrayView<float> getNeuralNetworkLayerBiases(std::uint16_t neuralNetIndex, std::uint16_t layerIndex) const override {
        return mlbNetBiases[neuralNetIndex][layerIndex];
    }

    ConstArrayView<float> getNeuralNetworkLayerWeights(std::uint16_t neuralNetIndex, std::uint16_t layerIndex) const override {
        return mlbNetWeights[neuralNetIndex][layerIndex];
    }
};

}  // namespace chained

}  // namespace ml

}  // namespace rltests
