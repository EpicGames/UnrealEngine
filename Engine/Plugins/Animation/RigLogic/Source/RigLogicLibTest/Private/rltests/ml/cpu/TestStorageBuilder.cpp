// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MSC_VER
    #pragma warning(disable : 4503)
#endif

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/ml/cpu/FixturesBlock4.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorEvaluator.h"
#include "riglogic/ml/cpu/CPUMachineLearnedBehaviorFactory.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Utils.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace rl4 {

namespace ml {

namespace cpu {

template<typename T, typename TF256, typename TF128>
struct MLPSetVerifier {

    void operator()(const OperationSet::Pointer& setPtr, std::uint16_t mlOperationIndex, std::uint16_t netIndex) {
        const auto* set = static_cast<const MLPOperationSet<T, TF256, TF128>*>(setPtr.get());
        ASSERT_LT(mlOperationIndex, set->ops.size());
        const auto& op = set->ops[mlOperationIndex];
        ASSERT_EQ(op.neuralNet.layers.size(), rltests::ml::block4::unoptimized::mlbNetActivationFunctions[netIndex].size());
        for (std::uint16_t layerIndex = {}; layerIndex < op.neuralNet.layers.size(); ++layerIndex) {
            const auto& layer = op.neuralNet.layers[layerIndex];
            using OptimizedValues = typename rltests::ml::block4::optimized::Values<T>;
            const T* flatPtr = op.neuralNet.flatData.template data<T>();
            rl4::ConstArrayView<T> weights(flatPtr + layer.weightOffset, layer.weights.padded.size());
            rl4::ConstArrayView<T> biases(flatPtr + layer.biasOffset, layer.weights.padded.rows);
            ASSERT_EQ(weights, OptimizedValues::weights()[netIndex][layerIndex]);
            ASSERT_EQ(biases, OptimizedValues::biases()[netIndex][layerIndex]);
            ASSERT_EQ(layer.activationFunction,
                      rltests::ml::block4::unoptimized::mlbNetActivationFunctions[netIndex][layerIndex]);
            ASSERT_EQ(layer.activationFunctionParameters,
                      rltests::ml::block4::unoptimized::mlbNetActivationFunctionParameters[netIndex][layerIndex]);
        }
    }
};

struct Evaluator::Accessor {

    static void assertRawDataEqual(const Evaluator& result, const Configuration& config) {
        const auto mlTypeCount = rltests::ml::block4::unoptimized::mlOperationTypes.size();
        ASSERT_EQ(result.lods.size(), mlTypeCount);
        ASSERT_EQ(result.mlOperations.size(), mlTypeCount);
        ASSERT_EQ(result.bufferSizes.size(), mlTypeCount);

        for (std::uint16_t mlTypeIndex = {}; mlTypeIndex < mlTypeCount; ++mlTypeIndex) {
            const auto mlOperationSetCount = rltests::ml::block4::optimized::mlOperationTypes[mlTypeIndex].size();
            ASSERT_EQ(result.mlOperations[mlTypeIndex].size(), mlOperationSetCount);

            for (std::uint16_t mlOperationSetIndex = {}; mlOperationSetIndex < mlOperationSetCount; ++mlOperationSetIndex) {
                ASSERT_EQ(result.lods[mlTypeIndex][mlOperationSetIndex].indicesPerLOD,
                          rltests::ml::block4::optimized::lods[mlTypeIndex][mlOperationSetIndex].indicesPerLOD);
                ASSERT_EQ(result.lods[mlTypeIndex][mlOperationSetIndex].count,
                          rltests::ml::block4::optimized::lods[mlTypeIndex][mlOperationSetIndex].count);

                const auto mlOperationCount =
                    rltests::ml::block4::optimized::mlOperationTypes[mlTypeIndex][mlOperationSetIndex].size();

                const auto& opSet = result.mlOperations[mlTypeIndex][mlOperationSetIndex];
                ASSERT_NE(opSet, nullptr);

                for (std::uint16_t mlOperationIndex = {}; mlOperationIndex < mlOperationCount; ++mlOperationIndex) {
                    const auto expectedType =
                        rltests::ml::block4::optimized::mlOperationTypes[mlTypeIndex][mlOperationSetIndex][mlOperationIndex];
                    if (expectedType == rl4::ml::cpu::OperationSetType::MLP) {
                        ASSERT_EQ(opSet->getType(), rl4::ml::cpu::OperationSetType::MLP);
                        const auto& params =
                            rltests::ml::block4::optimized::mlOperationParameters[mlTypeIndex][mlOperationSetIndex]
                                                                                 [mlOperationIndex];
                        const auto neuralNetIndex = static_cast<std::uint16_t>(params[0]);
                        RuntimeTemplateInstantiator rti{&config};
                        rti.invoke<MLPSetVerifier, void>(opSet, mlOperationIndex, neuralNetIndex);
                    } else {
                        ASSERT_NE(opSet->getType(), rl4::ml::cpu::OperationSetType::MLP);
                    }
                }
            }
        }
    }
};

}  // namespace cpu

}  // namespace ml

}  // namespace rl4

namespace {

class MLBSStorageBuilderTest : public ::testing::TestWithParam<rl4::CalculationType> {};

}  // namespace

TEST_P(MLBSStorageBuilderTest, LayoutOptimization) {
    pma::AlignedMemoryResource memRes;
    rltests::ml::block4::CanonicalReader reader;
    rl4::Configuration config = {};
    config.calculationType = GetParam();
    auto meta = rl4::RigMetadata::create(config, &reader, &memRes, rl4::InitializationMethod::Create);
    auto evaluator = rl4::ml::cpu::Factory::create(config, meta.get(), &reader, &memRes);
    auto evaluatorImpl = static_cast<rl4::ml::cpu::Evaluator*>(evaluator.get());
    rl4::ml::cpu::Evaluator::Accessor::assertRawDataEqual(*evaluatorImpl, config);
}

INSTANTIATE_TEST_SUITE_P(MLBSStorageBuilderTestSuite,
                         MLBSStorageBuilderTest,
                         ::testing::Values(rl4::CalculationType::AVX,
                                           rl4::CalculationType::SSE,
                                           rl4::CalculationType::NEON,
                                           rl4::CalculationType::Scalar));

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
