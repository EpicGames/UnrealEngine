// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/controls/ControlFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/psdnet/PSDNetImpl.h"
#include "riglogic/psdnet/PSDNetImplOutputInstance.h"

TEST(PSDNetTest, PSDsAppendToOutput) {
    pma::AlignedMemoryResource amr;

    auto inputInstanceFactory = ControlsFactory::getInstanceFactory(0u, 2u, 2u, 0u, 0u);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &amr);
    auto inputs = inputInstance->getInputBuffer();
    inputs[0] = 0.1f;
    inputs[1] = 0.2f;

    rl4::PSDNetImplOutputInstance outputInstance{4u, &amr};

    rl4::Matrix<std::uint16_t> inputLODs{{0u, 1u}};
    rl4::Matrix<std::uint16_t> outputLODs{{2u, 3u}};
    rl4::Vector<std::uint16_t> cols{0u, 1u, 0u, 1u};
    // PSD weights {4.0f, 3.0f, 0.5f, 2.0f}
    rl4::Vector<rl4::PSD> psds{{0u, 2u, 12.0f}, {2u, 2u, 1.0f}};
    rl4::PSDNetImpl psdNet{std::move(inputLODs), std::move(outputLODs), std::move(cols), std::move(psds), 2u, 3u, nullptr};

    const float expected[] = {0.1f, 0.2f, 0.24f, 0.02f};
    psdNet.calculate(inputInstance.get(), &outputInstance, 0u);
    ASSERT_ELEMENTS_NEAR(inputs, expected, 4ul, 0.0001f);
}

TEST(PSDNetTest, OutputsAreClamped) {
    pma::AlignedMemoryResource amr;

    auto inputInstanceFactory = ControlsFactory::getInstanceFactory(0u, 1u, 1u, 0u, 0u);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &amr);
    auto inputs = inputInstance->getInputBuffer();
    inputs[0] = 0.1f;

    rl4::PSDNetImplOutputInstance outputInstance{2u, &amr};

    rl4::Matrix<std::uint16_t> inputLODs{{0u}};
    rl4::Matrix<std::uint16_t> outputLODs{{1u}};
    rl4::Vector<std::uint16_t> cols{0u};
    // PSD weights {100.0f}
    rl4::Vector<rl4::PSD> psds{{0u, 1u, 100.0f}};
    rl4::PSDNetImpl psdNet{std::move(inputLODs), std::move(outputLODs), std::move(cols), std::move(psds), 1u, 1u, nullptr};

    const float expected[] = {0.1f, 1.0f};
    psdNet.calculate(inputInstance.get(), &outputInstance, 0u);
    ASSERT_ELEMENTS_EQ(inputs, expected, 2);
}

TEST(PSDNetTest, OutputsKeepExistingProduct) {
    pma::AlignedMemoryResource amr;

    auto inputInstanceFactory = ControlsFactory::getInstanceFactory(0u, 2u, 1u, 0u, 0u);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &amr);
    auto inputs = inputInstance->getInputBuffer();
    inputs[0] = 0.1f;
    inputs[1] = 0.2f;

    rl4::PSDNetImplOutputInstance outputInstance{3u, &amr};

    rl4::Matrix<std::uint16_t> inputLODs{{0u, 1u}};
    rl4::Matrix<std::uint16_t> outputLODs{{2u}};
    rl4::Vector<std::uint16_t> cols{0u, 1u};
    // PSD weights {4.0f, 10.0f}
    rl4::Vector<rl4::PSD> psds{{0u, 2u, 40.0f}};
    rl4::PSDNetImpl psdNet{std::move(inputLODs), std::move(outputLODs), std::move(cols), std::move(psds), 2u, 2u, nullptr};

    const float expected[] = {0.1f, 0.2f, 0.8f};
    psdNet.calculate(inputInstance.get(), &outputInstance, 0u);
    ASSERT_ELEMENTS_EQ(inputs, expected, 3ul);
}

TEST(PSDNetTest, RowsSpecifyDestinationIndex) {
    pma::AlignedMemoryResource amr;

    auto inputInstanceFactory = ControlsFactory::getInstanceFactory(0u, 2u, 2u, 0u, 0u);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &amr);
    auto inputs = inputInstance->getInputBuffer();
    inputs[0] = 0.1f;
    inputs[1] = 0.2f;

    rl4::PSDNetImplOutputInstance outputInstance{4u, &amr};

    rl4::Matrix<std::uint16_t> inputLODs{{0u, 1u}};
    rl4::Matrix<std::uint16_t> outputLODs{{2u, 3u}};
    rl4::Vector<std::uint16_t> cols{0u, 1u};
    // PSD weights {4.0f, 3.0f}
    rl4::Vector<rl4::PSD> psds{{0u, 1u, 4.0f}, {1u, 1u, 3.0f}};
    rl4::PSDNetImpl psdNet{std::move(inputLODs), std::move(outputLODs), std::move(cols), std::move(psds), 2u, 2u, nullptr};

    const float expected[] = {0.1f, 0.2f, 0.4f, 0.6f};
    psdNet.calculate(inputInstance.get(), &outputInstance, 0u);
    ASSERT_ELEMENTS_EQ(inputs, expected, 4ul);
}
