// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/rbf/cpu/RBFFixtures.h"

namespace rltests {

namespace rbf {

#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wglobal-constructors"
    #pragma clang diagnostic ignored "-Wexit-time-destructors"
#endif

namespace unoptimized {

const std::uint16_t lodCount = 3u;

const std::uint16_t rawControlCount = 8u;

const pma::Matrix<std::uint16_t> solverIndicesPerLOD = {{0u, 1u}, {0u}, {}};

const pma::Vector<float> poseScales = {1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f};

const pma::Vector<dna::RBFSolverType> solverTypes = {dna::RBFSolverType::Interpolative, dna::RBFSolverType::Interpolative};

const pma::Vector<dna::RBFDistanceMethod> solverDistanceMethods = {dna::RBFDistanceMethod::SwingAngle,
                                                                   dna::RBFDistanceMethod::SwingAngle};

const pma::Vector<dna::RBFFunctionType> solverFunctionType = {dna::RBFFunctionType::Gaussian, dna::RBFFunctionType::Gaussian};

const pma::Vector<dna::RBFNormalizeMethod> solverNormalizeMethods = {dna::RBFNormalizeMethod::AlwaysNormalize,
                                                                     dna::RBFNormalizeMethod::AlwaysNormalize};

const pma::Vector<dna::TwistAxis> solverTwistAxis = {dna::TwistAxis::X, dna::TwistAxis::X};

const pma::Vector<dna::AutomaticRadius> solverAutomaticRadius = {dna::AutomaticRadius::On, dna::AutomaticRadius::On};

const pma::Vector<float> solverRadius = {0.0f, 0.0f};

const pma::Vector<float> solverWeightThreshold = {0.001f, 0.001f, 0.001f};

const pma::Matrix<std::uint16_t> solverPoseIndices = {{0u, 1u, 2u, 3u, 4u, 5u, 6u, 7u, 8u, 9u, 10u, 11u}, {0u, 12u, 13u, 14u}};

const pma::Matrix<std::uint16_t> solverRawControlIndices = {
    {0u, 1u, 2u, 3u},
    {4u, 5u, 6u, 7u},
};

const std::uint16_t poseControlCount = 16u;

const Matrix<std::uint16_t> poseInputControlIndices = {{}, {}, {}, {}, {}, {}, {}, {}, {}, {}, {}, {}, {}, {}, {1u}};

const Matrix<std::uint16_t> poseOutputControlIndices =
    {{8u}, {9u}, {10u}, {11u}, {12u}, {13u}, {14u}, {15u}, {16u}, {17u}, {18u}, {19u}, {20u}, {21u, 22u}, {22u, 23u}};

const Matrix<float> poseOutputControlWeights = {{1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f},
                                                {1.0f, 1.0f},
                                                {0.5f, 0.5f}};

const pma::Matrix<float> solverRawControlValues = {
    {0.000000000000000f,  0.000000000000000f,  0.000000000000000f,  1.000000000000000f,  -0.003081271657720f, -0.118239738047123f,
     -0.009329595603049f, 0.992936491966248f,  -0.008705757558346f, 0.009779179468751f,  -0.141530960798264f, 0.989847242832184f,
     0.026532903313637f,  -0.811531901359558f, -0.024197027087212f, 0.583203732967377f,  -0.000952127971686f, 0.013058164156973f,
     0.076173260807991f,  0.997008621692657f,  -0.044993601739407f, -0.664866507053375f, 0.044108338654041f,  0.744300007820129f,
     -0.005394733510911f, 0.099454566836357f,  -0.012115634977818f, 0.994953811168671f,  0.009781738743186f,  0.008702844381332f,
     0.372627735137939f,  0.927888572216034f,  -0.009282855316997f, 0.312406390905380f,  -0.014897738583386f, 0.949786365032196f,
     -0.003883346682414f, -0.450696706771851f, -0.001544478582218f, 0.892667353153229f,  -0.005706345662475f, -0.011783968657255f,
     -0.714682221412659f, 0.699326753616333f,  0.000949318520725f,  -0.013058470562100f, -0.825225293636322f, 0.564651966094971f},
    {0.0f,
     0.0f,
     0.0f,
     1.0f,
     -0.0936718f,
     -0.12003f,
     0.663135f,
     0.732851f,
     0.123443f,
     0.0891258f,
     -0.199695f,
     0.967957f,
     -0.12003f,
     0.0936719f,
     -0.732851f,
     0.663135f}};

}  // namespace unoptimized

namespace optimized {

const LODSpec<std::uint16_t> lods = {
    {{0, 1}, {0}, {}},
    2u  // Solver count
};

const Matrix<std::uint16_t> solverRawControlInputIndices = {{0u, 1u, 2u, 3u}, {4u, 5u, 6u, 7u}};

const Vector<std::uint16_t> inputCountPerSolver = {4u, 4u};
const Vector<std::uint16_t> targetCountPerSolver = {12u, 4u};

const Vector<Matrix<float>> targetValues = {{{0.0f, -0.0f, -0.0f, -1.0f},
                                             {0.0f, 0.118210219f, 0.00969646964f, -0.99294126f},
                                             {0.0f, -0.0110235251f, 0.141439483f, -0.989885509f},
                                             {0.0f, 0.811793089f, -0.0127105378f, -0.583806992f},
                                             {0.0f, -0.0129854148f, -0.0761857033f, -0.997009039f},
                                             {0.0f, 0.666316569f, -0.00390942581f, -0.745658696f},
                                             {0.0f, -0.0995188057f, 0.0115762129f, -0.994968414f},
                                             {0.0f, -0.0126303593f, -0.372515291f, -0.92794013f},
                                             {0.0f, -0.312537074f, 0.0118438303f, -0.949831724f},
                                             {0.0f, 0.45068571f, 0.0035050991f, -0.892675817f},
                                             {0.0f, 0.00595212681f, 0.714754522f, -0.699350059f},
                                             {0.0f, 0.0144458571f, 0.825202227f, -0.564652741f}},
                                            {{0.0f, -0.0f, -0.0f, -1.0f},
                                             {0.0f, 0.203138143f, -0.64256525f, -0.738813281f},
                                             {0.0f, -0.063147366f, 0.209365502f, -0.97579658f},
                                             {0.0f, -0.222702071f, 0.704449356f, -0.673910379f}}};

const Vector<Matrix<float>> coefficients = {{{10.4131117f,
                                              -2.50109625f,
                                              -1.68585944f,
                                              0.00226313062f,
                                              -4.21029043f,
                                              0.00851259567f,
                                              -2.70837259f,
                                              0.19591637f,
                                              0.28792578f,
                                              0.11638914f,
                                              0.0584293604f,
                                              0.00188750029f},
                                             {-2.50109625f,
                                              5.29161167f,
                                              -1.25280869f,
                                              0.0132418955f,
                                              -1.07533145f,
                                              0.0170967579f,
                                              0.630707622f,
                                              -0.224908382f,
                                              0.209685102f,
                                              -1.06077302f,
                                              -0.053258799f,
                                              0.0111311972f},
                                             {-1.68586028f,
                                              -1.25280893f,
                                              4.58401918f,
                                              0.00919136312f,
                                              0.76852268f,
                                              -0.00866004918f,
                                              -1.57939589f,
                                              0.237331465f,
                                              -0.384880483f,
                                              -0.169630617f,
                                              -0.486707956f,
                                              0.0163090285f},
                                             {0.00226316857f,
                                              0.0132418117f,
                                              0.00919136778f,
                                              2.45947075f,
                                              0.00964977313f,
                                              -1.88095915f,
                                              0.00817291997f,
                                              -0.0562427454f,
                                              0.0234976001f,
                                              0.0126816928f,
                                              -0.0217441544f,
                                              -0.0700526312f},
                                             {-4.210289f,
                                              -1.07533169f,
                                              0.768522859f,
                                              0.00964973494f,
                                              7.6117754f,
                                              0.00983878039f,
                                              -1.76371253f,
                                              -1.1520133f,
                                              -0.220182136f,
                                              -0.0616565943f,
                                              0.0662193298f,
                                              0.0089058429f},
                                             {0.00851258356f,
                                              0.0170967635f,
                                              -0.00866001379f,
                                              -1.88095927f,
                                              0.00983874034f,
                                              3.59918761f,
                                              0.013124981f,
                                              -0.0629767478f,
                                              0.022830274f,
                                              -1.55218458f,
                                              -0.0439120345f,
                                              -0.0208300687f},
                                             {-2.70837283f,
                                              0.630707622f,
                                              -1.57939577f,
                                              0.00817293487f,
                                              -1.76371217f,
                                              0.0131249651f,
                                              6.99775982f,
                                              -0.0686918572f,
                                              -1.6557734f,
                                              0.102362826f,
                                              0.00317596481f,
                                              0.00825250242f},
                                             {0.195916459f,
                                              -0.224908367f,
                                              0.237331495f,
                                              -0.0562427156f,
                                              -1.15201342f,
                                              -0.0629767776f,
                                              -0.0686918348f,
                                              2.05959892f,
                                              -0.373588949f,
                                              -0.214859918f,
                                              0.0606168285f,
                                              0.0199813042f},
                                             {0.287925869f,
                                              0.209685087f,
                                              -0.384880453f,
                                              0.0234975964f,
                                              -0.220182076f,
                                              0.0228302795f,
                                              -1.65577352f,
                                              -0.373588949f,
                                              2.60091233f,
                                              0.0772639886f,
                                              -0.170895383f,
                                              -0.0364022851f},
                                             {0.116389424f,
                                              -1.06077302f,
                                              -0.169630632f,
                                              0.0126816807f,
                                              -0.0616565533f,
                                              -1.55218446f,
                                              0.102362826f,
                                              -0.214859903f,
                                              0.0772639811f,
                                              2.94548035f,
                                              -0.107176304f,
                                              -0.00518302247f},
                                             {0.0584293269f,
                                              -0.0532588698f,
                                              -0.486707866f,
                                              -0.0217442084f,
                                              0.0662193596f,
                                              -0.0439120308f,
                                              0.00317585957f,
                                              0.0606168173f,
                                              -0.170895383f,
                                              -0.107176282f,
                                              3.21423078f,
                                              -2.36085558f},
                                             {0.00188759307f,
                                              0.0111312028f,
                                              0.0163090061f,
                                              -0.0700525865f,
                                              0.00890581496f,
                                              -0.0208300594f,
                                              0.00825257134f,
                                              0.0199813209f,
                                              -0.0364022776f,
                                              -0.00518305739f,
                                              -2.36085558f,
                                              2.9289515f}},
                                            {{4.09646654f, -0.869968235f, -3.05232024f, -6.16908073e-05f},
                                             {-0.869968295f, 1.50343823f, -3.51387607e-05f, 8.92129538e-06f},
                                             {-3.05232024f, -3.51387644e-05f, 4.27379131f, -1.06966126f},
                                             {-6.17128608e-05f, 8.92129538e-06f, -1.06966126f, 1.68079543f}}};

const pma::Vector<float> solverRadius = {1.29129f, 1.651938f};
const pma::Matrix<float> solverPoseScales = {{1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f},
                                             {1.0f, 1.0f, 1.0f, 1.0f}};

}  // namespace optimized

namespace input {

// Calculation input values
const rl4::Vector<float> values =
    {0.620591878890991f, 0.382426619529724f, -0.683809995651245f, 0.031930625438690f, 0.0f, 0.0f, -0.47362f, 0.880729f};

}  // namespace input

namespace output {

const Matrix<float> valuesPerLOD = {
    {
        0.0657254f,
        0.0f,
        0.0f,
        0.212102f,
        0.0116427f,
        0.0f,
        0.0f,
        0.279036f,
        0.473921f,
        0.0f,
        0.0033996f,
        0.0198993f,
        0.0f,
        0.453696f,
        0.5455889099f,
        0.09189290998017785f,
    },
    {0.0f, 0.0f, 0.0f, 0.212102f, 0.0116427f, 0.0f, 0.0f, 0.279036f, 0.473921f, 0.0f, 0.0033996f, 0.0198993f, 0.0f, 0.0f, 0.0f},
    {
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
        0.0f,
    }};

}  // namespace output

#ifdef __clang__
    #pragma clang diagnostic pop
#endif

RBFReader::~RBFReader() = default;

}  // namespace rbf

}  // namespace rltests
