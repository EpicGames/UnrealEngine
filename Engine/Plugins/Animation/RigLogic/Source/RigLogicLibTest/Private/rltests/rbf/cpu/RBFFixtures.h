// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "rltests/dna/FakeReader.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/types/LODSpec.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <functional>
#include <memory>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace rltests {

namespace rbf {

using namespace rl4;

namespace unoptimized {

extern const std::uint16_t rawControlCount;
extern const std::uint16_t lodCount;
extern const pma::Matrix<std::uint16_t> solverIndicesPerLOD;
extern const pma::Matrix<std::uint16_t> solverRawControlIndices;

extern const pma::Vector<float> poseScales;
extern const std::uint16_t poseControlCount;
extern const Matrix<std::uint16_t> poseInputControlIndices;
extern const Matrix<std::uint16_t> poseOutputControlIndices;
extern const Matrix<float> poseOutputControlWeights;
extern const pma::Vector<dna::RBFSolverType> solverTypes;
extern const pma::Vector<dna::RBFDistanceMethod> solverDistanceMethods;
extern const pma::Vector<dna::RBFFunctionType> solverFunctionType;
extern const pma::Vector<dna::RBFNormalizeMethod> solverNormalizeMethods;
extern const pma::Vector<dna::TwistAxis> solverTwistAxis;
extern const pma::Vector<dna::AutomaticRadius> solverAutomaticRadius;
extern const pma::Vector<float> solverRadius;
extern const pma::Vector<float> solverWeightThreshold;
extern const pma::Matrix<std::uint16_t> solverPoseIndices;
extern const pma::Matrix<float> solverRawControlValues;

}  // namespace unoptimized

namespace optimized {

extern const LODSpec<std::uint16_t> lods;
extern const Matrix<std::uint16_t> solverRawControlInputIndices;
extern const Vector<std::uint16_t> inputCountPerSolver;
extern const Vector<std::uint16_t> targetCountPerSolver;

extern const Vector<Matrix<float>> targetValues;
extern const Vector<Matrix<float>> coefficients;
extern const pma::Vector<float> solverRadius;
extern const pma::Matrix<float> solverPoseScales;

}  // namespace optimized

namespace input {

// Calculation input values
extern const Vector<float> values;

}  // namespace input

namespace output {

// Calculation output values
extern const Matrix<float> valuesPerLOD;

}  // namespace output

class RBFReader : public dna::FakeReader {
public:
    ~RBFReader();

    std::uint16_t getLODCount() const override {
        return unoptimized::lodCount;
    }

    std::uint16_t getRawControlCount() const override {
        return unoptimized::rawControlCount;
    }

    std::uint16_t getRBFPoseCount() const override {
        return static_cast<std::uint16_t>(unoptimized::poseScales.size());
    }

    float getRBFPoseScale(std::uint16_t poseIndex) const override {
        return unoptimized::poseScales[poseIndex];
    }

    std::uint16_t getRBFSolverCount() const override {
        return static_cast<std::uint16_t>(unoptimized::solverTypes.size());
    }

    ConstArrayView<std::uint16_t> getRBFSolverIndicesForLOD(std::uint16_t lod) const override {
        return unoptimized::solverIndicesPerLOD[lod];
    }

    ConstArrayView<std::uint16_t> getRBFSolverRawControlIndices(std::uint16_t solverIndex) const override {
        return unoptimized::solverRawControlIndices[solverIndex];
    }

    ConstArrayView<std::uint16_t> getRBFSolverPoseIndices(std::uint16_t solverIndex) const override {
        return unoptimized::solverPoseIndices[solverIndex];
    }

    ConstArrayView<float> getRBFSolverRawControlValues(std::uint16_t solverIndex) const override {
        return unoptimized::solverRawControlValues[solverIndex];
    }

    dna::RBFSolverType getRBFSolverType(std::uint16_t solverIndex) const override {
        return unoptimized::solverTypes[solverIndex];
    }

    float getRBFSolverRadius(std::uint16_t solverIndex) const override {
        return unoptimized::solverRadius[solverIndex];
    }

    dna::AutomaticRadius getRBFSolverAutomaticRadius(std::uint16_t solverIndex) const override {
        return unoptimized::solverAutomaticRadius[solverIndex];
    }

    float getRBFSolverWeightThreshold(std::uint16_t solverIndex) const override {
        return unoptimized::solverWeightThreshold[solverIndex];
    }

    dna::RBFDistanceMethod getRBFSolverDistanceMethod(std::uint16_t solverIndex) const override {
        return unoptimized::solverDistanceMethods[solverIndex];
    }

    dna::RBFNormalizeMethod getRBFSolverNormalizeMethod(std::uint16_t solverIndex) const override {
        return unoptimized::solverNormalizeMethods[solverIndex];
    }

    dna::RBFFunctionType getRBFSolverFunctionType(std::uint16_t solverIndex) const override {
        return unoptimized::solverFunctionType[solverIndex];
    }

    dna::TwistAxis getRBFSolverTwistAxis(std::uint16_t solverIndex) const override {
        return unoptimized::solverTwistAxis[solverIndex];
    }

    std::uint16_t getRBFPoseControlCount() const override {
        return static_cast<std::uint16_t>(unoptimized::poseControlCount);
    }

    ConstArrayView<std::uint16_t> getRBFPoseInputControlIndices(std::uint16_t poseIndex) const override {
        return unoptimized::poseInputControlIndices[poseIndex];
    }

    ConstArrayView<std::uint16_t> getRBFPoseOutputControlIndices(std::uint16_t poseIndex) const override {
        return unoptimized::poseOutputControlIndices[poseIndex];
    }

    ConstArrayView<float> getRBFPoseOutputControlWeights(std::uint16_t poseIndex) const override {
        return unoptimized::poseOutputControlWeights[poseIndex];
    }
};

}  // namespace rbf

}  // namespace rltests
