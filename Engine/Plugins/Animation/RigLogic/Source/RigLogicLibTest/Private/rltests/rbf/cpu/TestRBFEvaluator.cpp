// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/StorageValueType.h"
#include "rltests/controls/ControlFixtures.h"
#include "rltests/rbf/cpu/RBFFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/RBFBehaviorEvaluator.h"
#include "riglogic/rbf/cpu/CPURBFBehaviorFactory.h"
#include "riglogic/riglogic/RigMetadata.h"
#include "riglogic/system/simd/Detect.h"
#include "riglogic/system/simd/SIMD.h"

#include <tuple>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace {

template<typename TTestTypes>
class RBFBehaviorTest : public ::testing::Test {
protected:
    void SetUp() override {
        RBFBehaviorTest::SetUpImpl();
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value != 0ul, void>::type SetUpImpl() {
        using T = typename std::tuple_element<0, TestTypes>::type;
        using TF256 = typename std::tuple_element<1, TestTypes>::type;
        using TF128 = typename std::tuple_element<2, TestTypes>::type;
        rl4::Configuration config = {};
        auto meta = rl4::RigMetadata::create(config, &reader, &memRes, rl4::InitializationMethod::Create);
        evaluator = rl4::rbf::cpu::Factory<T, TF256, TF128>::create(meta.get(), &reader, &memRes);
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value == 0ul, void>::type SetUpImpl() {
        GTEST_SKIP();
    }

protected:
    pma::AlignedMemoryResource memRes;
    rltests::rbf::RBFReader reader;
    rl4::RBFBehaviorEvaluator::Pointer evaluator;
};

}  // namespace

#if defined(RL_BUILD_WITH_AVX) && defined(RL_BUILD_WITH_SSE)
using RBFEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128>
    #else
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128>,
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128>,
    std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_AVX)
using RBFEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128>
    #else
    std::tuple<StorageValueType, trimd::avx::F256, trimd::sse::F128>,
    std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_SSE)
using RBFEvaluatorTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128>
    #else
    std::tuple<StorageValueType, trimd::sse::F256, trimd::sse::F128>,
    std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#else
    #ifndef RL_BUILD_WITH_HALF_FLOATS
using RBFEvaluatorTypeList = ::testing::Types<std::tuple<StorageValueType, trimd::scalar::F256, trimd::scalar::F128>>;
    #else
using RBFEvaluatorTypeList = ::testing::Types<std::tuple<>>;
    #endif  // RL_BUILD_WITH_HALF_FLOATS
#endif

TYPED_TEST_SUITE(RBFBehaviorTest, RBFEvaluatorTypeList, );

TYPED_TEST(RBFBehaviorTest, SolverPerLOD) {
    auto inputInstanceFactory = ControlsFactory::getInstanceFactory(0,
                                                                    rltests::rbf::unoptimized::rawControlCount,
                                                                    0,
                                                                    0,
                                                                    rltests::rbf::unoptimized::poseControlCount);
    rl4::Vector<rl4::ControlInitializer> initialValues;
    auto inputInstance = inputInstanceFactory(initialValues, &this->memRes);
    auto inputBuffer = inputInstance->getInputBuffer();
    auto outputBuffer =
        inputBuffer.subview(rltests::rbf::unoptimized::rawControlCount, rltests::rbf::unoptimized::poseControlCount);
    const auto& inputValues = rltests::rbf::input::values;
    for (std::size_t i = {}; i < inputValues.size(); ++i) {
        inputBuffer[i] = inputValues[i];
    }

    auto intermediateOutputs = this->evaluator->createInstance(&this->memRes);

    for (std::uint16_t lod = 0u; lod < rltests::rbf::unoptimized::lodCount; ++lod) {
        std::fill(outputBuffer.begin(), outputBuffer.end(), 0.0f);
        this->evaluator->calculate(inputInstance.get(), intermediateOutputs.get(), lod);
        // Call twice to make sure the output control values are not accumulating between calls
        this->evaluator->calculate(inputInstance.get(), intermediateOutputs.get(), lod);
        const auto& expected = rltests::rbf::output::valuesPerLOD[lod];
#ifdef RL_BUILD_WITH_HALF_FLOATS
        static constexpr float threshold = 0.05f;
#else
        static constexpr float threshold = 0.0001f;
#endif  // RL_BUILD_WITH_HALF_FLOATS
        ASSERT_ELEMENTS_NEAR(outputBuffer, expected, expected.size(), threshold);
    }
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
