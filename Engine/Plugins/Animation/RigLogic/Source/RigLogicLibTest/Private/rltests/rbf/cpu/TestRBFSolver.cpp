// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"

#include "riglogic/rbf/cpu/RBFSolver.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace {

class RBFSolverTest : public ::testing::Test {
protected:
    void SetUp() override {
        recipe.normalizeMethod = rl4::RBFNormalizeMethod::AlwaysNormalize;
        recipe.isAutomaticRadius = true;
        recipe.radius = 0.0f;
        recipe.twistAxis = dna::TwistAxis::X;
        recipe.weightThreshold = 0.001f;
    }

protected:
    pma::AlignedMemoryResource memRes;
    rl4::RBFSolverRecipe recipe;
};

}  // namespace

TEST_F(RBFSolverTest, InterpolativeGaussianSwingAngle0) {
    recipe.solverType = rl4::RBFSolverType::Interpolative;
    recipe.distanceMethod = rl4::RBFDistanceMethod::SwingAngle;
    recipe.weightFunction = rl4::RBFFunctionType::Gaussian;
    std::size_t targetCount = 4;
    pma::Vector<float> scaleFactors{targetCount, 1.0f, &memRes};
    recipe.targetScales = scaleFactors;
    recipe.rawControlCount = 4u;
    pma::Vector<float> targetValues{0.0f,
                                    0.0f,
                                    0.0f,
                                    1.0f,
                                    -0.0936718f,
                                    -0.12003f,
                                    0.663135f,
                                    0.732851f,
                                    0.123443f,
                                    0.0891258f,
                                    -0.199695f,
                                    0.967957f,
                                    -0.12003f,
                                    0.0936719f,
                                    -0.732851f,
                                    0.663135f};
    recipe.targetValues = targetValues;
    auto solver = rl4::RBFSolver::create(recipe, &memRes);
    pma::Vector<float> input{-0.0936718f, -0.12003f, 0.663135f, 0.732851f};
    pma::Vector<float> buffer{targetCount, 0.0f, &memRes};
    pma::Vector<float> result{targetCount, 0.0f, &memRes};

    solver->solve(input, buffer, result);
    pma::Vector<float> expected{0.0f, 1.0f, 0.0f, 0.0f};
    EXPECT_ELEMENTS_NEAR(result, expected, targetCount, 0.0001f);
    input = {0.0f, 0.0f, -0.47362f, 0.880729f};
    solver->solve(input, buffer, result);
    expected = {0.0657254f, 0.0f, 0.453696f, 0.480578f};
    EXPECT_ELEMENTS_NEAR(result, expected, targetCount, 0.0001f);
}

TEST_F(RBFSolverTest, InterpolativeGaussianSwingAngle1) {
    recipe.solverType = rl4::RBFSolverType::Interpolative;
    recipe.distanceMethod = rl4::RBFDistanceMethod::SwingAngle;
    recipe.weightFunction = rl4::RBFFunctionType::Gaussian;
    std::size_t targetCount = 12u;
    pma::Vector<float> scaleFactors{targetCount, 1.0f, &memRes};
    recipe.targetScales = scaleFactors;
    recipe.rawControlCount = 4u;
    pma::Vector<float> targetValues{
        0.000000000000000f,  0.000000000000000f,  0.000000000000000f,  1.000000000000000f,  -0.003081271657720f,
        -0.118239738047123f, -0.009329595603049f, 0.992936491966248f,  -0.008705757558346f, 0.009779179468751f,
        -0.141530960798264f, 0.989847242832184f,  0.026532903313637f,  -0.811531901359558f, -0.024197027087212f,
        0.583203732967377f,  -0.000952127971686f, 0.013058164156973f,  0.076173260807991f,  0.997008621692657f,
        -0.044993601739407f, -0.664866507053375f, 0.044108338654041f,  0.744300007820129f,  -0.005394733510911f,
        0.099454566836357f,  -0.012115634977818f, 0.994953811168671f,  0.009781738743186f,  0.008702844381332f,
        0.372627735137939f,  0.927888572216034f,  -0.009282855316997f, 0.312406390905380f,  -0.014897738583386f,
        0.949786365032196f,  -0.003883346682414f, -0.450696706771851f, -0.001544478582218f, 0.892667353153229f,
        -0.005706345662475f, -0.011783968657255f, -0.714682221412659f, 0.699326753616333f,  0.000949318520725f,
        -0.013058470562100f, -0.825225293636322f, 0.564651966094971f};

    recipe.targetValues = targetValues;
    auto solver = rl4::RBFSolver::create(recipe, &memRes);
    pma::Vector<float> input{-0.005706345662475f, -0.011783968657255f, -0.714682221412659f, 0.699326753616333f};
    pma::Vector<float> buffer{targetCount, 0.0f, &memRes};
    pma::Vector<float> result{targetCount, 0.0f, &memRes};

    solver->solve(input, buffer, result);
    pma::Vector<float> expected{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f};
    EXPECT_ELEMENTS_NEAR(result, expected, targetCount, 0.0001f);

    input = {0.620591878890991f, 0.382426619529724f, -0.683809995651245f, 0.031930625438690f};
    solver->solve(input, buffer, result);
    expected = {0.0f, 0.0f, 0.0f, 0.212102f, 0.0116427f, 0.0f, 0.0f, 0.279036f, 0.473921f, 0.0f, 0.0033996f, 0.0198993f};

    EXPECT_ELEMENTS_NEAR(result, expected, targetCount, 0.0001f);
}

TEST_F(RBFSolverTest, AdditiveGaussianSwingAngle0) {
    recipe.solverType = rl4::RBFSolverType::Additive;
    recipe.distanceMethod = rl4::RBFDistanceMethod::SwingAngle;
    recipe.weightFunction = rl4::RBFFunctionType::Gaussian;
    std::size_t targetCount = 4;
    pma::Vector<float> scaleFactors{targetCount, 1.0f, &memRes};
    recipe.targetScales = scaleFactors;
    recipe.rawControlCount = 4u;
    pma::Vector<float> targetValues{0.0f,
                                    0.0f,
                                    0.0f,
                                    1.0f,
                                    -0.0936718f,
                                    -0.12003f,
                                    0.663135f,
                                    0.732851f,
                                    0.123443f,
                                    0.0891258f,
                                    -0.199695f,
                                    0.967957f,
                                    -0.12003f,
                                    0.0936719f,
                                    -0.732851f,
                                    0.663135f};
    recipe.targetValues = targetValues;
    auto solver = rl4::RBFSolver::create(recipe, &memRes);
    pma::Vector<float> input{-0.0936718f, -0.12003f, 0.663135f, 0.732851f};
    pma::Vector<float> buffer{targetCount, 0.0f, &memRes};
    pma::Vector<float> result{targetCount, 0.0f, &memRes};

    solver->solve(input, buffer, result);
    pma::Vector<float> expected{0.242209f, 0.41645f, 0.20938f, 0.131961f};

    EXPECT_ELEMENTS_NEAR(result, expected, targetCount, 0.0001f);
}
