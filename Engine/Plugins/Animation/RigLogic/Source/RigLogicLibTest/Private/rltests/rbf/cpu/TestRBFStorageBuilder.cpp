// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/joints/Helpers.h"
#include "rltests/rbf/cpu/RBFFixtures.h"

#include "riglogic/TypeDefs.h"
#include "riglogic/rbf/cpu/CPURBFBehaviorFactory.h"
#include "riglogic/rbf/cpu/RBFSolver.h"
#include "riglogic/riglogic/RigMetadata.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4324)
#endif

namespace rl4 {

namespace rbf {

namespace cpu {

template<typename T, typename TF256, typename TF128>
struct Evaluator<T, TF256, TF128>::Accessor {

    static void assertRawDataEqual(const Evaluator<T, TF256, TF128>& result) {
        ASSERT_EQ(result.lods.indicesPerLOD, rltests::rbf::optimized::lods.indicesPerLOD);
        ASSERT_EQ(result.lods.count, rltests::rbf::optimized::lods.count);
        ASSERT_EQ(result.solvers.size(), rltests::rbf::optimized::lods.count);
        ASSERT_EQ(result.inputCountPerSolver, rltests::rbf::optimized::inputCountPerSolver);
        ASSERT_EQ(result.targetCountPerSolver, rltests::rbf::optimized::targetCountPerSolver);

        for (std::uint16_t si = {}; si < rltests::rbf::unoptimized::solverTypes.size(); ++si) {
            const auto& solver = result.solvers[si];
            ASSERT_EQ(solver->getSolverType(), rltests::rbf::unoptimized::solverTypes[si]);
            ASSERT_EQ(solver->getTargetScales(), ConstArrayView<float>(rltests::rbf::optimized::solverPoseScales[si]));
            ASSERT_NEAR(solver->getRadius(), rltests::rbf::optimized::solverRadius[si], 0.001f);
            ASSERT_EQ(solver->getWeightThreshold(), rltests::rbf::unoptimized::solverWeightThreshold[si]);
            ASSERT_EQ(solver->getDistanceMethod(), rltests::rbf::unoptimized::solverDistanceMethods[si]);
            ASSERT_EQ(solver->getWeightFunction(), rltests::rbf::unoptimized::solverFunctionType[si]);
            ASSERT_EQ(solver->getNormalizeMethod(), rltests::rbf::unoptimized::solverNormalizeMethods[si]);
            ASSERT_EQ(solver->getTwistAxis(), rltests::rbf::unoptimized::solverTwistAxis[si]);

            ASSERT_EQ(result.solverPoseIndices[si], rltests::rbf::unoptimized::solverPoseIndices[si]);
        }
        for (std::uint16_t pi = {}; pi < rltests::rbf::unoptimized::poseScales.size(); ++pi) {
            ASSERT_EQ(result.poseInputControlIndices[pi], rltests::rbf::unoptimized::poseInputControlIndices[pi]);
            ASSERT_EQ(result.poseOutputControlIndices[pi], rltests::rbf::unoptimized::poseOutputControlIndices[pi]);
            ASSERT_EQ(result.poseOutputControlWeights[pi], rltests::rbf::unoptimized::poseOutputControlWeights[pi]);
        }
    }
};

}  // namespace cpu

}  // namespace rbf

}  // namespace rl4

namespace {

template<typename TTestTypes>
class RBFStorageBuilderTest : public ::testing::Test {
protected:
    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value != 0ul, void>::type buildStorage() {
        using T = typename std::tuple_element<0, TTestTypes>::type;
        using TF256 = typename std::tuple_element<1, TTestTypes>::type;
        using TF128 = typename std::tuple_element<2, TTestTypes>::type;
        rl4::Configuration config = {};
        auto meta = rl4::RigMetadata::create(config, &reader, &memRes, rl4::InitializationMethod::Create);
        auto evaluator = rl4::rbf::cpu::Factory<T, TF256, TF128>::create(meta.get(), &reader, &memRes);
        auto evaluatorImpl = static_cast<rl4::rbf::cpu::Evaluator<T, TF256, TF128>*>(evaluator.get());
        rl4::rbf::cpu::Evaluator<T, TF256, TF128>::Accessor::assertRawDataEqual(*evaluatorImpl);
    }

    template<typename TestTypes = TTestTypes>
    typename std::enable_if<std::tuple_size<TestTypes>::value == 0ul, void>::type buildStorage() {
    }

protected:
    pma::AlignedMemoryResource memRes;
    rltests::rbf::RBFReader reader;
};

}  // namespace

#if defined(RL_BUILD_WITH_AVX) && defined(RL_BUILD_WITH_SSE)
using StorageValueTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::avx::F256, trimd::sse::F128>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128>
    #else
    std::tuple<float, trimd::avx::F256, trimd::sse::F128>,
    std::tuple<float, trimd::sse::F256, trimd::sse::F128>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_AVX)
using StorageValueTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::avx::F256, trimd::sse::F128>
    #else
    std::tuple<float, trimd::avx::F256, trimd::sse::F128>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#elif defined(RL_BUILD_WITH_SSE)
using StorageValueTypeList = ::testing::Types<
    #ifdef RL_BUILD_WITH_HALF_FLOATS
    std::tuple<float, trimd::sse::F256, trimd::sse::F128>
    #else
    std::tuple<float, trimd::sse::F256, trimd::sse::F128>,
    std::tuple<float, trimd::scalar::F256, trimd::scalar::F128>
    #endif  // RL_BUILD_WITH_HALF_FLOATS
    >;
#else
    #ifndef RL_BUILD_WITH_HALF_FLOATS
using StorageValueTypeList = ::testing::Types<std::tuple<float, trimd::scalar::F256, trimd::scalar::F128>>;
    #else
using StorageValueTypeList = ::testing::Types<std::tuple<>>;
    #endif  // RL_BUILD_WITH_HALF_FLOATS
#endif

TYPED_TEST_SUITE(RBFStorageBuilderTest, StorageValueTypeList, );

TYPED_TEST(RBFStorageBuilderTest, LayoutOptimization) {
    this->buildStorage();
}

#ifdef _MSC_VER
    #pragma warning(pop)
#endif
