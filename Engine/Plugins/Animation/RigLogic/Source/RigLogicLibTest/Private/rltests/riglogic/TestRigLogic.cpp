// Copyright Epic Games, Inc. All Rights Reserved.

#include "rltests/Defs.h"
#include "rltests/dna/DNAFixtures.h"

#include "riglogic/RigLogic.h"
#include "riglogic/TypeDefs.h"
#include "riglogic/utils/Extd.h"

namespace {

class RigLogicTest : public ::testing::Test {
protected:
    void SetUp() override {
        const auto bytes = rltests::raw::getBytes();
        stream = pma::makeScoped<trio::MemoryStream>();
        stream->write(bytes.data(), bytes.size());
        stream->seek(0);

        reader = pma::makeScoped<dna::BinaryStreamReader>(stream.get());
        reader->read();

        rigLogic = pma::makeScoped<rl4::RigLogic>(reader.get());
        rigInstance = pma::makeScoped<rl4::RigInstance>(rigLogic.get(), &memRes);
    }

protected:
    pma::AlignedMemoryResource memRes;
    pma::ScopedPtr<trio::MemoryStream> stream;
    pma::ScopedPtr<dna::BinaryStreamReader> reader;
    pma::ScopedPtr<rl4::RigLogic> rigLogic;
    pma::ScopedPtr<rl4::RigInstance> rigInstance;
};

}  // namespace

TEST_F(RigLogicTest, EvaluateRigInstance) {
    // Try both approaches
    const float guiControls[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f};
    auto guiBuffer = rigInstance->getGUIControlValues();
    extd::copy(rl4::ConstArrayView<float>{guiControls, 9ul}, guiBuffer);
    rigLogic->mapGUIToRawControls(rigInstance.get());
    // Regardless that this overwrites the computed gui to raw values
    float rawControls[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f};
    auto rawBuffer = rigInstance->getRawControlValues();
    extd::copy(rl4::ConstArrayView<float>{rawControls, 9ul}, rawBuffer);

    rigLogic->calculate(rigInstance.get());

    ASSERT_EQ(rigInstance->getJointOutputs().size(), reader->getJointRowCount());
    ASSERT_EQ(rigInstance->getBlendShapeOutputs().size(), reader->getBlendShapeChannelCount());
    ASSERT_EQ(rigInstance->getAnimatedMapOutputs().size(), reader->getAnimatedMapCount());
}

TEST_F(RigLogicTest, AccessJointVariableAttributeIndices) {
    for (std::uint16_t lod = 0u; lod < rigLogic->getLODCount(); ++lod) {
        auto actual = rigLogic->getJointVariableAttributeIndices(lod);
        auto expected = rl4::ConstArrayView<std::uint16_t>{rltests::decoded::jointVariableIndices[0ul][lod]};
        ASSERT_EQ(actual.size(), expected.size());
        // Since implementation relies on std::set which has different implementations across compilers we cannot guarantee order
        // of elements
        for (const auto attrIndex : expected) {
            ASSERT_NE(std::find(actual.begin(), actual.end(), attrIndex), actual.end());
        }
    }
}

TEST_F(RigLogicTest, DumpStateThenRestore) {
    const float guiControls[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f};

    auto dumpedState = pma::makeScoped<trio::MemoryStream>();
    rigLogic->dump(dumpedState.get());
    dumpedState->seek(0);
    auto cloneRigLogic = rl4::RigLogic::restore(dumpedState.get(), &memRes);
    auto cloneRigInstance = rl4::RigInstance::create(cloneRigLogic, &memRes);

    for (std::uint16_t lod = 0u; lod < rigLogic->getLODCount(); ++lod) {
        rigInstance->setLOD(lod);
        auto guiBuffer = rigInstance->getGUIControlValues();
        extd::copy(rl4::ConstArrayView<float>{guiControls, 9ul}, guiBuffer);
        rigLogic->mapGUIToRawControls(rigInstance.get());
        rigLogic->calculate(rigInstance.get());

        cloneRigInstance->setLOD(lod);
        auto cloneGuiBuffer = cloneRigInstance->getGUIControlValues();
        extd::copy(rl4::ConstArrayView<float>{guiControls, 9ul}, cloneGuiBuffer);
        cloneRigLogic->mapGUIToRawControls(cloneRigInstance);
        cloneRigLogic->calculate(cloneRigInstance);

        auto origJointOutputs = rigInstance->getJointOutputs();
        auto origBlendShapeOutputs = rigInstance->getBlendShapeOutputs();
        auto origAnimatedMapOutputs = rigInstance->getAnimatedMapOutputs();

        auto cloneJointOutputs = cloneRigInstance->getJointOutputs();
        auto cloneBlendShapeOutputs = cloneRigInstance->getBlendShapeOutputs();
        auto cloneAnimatedMapOutputs = cloneRigInstance->getAnimatedMapOutputs();

        ASSERT_EQ(origJointOutputs, cloneJointOutputs);
        ASSERT_EQ(origBlendShapeOutputs, cloneBlendShapeOutputs);
        ASSERT_EQ(origAnimatedMapOutputs, cloneAnimatedMapOutputs);
    }

    rl4::RigInstance::destroy(cloneRigInstance);
    rl4::RigLogic::destroy(cloneRigLogic);
}

TEST_F(RigLogicTest, JointOutputBufferInitialized) {
    rl4::Configuration config{};
    config.rotationType = rl4::RotationType::Quaternions;
    auto qRigLogic = pma::makeScoped<rl4::RigLogic>(reader.get(), config);
    auto qRigInstance = pma::makeScoped<rl4::RigInstance>(qRigLogic.get());
    auto jointOutputs = qRigInstance->getJointOutputs();
    static constexpr std::size_t qwOffset = 6;
    static constexpr std::size_t jointAttrCount = 10;
    for (std::size_t i = {}; i < jointOutputs.size(); ++i) {
        if (i % jointAttrCount == qwOffset) {
            ASSERT_EQ(jointOutputs[i], 1.0f);
        } else {
            ASSERT_EQ(jointOutputs[i], 0.0f);
        }
    }
}
