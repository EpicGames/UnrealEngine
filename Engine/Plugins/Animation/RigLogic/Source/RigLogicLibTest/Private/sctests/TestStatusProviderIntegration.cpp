// Copyright Epic Games, Inc. All Rights Reserved.

#include "sctests/Defs.h"

#include "status/Provider.h"
#include "status/Status.h"

#include <cstring>

static sc::StatusCode NotOk{10, "Uh-oh!"};
static sc::StatusCode WithData{20, "Some data %zu with string %s"};

TEST(StatusProviderIntegrationTest, SetAndCheckStatus) {
    sc::StatusProvider provider{NotOk};
    ASSERT_TRUE(sc::Status::isOk());
    provider.set(NotOk);
    ASSERT_FALSE(sc::Status::isOk());
    ASSERT_EQ(sc::Status::get(), NotOk);
    provider.reset();
    ASSERT_TRUE(sc::Status::isOk());
}

const char* translate(sc::StatusCode status, std::size_t index, const char* data) {
    if (status == WithData) {
        if (index == 0ul) {
            return "Translated message %zu with translated args %s";
        } else if (index == 2ul) {
            return "i18n arg";
        }
    } else if (status == NotOk) {
        return "Translated not ok";
    }
    return data;
}

TEST(StatusProviderIntegrationTest, UsePreSetHook) {
    sc::StatusProvider provider{WithData};
    ASSERT_EQ(sc::Status::getHook(), nullptr);
    sc::Status::setHook(translate);
    ASSERT_TRUE(sc::Status::getHook() == translate);

    provider.set(WithData, static_cast<std::size_t>(42), "arg to be translated");
    auto status = sc::Status::get();
    ASSERT_EQ(status, WithData);
    ASSERT_STREQ(status.message, "Translated message 42 with translated args i18n arg");

    provider.set(NotOk);
    status = sc::Status::get();
    ASSERT_EQ(status, NotOk);
    ASSERT_STREQ(status.message, "Translated not ok");

    sc::Status::setHook(nullptr);
}
