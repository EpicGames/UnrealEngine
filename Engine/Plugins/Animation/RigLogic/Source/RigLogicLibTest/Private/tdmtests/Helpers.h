// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define ASSERT_VEC_NEAR(v1, v2, threshold)                                                                                       \
    ASSERT_EQ(v1.dimensions(), v2.dimensions());                                                                                 \
    for (tdm::dim_t ri = 0u; ri < v1.dimensions(); ++ri) {                                                                       \
        ASSERT_NEAR(v1[ri], v2[ri], threshold);                                                                                  \
    }

#define ASSERT_MAT_NEAR(m1, m2, threshold)                                                                                       \
    ASSERT_EQ(m1.rows(), m2.rows());                                                                                             \
    ASSERT_EQ(m1.columns(), m2.columns());                                                                                       \
    for (tdm::dim_t ri = 0u; ri < m1.rows(); ++ri) {                                                                             \
        for (tdm::dim_t ci = 0u; ci < m1.columns(); ++ci) {                                                                      \
            ASSERT_NEAR(m1[ri][ci], m2[ri][ci], threshold);                                                                      \
        }                                                                                                                        \
    }
