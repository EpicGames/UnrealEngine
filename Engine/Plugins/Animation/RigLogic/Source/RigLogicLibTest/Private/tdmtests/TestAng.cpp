// Copyright Epic Games, Inc. All Rights Reserved.

#include "tdmtests/Defs.h"
#include "tdmtests/Helpers.h"

#include "tdm/TDM.h"

TEST(AngTestConstruction, FromRadians) {
    const tdm::frad r{1.570796f};
    const float expected = 90.0f;
    ASSERT_NEAR(expected, tdm::fdeg{r}.value, 1e-4f);
}

TEST(AngTestConstruction, FromDegrees) {
    const tdm::fdeg d{-270.0f};
    const float expected = -4.712389f;
    ASSERT_NEAR(expected, tdm::frad{d}.value, 1e-4f);
}

TEST(AngTestConstruction, CopyConstructionRadians) {
    const tdm::frad r{1.570796f};
    const float expected = 1.570796f;
    ASSERT_NEAR(expected, tdm::frad{r}.value, 1e-4f);
}

TEST(AngTestConstruction, CopyConstructionDegrees) {
    const tdm::fdeg d{-270.0f};
    const float expected = -270.0f;
    ASSERT_NEAR(expected, tdm::fdeg{d}.value, 1e-4f);
}

TEST(AngTestOperators, CompoundAssignmentAdd) {
    tdm::frad r1{1.570796f};
    const tdm::frad r2{-1.570796f};
    const float expected = 0.0f;
    r1 += r2;
    ASSERT_NEAR(expected, tdm::fdeg{r1}.value, 1e-4f);
}

TEST(AngTestOperators, CompoundAssignmentSubtract) {
    tdm::frad r1{1.570796f};
    const tdm::frad r2{-1.570796f};
    const float expected = 180.0f;
    r1 -= r2;
    ASSERT_NEAR(expected, tdm::fdeg{r1}.value, 1e-4f);
}

TEST(AngTestOperators, CompoundAssignmentMultiplyScalar) {
    tdm::frad r1{1.570796f};
    const float expected = 180.0f;
    r1 *= 2.0f;
    ASSERT_NEAR(expected, tdm::fdeg{r1}.value, 1e-4f);
}

TEST(AngTestOperators, CompoundAssignmentDivideScalar) {
    tdm::frad r1{1.570796f};
    const float expected = 45.0f;
    r1 /= 2.0f;
    ASSERT_NEAR(expected, tdm::fdeg{r1}.value, 1e-4f);
}

TEST(AngTestLiterals, LiteralRadians) {
    using namespace tdm::ang_literals;
    const auto r = 1.570796_frad;
    const float expected = 90.0f;
    ASSERT_NEAR(expected, tdm::fdeg{r}.value, 1e-4f);
}

TEST(AngTestLiterals, LiteralDegrees) {
    using namespace tdm::ang_literals;
    const auto d = -270.0_fdeg;
    const float expected = -4.712389f;
    ASSERT_NEAR(expected, tdm::frad{d}.value, 1e-4f);
}

TEST(AngTestLiterals, SumDegreesRadians) {
    using namespace tdm::ang_literals;
    const auto a = -270.0_fdeg;
    const auto b = 1.570796_frad;
    const auto expected = -180.0_fdeg;
    ASSERT_NEAR(tdm::frad{expected}.value, tdm::frad{tdm::frad{a} + b}.value, 1e-4f);
    ASSERT_NEAR(expected.value, tdm::fdeg{a + tdm::fdeg{b}}.value, 1e-4f);
}

TEST(AngTest, SumRotations) {
    using namespace tdm::ang_literals;
    const auto ang1 = tdm::fdeg3{90.0_fdeg, -270.0_fdeg, -90.0_fdeg};
    const auto ang2 = tdm::frad3{1.570796_frad, -4.712388_frad, -1.570796_frad};
    const auto expected = tdm::fdeg3{180.0_fdeg, -540.0_fdeg, -180.0_fdeg};
    tdm::fdeg3 sum;
    for (std::size_t i = 0; i < sum.dimensions(); ++i) {
        sum[i] = ang1[i] + tdm::fdeg{ang2[i]};
        ASSERT_NEAR(tdm::frad{expected[i]}.value, tdm::frad{sum[i]}.value, 1e-4f);
        ASSERT_NEAR(expected[i].value, sum[i].value, 1e-4f);
    }
}
