// Copyright Epic Games, Inc. All Rights Reserved.

#include "tdmtests/Defs.h"
#include "tdmtests/Helpers.h"

#include "tdm/TDM.h"

TEST(ComputationsTest, NegateVec) {
    tdm::vec4<int> v{-2, -5, 1, 3};
    ASSERT_EQ(-v, tdm::vec4<int>(2, 5, -1, -3));
    ASSERT_EQ(tdm::negate(v), tdm::vec4<int>(2, 5, -1, -3));
}

TEST(ComputationsTest, NegateQuat) {
    tdm::quat<float> q{0.2f, 0.3f, -0.1f, -0.5f};
    ASSERT_EQ(-q, tdm::quat<float>(-0.2f, -0.3f, 0.1f, 0.5f));
    ASSERT_EQ(tdm::negate(q), tdm::quat<float>(-0.2f, -0.3f, 0.1f, 0.5f));
}

TEST(ComputationsTest, NegateMat) {
    tdm::vec4<int> v{-2, -5, 1, 3};
    tdm::mat4<int> m{v, v, v, v};
    auto nm = -m;
    tdm::vec4<int> expected{2, 5, -1, -3};
    ASSERT_EQ(nm[0], expected);
    ASSERT_EQ(nm[1], expected);
    ASSERT_EQ(nm[2], expected);
    ASSERT_EQ(nm[3], expected);
    ASSERT_EQ(nm, tdm::negate(m));
}

TEST(ComputationsTest, CrossProduct) {
    tdm::vec3<int> v1{1, 2, 3};
    tdm::vec3<int> v2{4, 5, 6};
    auto res = tdm::cross(v1, v2);
    ASSERT_EQ(res, tdm::vec3<int>(-3, 6, -3));
}

TEST(ComputationsTest, DotProduct) {
    tdm::vec4<int> v1{1, 2, 3, 4};
    tdm::vec4<int> v2{5, 6, 7, 8};
    auto res = tdm::dot(v1, v2);
    ASSERT_EQ(res, 70);
}

TEST(ComputationsTest, QuatDotProduct) {
    tdm::quat<float> q1{0.1f, 0.2f, 0.3f, -0.5f};
    tdm::quat<float> q2{-0.1f, -0.2f, -0.3f, 0.5f};
    ASSERT_NEAR(tdm::dot(q1, q2), -0.39f, 0.0001f);
}

TEST(ComputationsTest, LengthVec) {
    tdm::vec4<float> v{0.5f, -0.78f, 0.12f, 1.0f};
    ASSERT_NEAR(tdm::length(v), 1.3685f, 0.0001f);
}

TEST(ComputationsTest, LengthQuat) {
    tdm::quat<float> q{0.1f, -0.2f, 0.3f, 0.25f};
    ASSERT_NEAR(tdm::length(q), 0.45f, 0.0001f);
}

TEST(ComputationsTest, NormalizeVec) {
    tdm::vec4<float> v{0.5f, -0.5f, -0.2f, 1.0f};
    auto normalized = tdm::normalize(v);
    ASSERT_NEAR(normalized[0], 0.4029f, 0.0001f);
    ASSERT_NEAR(normalized[1], -0.4029f, 0.0001f);
    ASSERT_NEAR(normalized[2], -0.1611f, 0.0001f);
    ASSERT_NEAR(normalized[3], 0.8058f, 0.0001f);
    // Make sure original vector remains untouched
    ASSERT_NEAR(v[0], 0.5f, 0.0001f);
    ASSERT_NEAR(v[1], -0.5f, 0.0001f);
    ASSERT_NEAR(v[2], -0.2f, 0.0001f);
    ASSERT_NEAR(v[3], 1.0f, 0.0001f);
}

TEST(ComputationsTest, NormalizeQuat) {
    const tdm::quat<float> q{0.1f, -0.2f, 0.3f, 0.25f};
    const auto normalized = tdm::normalize(q);
    ASSERT_NEAR(normalized.x, 0.2222f, 0.0001f);
    ASSERT_NEAR(normalized.y, -0.4444f, 0.0001f);
    ASSERT_NEAR(normalized.z, 0.6666f, 0.0001f);
    ASSERT_NEAR(normalized.w, 0.5555f, 0.0001f);
}

TEST(ComputationsTest, ConjugateQuat) {
    const tdm::quat<float> q{0.1f, -0.2f, 0.3f, 0.25f};
    const auto conj = tdm::conjugate(q);
    ASSERT_EQ(conj.x, -0.1f);
    ASSERT_EQ(conj.y, 0.2f);
    ASSERT_EQ(conj.z, -0.3f);
    ASSERT_EQ(conj.w, 0.25f);
}

TEST(ComputationsTest, InverseQuat) {
    const tdm::quat<float> q{0.1f, -0.2f, 0.3f, 0.25f};
    const auto inv = tdm::inverse(q);
    ASSERT_NEAR(inv.x, -0.49382716049382713f, 0.0001f);
    ASSERT_NEAR(inv.y, 0.9876543209876543f, 0.0001f);
    ASSERT_NEAR(inv.z, -1.4814814814814814f, 0.0001f);
    ASSERT_NEAR(inv.w, 1.2345679012345678f, 0.0001f);
}

TEST(ComputationsTest, Lerp) {
    tdm::quat<float> q1{0.1f, 0.2f, 0.3f, -0.5f};
    tdm::quat<float> q2{-0.1f, -0.2f, -0.3f, 0.25f};
    const auto lerped = tdm::lerp(q1, q2, 0.4f);
    ASSERT_NEAR(lerped.x, 0.01999f, 0.0001f);
    ASSERT_NEAR(lerped.y, 0.03999f, 0.0001f);
    ASSERT_NEAR(lerped.z, 0.06f, 0.0001f);
    ASSERT_NEAR(lerped.w, -0.2f, 0.0001f);
}

TEST(ComputationsTest, Slerp) {
    tdm::quat<float> q1{0.1f, 0.2f, 0.3f, -0.5f};
    tdm::quat<float> q2{-0.1f, -0.2f, -0.3f, 0.25f};
    const auto slerped = tdm::slerp(q1, q2, 0.4f);
    ASSERT_NEAR(slerped.x, 0.12467f, 0.0001f);
    ASSERT_NEAR(slerped.y, 0.24934f, 0.0001f);
    ASSERT_NEAR(slerped.z, 0.37402f, 0.0001f);
    ASSERT_NEAR(slerped.w, -0.4943f, 0.0001f);
}

TEST(ComputationsTest, SlerpFallbackToLerp) {
    tdm::quat<float> q1{0.5f, 0.5f, 0.5f, 0.5f};
    tdm::quat<float> q2{0.75f, 0.75f, 0.75f, 0.75f};
    const auto slerped = tdm::slerp(q1, q2, 0.4f);
    ASSERT_NEAR(slerped.x, 0.60f, 0.0001f);
    ASSERT_NEAR(slerped.y, 0.60f, 0.0001f);
    ASSERT_NEAR(slerped.z, 0.60f, 0.0001f);
    ASSERT_NEAR(slerped.w, 0.60f, 0.0001f);
}

TEST(ComputationsTest, TransposeSquare) {
    tdm::vec4<int> v{1, 2, 3, 4};
    tdm::mat4<int> m{v, v, v, v};
    auto transposed = tdm::transpose(m);
    ASSERT_EQ(transposed[0], tdm::vec4<int>(1, 1, 1, 1));
    ASSERT_EQ(transposed[1], tdm::vec4<int>(2, 2, 2, 2));
    ASSERT_EQ(transposed[2], tdm::vec4<int>(3, 3, 3, 3));
    ASSERT_EQ(transposed[3], tdm::vec4<int>(4, 4, 4, 4));
}

TEST(ComputationsTest, TransposeNonSquare) {
    tdm::mat<4, 2, int> m{
        tdm::vec2<int>{1, 1},
        tdm::vec2<int>{2, 2},
        tdm::vec2<int>{3, 3},
        tdm::vec2<int>{4, 4}
    };
    tdm::mat<2, 4, int> transposed = tdm::transpose(m);
    ASSERT_EQ(transposed[0], tdm::vec4<int>(1, 2, 3, 4));
    ASSERT_EQ(transposed[1], tdm::vec4<int>(1, 2, 3, 4));
}

TEST(ComputationsTest, DeterminantMat2) {
    tdm::mat2<int> m{
        tdm::vec2<int>{1, 2},
        tdm::vec2<int>{3, 4}
    };
    auto res = tdm::determinant(m);
    ASSERT_EQ(res, -2);
}

TEST(ComputationsTest, DeterminantMat3) {
    tdm::mat3<int> m{
        tdm::vec3<int>{2, -3, 1 },
        tdm::vec3<int>{2, 0,  -1},
        tdm::vec3<int>{1, 4,  5 }
    };
    auto res = tdm::determinant(m);
    ASSERT_EQ(res, 49);
}

TEST(ComputationsTest, DeterminantMat4) {
    tdm::mat4<int> m{
        tdm::vec4<int>{4, 3,  2,  2},
        tdm::vec4<int>{0, 1,  -3, 3},
        tdm::vec4<int>{0, -1, 3,  3},
        tdm::vec4<int>{0, 3,  1,  1}
    };
    auto res = tdm::determinant(m);
    ASSERT_EQ(res, -240);
}

TEST(ComputationsTest, DeterminantMat5) {
    tdm::mat<5, 5, int> m{
        tdm::vec<5, int>{1,  2, 3,  3, 5},
        tdm::vec<5, int>{3,  2, 1,  2, 2},
        tdm::vec<5, int>{1,  2, 3,  4, 5},
        tdm::vec<5, int>{-1, 0, -8, 1, 2},
        tdm::vec<5, int>{7,  2, 1,  3, 2}
    };
    auto res = tdm::determinant(m);
    ASSERT_EQ(res, -224);
}

TEST(ComputationsTest, InverseMat1) {
    tdm::mat<1, 1, float> m{tdm::vec<1, float>{5.0f}};
    auto mi = tdm::inverse(m);
    tdm::mat<1, 1, float> expected{tdm::vec<1, float>{0.2f}};
    for (tdm::dim_t y = 0u; y < m.rows(); ++y) {
        for (tdm::dim_t x = 0u; x < m.columns(); ++x) {
            ASSERT_NEAR(mi[y][x], expected[y][x], 0.0001f);
        }
    }
}

TEST(ComputationsTest, InverseLUMat1) {
    tdm::mat<1, 1, float> m{tdm::vec<1, float>{5.0f}};
    auto mi = tdm::lu::inverse(m);
    tdm::mat<1, 1, float> expected{tdm::vec<1, float>{0.2f}};
    for (tdm::dim_t y = 0u; y < m.rows(); ++y) {
        for (tdm::dim_t x = 0u; x < m.columns(); ++x) {
            ASSERT_NEAR(mi[y][x], expected[y][x], 0.0001f);
        }
    }
}

TEST(ComputationsTest, InverseMat3) {
    tdm::mat3<int> m{
        tdm::vec3<int>{1, 2, 3},
        tdm::vec3<int>{0, 1, 4},
        tdm::vec3<int>{5, 6, 0}
    };
    auto mi = tdm::inverse(m);
    ASSERT_EQ(mi[0], tdm::vec3<int>(-24, 18, 5));
    ASSERT_EQ(mi[1], tdm::vec3<int>(20, -15, -4));
    ASSERT_EQ(mi[2], tdm::vec3<int>(-5, 4, 1));
}

TEST(ComputationsTest, InverseMat4) {
    tdm::mat4<float> m{
        tdm::vec4<float>{0.6f, 0.2f, 0.3f, 0.4f},
        tdm::vec4<float>{0.2f, 0.7f, 0.5f, 0.3f},
        tdm::vec4<float>{0.3f, 0.5f, 0.7f, 0.2f},
        tdm::vec4<float>{0.4f, 0.3f, 0.2f, 0.6f}
    };
    auto mi = tdm::inverse(m);
    tdm::mat4<float> expected{
        tdm::vec4<float>{3.9649f,  1.4035f,  -1.9298f, -2.7017f},
        tdm::vec4<float>{1.4035f,  3.8596f,  -2.8070f, -1.9298f},
        tdm::vec4<float>{-1.9298f, -2.8070f, 3.8596f,  1.4035f },
        tdm::vec4<float>{-2.7017f, -1.9298f, 1.4035f,  3.9649f }
    };
    for (tdm::dim_t y = 0u; y < m.rows(); ++y) {
        for (tdm::dim_t x = 0u; x < m.columns(); ++x) {
            ASSERT_NEAR(mi[y][x], expected[y][x], 0.0001f);
        }
    }
}

TEST(ComputationsTest, InverseLUMat4) {
    tdm::mat4<float> m{
        tdm::vec4<float>{0.6f, 0.2f, 0.3f, 0.4f},
        tdm::vec4<float>{0.2f, 0.7f, 0.5f, 0.3f},
        tdm::vec4<float>{0.3f, 0.5f, 0.7f, 0.2f},
        tdm::vec4<float>{0.4f, 0.3f, 0.2f, 0.6f}
    };
    auto mi = tdm::lu::inverse(m);
    tdm::mat4<float> expected{
        tdm::vec4<float>{3.9649f,  1.4035f,  -1.9298f, -2.7017f},
        tdm::vec4<float>{1.4035f,  3.8596f,  -2.8070f, -1.9298f},
        tdm::vec4<float>{-1.9298f, -2.8070f, 3.8596f,  1.4035f },
        tdm::vec4<float>{-2.7017f, -1.9298f, 1.4035f,  3.9649f }
    };
    for (tdm::dim_t y = 0u; y < m.rows(); ++y) {
        for (tdm::dim_t x = 0u; x < m.columns(); ++x) {
            ASSERT_NEAR(mi[y][x], expected[y][x], 0.0001f);
        }
    }
}

TEST(ComputationsTest, InverseMat7) {
    tdm::mat<7, 7, float> m{
        tdm::vec<7, float>{0.6f, 0.2f, 0.3f, 0.4f, 0.5f, 0.7f, 0.8f},
        tdm::vec<7, float>{0.2f, 0.7f, 0.5f, 0.3f, 0.4f, 0.1f, 0.6f},
        tdm::vec<7, float>{0.3f, 0.5f, 0.7f, 0.2f, 0.6f, 0.4f, 0.1f},
        tdm::vec<7, float>{0.4f, 0.3f, 0.2f, 0.6f, 0.1f, 0.5f, 0.7f},
        tdm::vec<7, float>{0.5f, 0.4f, 0.6f, 0.1f, 0.2f, 0.8f, 0.3f},
        tdm::vec<7, float>{0.7f, 0.1f, 0.4f, 0.5f, 0.8f, 0.3f, 0.2f},
        tdm::vec<7, float>{0.8f, 0.6f, 0.1f, 0.7f, 0.3f, 0.2f, 0.4f}
    };
    auto identity = m * tdm::inverse(m);
    auto expected = decltype(m)::identity();
    for (tdm::dim_t y = 0u; y < m.rows(); ++y) {
        for (tdm::dim_t x = 0u; x < m.columns(); ++x) {
            ASSERT_NEAR(identity[y][x], expected[y][x], 0.0001f);
        }
    }
}

TEST(ComputationsTest, InverseLUMat7) {
    tdm::mat<7, 7, float> m{
        tdm::vec<7, float>{0.6f, 0.2f, 0.3f, 0.4f, 0.5f, 0.7f, 0.8f},
        tdm::vec<7, float>{0.2f, 0.7f, 0.5f, 0.3f, 0.4f, 0.1f, 0.6f},
        tdm::vec<7, float>{0.3f, 0.5f, 0.7f, 0.2f, 0.6f, 0.4f, 0.1f},
        tdm::vec<7, float>{0.4f, 0.3f, 0.2f, 0.6f, 0.1f, 0.5f, 0.7f},
        tdm::vec<7, float>{0.5f, 0.4f, 0.6f, 0.1f, 0.2f, 0.8f, 0.3f},
        tdm::vec<7, float>{0.7f, 0.1f, 0.4f, 0.5f, 0.8f, 0.3f, 0.2f},
        tdm::vec<7, float>{0.8f, 0.6f, 0.1f, 0.7f, 0.3f, 0.2f, 0.4f}
    };
    auto identity = m * tdm::lu::inverse(m);
    auto expected = decltype(m)::identity();
    for (tdm::dim_t y = 0u; y < m.rows(); ++y) {
        for (tdm::dim_t x = 0u; x < m.columns(); ++x) {
            ASSERT_NEAR(identity[y][x], expected[y][x], 0.0001f);
        }
    }
}

TEST(ComputationsTest, TraceMat) {
    tdm::fmat4 m{
        tdm::fvec4{1.0f,  2.0f,  3.0f,  4.0f },
        tdm::fvec4{5.0f,  6.0f,  7.0f,  8.0f },
        tdm::fvec4{9.0f,  10.0f, 11.0f, 12.0f},
        tdm::fvec4{13.0f, 14.0f, 15.0f, 16.0f}
    };
    ASSERT_EQ(trace(m), 34.0f);
}
