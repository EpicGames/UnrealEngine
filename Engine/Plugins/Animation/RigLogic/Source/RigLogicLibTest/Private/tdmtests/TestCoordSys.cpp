// Copyright Epic Games, Inc. All Rights Reserved.

#include "tdmtests/Defs.h"

#include "tdm/TDM.h"

#include <cmath>

namespace {

// Define coordinate systems for tests
// Convention: front = into screen = (0,0,1), back = towards viewer (out of screen) = (0,0,-1)
// Maya: X=left, Y=up, Z=front (right-handed, Y-up, Z into screen)
constexpr tdm::coord_sys maya_cs{tdm::axis_dir::left, tdm::axis_dir::up, tdm::axis_dir::front};
// UE: X=front (into screen), Y=right, Z=up (left-handed, Z-up)
constexpr tdm::coord_sys ue_cs{tdm::axis_dir::front, tdm::axis_dir::right, tdm::axis_dir::up};
// Blender: X=right, Y=front (forward into screen), Z=up (right-handed, Z-up)
constexpr tdm::coord_sys blender_cs{tdm::axis_dir::right, tdm::axis_dir::front, tdm::axis_dir::up};
// Houdini: X=right, Y=up, Z=back (right-handed, Y-up, same as Maya)
constexpr tdm::coord_sys houdini_cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::back};

constexpr tdm::rot_sign rot_sign_positive{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive};
constexpr tdm::rot_sign rot_sign_negative{tdm::rot_dir::negative, tdm::rot_dir::negative, tdm::rot_dir::negative};

}  // namespace

// Test that euler to/from mat extraction round-trips correctly for all 6 rotation sequences
TEST(TestCoordSys, EulerToFromMatExtractionRoundTrip) {
    const tdm::frad3 input{tdm::frad{tdm::fdeg{30.0f}}, tdm::frad{tdm::fdeg{45.0f}}, tdm::frad{tdm::fdeg{60.0f}}};

    // Test XYZ
    {
        const auto mat = tdm::impl::euler2mat(input, tdm::rot_seq::xyz, rot_sign_positive);
        const auto extracted = tdm::impl::mat2euler(mat, tdm::rot_seq::xyz, rot_sign_positive);
        ASSERT_NEAR(tdm::fdeg{extracted[0]}.value, tdm::fdeg{tdm::frad{input[0]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[1]}.value, tdm::fdeg{tdm::frad{input[1]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[2]}.value, tdm::fdeg{tdm::frad{input[2]}}.value, 0.01f);
    }

    // Test XZY
    {
        const auto mat = tdm::impl::euler2mat(input, tdm::rot_seq::xzy, rot_sign_positive);
        const auto extracted = tdm::impl::mat2euler(mat, tdm::rot_seq::xzy, rot_sign_positive);
        ASSERT_NEAR(tdm::fdeg{extracted[0]}.value, tdm::fdeg{tdm::frad{input[0]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[1]}.value, tdm::fdeg{tdm::frad{input[1]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[2]}.value, tdm::fdeg{tdm::frad{input[2]}}.value, 0.01f);
    }

    // Test YXZ
    {
        const auto mat = tdm::impl::euler2mat(input, tdm::rot_seq::yxz, rot_sign_positive);
        const auto extracted = tdm::impl::mat2euler(mat, tdm::rot_seq::yxz, rot_sign_positive);
        ASSERT_NEAR(tdm::fdeg{extracted[0]}.value, tdm::fdeg{tdm::frad{input[0]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[1]}.value, tdm::fdeg{tdm::frad{input[1]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[2]}.value, tdm::fdeg{tdm::frad{input[2]}}.value, 0.01f);
    }

    // Test YZX
    {
        const auto mat = tdm::impl::euler2mat(input, tdm::rot_seq::yzx, rot_sign_positive);
        const auto extracted = tdm::impl::mat2euler(mat, tdm::rot_seq::yzx, rot_sign_positive);
        ASSERT_NEAR(tdm::fdeg{extracted[0]}.value, tdm::fdeg{tdm::frad{input[0]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[1]}.value, tdm::fdeg{tdm::frad{input[1]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[2]}.value, tdm::fdeg{tdm::frad{input[2]}}.value, 0.01f);
    }

    // Test ZXY
    {
        const auto mat = tdm::impl::euler2mat(input, tdm::rot_seq::zxy, rot_sign_positive);
        const auto extracted = tdm::impl::mat2euler(mat, tdm::rot_seq::zxy, rot_sign_positive);
        ASSERT_NEAR(tdm::fdeg{extracted[0]}.value, tdm::fdeg{tdm::frad{input[0]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[1]}.value, tdm::fdeg{tdm::frad{input[1]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[2]}.value, tdm::fdeg{tdm::frad{input[2]}}.value, 0.01f);
    }

    // Test ZYX
    {
        const auto mat = tdm::impl::euler2mat(input, tdm::rot_seq::zyx, rot_sign_positive);
        const auto extracted = tdm::impl::mat2euler(mat, tdm::rot_seq::zyx, rot_sign_positive);
        ASSERT_NEAR(tdm::fdeg{extracted[0]}.value, tdm::fdeg{tdm::frad{input[0]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[1]}.value, tdm::fdeg{tdm::frad{input[1]}}.value, 0.01f);
        ASSERT_NEAR(tdm::fdeg{extracted[2]}.value, tdm::fdeg{tdm::frad{input[2]}}.value, 0.01f);
    }
}

// ---------------- Rotation Conversion Tests ----------------

// Test same-handedness coordinate system rotation conversion (e.g., two right-handed systems)
TEST(TestCoordSys, TwoRightHandedSystemsRotationRoundTrip) {
    // Two right-handed systems
    tdm::coord_sys src_cs{tdm::axis_dir::right,
                          tdm::axis_dir::up,
                          tdm::axis_dir::back};  // RH: X=right, Y=up, Z=back (towards viewer)
    tdm::coord_sys dst_cs{tdm::axis_dir::right, tdm::axis_dir::front, tdm::axis_dir::up};  // RH: X=right, Y=front, Z=up

    const tdm::frad3 input{tdm::frad{tdm::fdeg{15.0f}}, tdm::frad{tdm::fdeg{30.0f}}, tdm::frad{tdm::fdeg{45.0f}}};

    // Convert and convert back
    const auto converted = tdm::convert_rotation<float>(input,
                                                        src_cs,
                                                        tdm::rot_seq::xyz,
                                                        rot_sign_positive,
                                                        dst_cs,
                                                        tdm::rot_seq::xyz,
                                                        rot_sign_positive);

    const auto back_to_src = tdm::convert_rotation<float>(converted,
                                                          dst_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive,
                                                          src_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{back_to_src[0]}.value, tdm::fdeg{tdm::frad{input[0]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_src[1]}.value, tdm::fdeg{tdm::frad{input[1]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_src[2]}.value, tdm::fdeg{tdm::frad{input[2]}}.value, 0.01f);
}

// Test two left-handed coordinate systems
TEST(TestCoordSys, TwoLeftHandedSystemsRotationRoundTrip) {
    // Two left-handed systems with different axis orientations
    tdm::coord_sys src_cs{tdm::axis_dir::right,
                          tdm::axis_dir::up,
                          tdm::axis_dir::front};  // LH: X=right, Y=up, Z=front (into screen)
    tdm::coord_sys dst_cs{tdm::axis_dir::back, tdm::axis_dir::up, tdm::axis_dir::right};  // LH (axes rotated)

    const tdm::frad3 original{tdm::frad{tdm::fdeg{35.0f}}, tdm::frad{tdm::fdeg{-25.0f}}, tdm::frad{tdm::fdeg{70.0f}}};

    const auto converted = tdm::convert_rotation<float>(original,
                                                        src_cs,
                                                        tdm::rot_seq::xyz,
                                                        rot_sign_positive,
                                                        dst_cs,
                                                        tdm::rot_seq::xyz,
                                                        rot_sign_positive);

    const auto back_to_src = tdm::convert_rotation<float>(converted,
                                                          dst_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive,
                                                          src_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{back_to_src[0]}.value, tdm::fdeg{tdm::frad{original[0]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_src[1]}.value, tdm::fdeg{tdm::frad{original[1]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_src[2]}.value, tdm::fdeg{tdm::frad{original[2]}}.value, 0.01f);
}

// Test conversion with different rotation sequences (XYZ to ZYX)
TEST(TestCoordSys, DifferentRotationSequences) {
    tdm::coord_sys cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::back};  // RH system

    const tdm::frad3 original{tdm::frad{tdm::fdeg{10.0f}}, tdm::frad{tdm::fdeg{20.0f}}, tdm::frad{tdm::fdeg{30.0f}}};

    // Convert from XYZ to ZYX in same coordinate system
    const auto to_zyx = tdm::convert_rotation<float>(original,
                                                     cs,
                                                     tdm::rot_seq::xyz,
                                                     rot_sign_positive,
                                                     cs,
                                                     tdm::rot_seq::zyx,
                                                     rot_sign_positive);

    // Convert back from ZYX to XYZ
    const auto back_to_xyz =
        tdm::convert_rotation<float>(to_zyx, cs, tdm::rot_seq::zyx, rot_sign_positive, cs, tdm::rot_seq::xyz, rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{back_to_xyz[0]}.value, tdm::fdeg{tdm::frad{original[0]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_xyz[1]}.value, tdm::fdeg{tdm::frad{original[1]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_xyz[2]}.value, tdm::fdeg{tdm::frad{original[2]}}.value, 0.01f);
}

// Test conversion with different rotation signs (positive to negative)
TEST(TestCoordSys, DifferentRotationSigns) {
    tdm::coord_sys cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::back};  // RH system

    const tdm::frad3 original{tdm::frad{tdm::fdeg{10.0f}}, tdm::frad{tdm::fdeg{20.0f}}, tdm::frad{tdm::fdeg{30.0f}}};

    // Convert from positive rotation sign to negative rotation sign in same coordinate system
    const auto to_negative = tdm::convert_rotation<float>(original,
                                                          cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive,
                                                          cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_negative);

    // Convert back from negative to positive
    const auto back_to_positive = tdm::convert_rotation<float>(to_negative,
                                                               cs,
                                                               tdm::rot_seq::xyz,
                                                               rot_sign_negative,
                                                               cs,
                                                               tdm::rot_seq::xyz,
                                                               rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{back_to_positive[0]}.value, tdm::fdeg{tdm::frad{original[0]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_positive[1]}.value, tdm::fdeg{tdm::frad{original[1]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_positive[2]}.value, tdm::fdeg{tdm::frad{original[2]}}.value, 0.01f);
}

// Test OpenGL-style to DirectX-style rotation coordinate conversion (RH Z-back to LH Z-front)
TEST(TestCoordSys, OpenGLToDirectXRotationRoundTrip) {
    // OpenGL: X=right, Y=up, Z=back (right-handed, Z towards viewer)
    // DirectX: X=right, Y=up, Z=front (left-handed, Z into screen)
    tdm::coord_sys opengl_cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::back};
    tdm::coord_sys directx_cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::front};

    const tdm::frad3 original{tdm::frad{tdm::fdeg{45.0f}}, tdm::frad{tdm::fdeg{-60.0f}}, tdm::frad{tdm::fdeg{30.0f}}};

    // OpenGL -> DirectX
    const auto to_directx = tdm::convert_rotation<float>(original,
                                                         opengl_cs,
                                                         tdm::rot_seq::xyz,
                                                         rot_sign_positive,
                                                         directx_cs,
                                                         tdm::rot_seq::xyz,
                                                         rot_sign_positive);

    // DirectX -> OpenGL
    const auto back_to_opengl = tdm::convert_rotation<float>(to_directx,
                                                             directx_cs,
                                                             tdm::rot_seq::xyz,
                                                             rot_sign_positive,
                                                             opengl_cs,
                                                             tdm::rot_seq::xyz,
                                                             rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{back_to_opengl[0]}.value, tdm::fdeg{tdm::frad{original[0]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_opengl[1]}.value, tdm::fdeg{tdm::frad{original[1]}}.value, 0.01f);
    ASSERT_NEAR(tdm::fdeg{back_to_opengl[2]}.value, tdm::fdeg{tdm::frad{original[2]}}.value, 0.01f);
}

// ---------------- Position Conversion Tests ----------------

// Test position conversion with handedness change (OpenGL to DirectX)
TEST(TestCoordSys, PositionConversionWithHandednessChange) {
    // OpenGL: X=right, Y=up, Z=back (right-handed, Z towards viewer)
    // DirectX: X=right, Y=up, Z=front (left-handed, Z into screen)
    tdm::coord_sys opengl_cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::back};
    tdm::coord_sys directx_cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::front};

    const tdm::fvec3 original{1.0f, 2.0f, 3.0f};

    // OpenGL -> DirectX (Z axis flips)
    const auto to_directx = tdm::convert_position(original, opengl_cs, directx_cs);

    // X and Y stay the same, Z flips sign
    ASSERT_NEAR(to_directx[0], 1.0f, 0.001f);
    ASSERT_NEAR(to_directx[1], 2.0f, 0.001f);
    ASSERT_NEAR(to_directx[2], -3.0f, 0.001f);

    // Round-trip should give original
    const auto back_to_opengl = tdm::convert_position(to_directx, directx_cs, opengl_cs);
    ASSERT_NEAR(back_to_opengl[0], original[0], 0.001f);
    ASSERT_NEAR(back_to_opengl[1], original[1], 0.001f);
    ASSERT_NEAR(back_to_opengl[2], original[2], 0.001f);
}

// ---------------- Normal Conversion Tests ----------------

// Test that normals remain normalized after conversion
TEST(TestCoordSys, NormalStaysNormalized) {
    const tdm::fvec3 unit_normal{0.0f, 0.0f, 1.0f};
    const auto converted = tdm::convert_direction(unit_normal, maya_cs, ue_cs);

    // Check length is still 1
    const float length = std::sqrt(converted[0] * converted[0] + converted[1] * converted[1] + converted[2] * converted[2]);
    ASSERT_NEAR(length, 1.0f, 0.001f);
}

// -------------------- Test coord_sys member functions --------------------

TEST(TestCoordSys, BasisMatrix) {
    // Test that basis matrix is constructed correctly
    tdm::coord_sys cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::front};
    const auto basis = cs.basis<float>();

    // Column 0 (X axis): (1, 0, 0)
    // Column 1 (Y axis): (0, 1, 0)
    // Column 2 (Z axis): (0, 0, 1)
    ASSERT_NEAR(basis(0, 0), 1.0f, 0.001f);
    ASSERT_NEAR(basis(1, 0), 0.0f, 0.001f);
    ASSERT_NEAR(basis(2, 0), 0.0f, 0.001f);

    ASSERT_NEAR(basis(0, 1), 0.0f, 0.001f);
    ASSERT_NEAR(basis(1, 1), 1.0f, 0.001f);
    ASSERT_NEAR(basis(2, 1), 0.0f, 0.001f);

    ASSERT_NEAR(basis(0, 2), 0.0f, 0.001f);
    ASSERT_NEAR(basis(1, 2), 0.0f, 0.001f);
    ASSERT_NEAR(basis(2, 2), 1.0f, 0.001f);
}

TEST(TestCoordSys, Validity) {
    // Valid coordinate systems have orthogonal axes
    tdm::coord_sys valid_cs{tdm::axis_dir::right, tdm::axis_dir::up, tdm::axis_dir::front};
    ASSERT_TRUE(valid_cs.valid<float>());

    // Invalid coordinate systems don't have orthogonal axes
    tdm::coord_sys invalid_cs{tdm::axis_dir::right, tdm::axis_dir::right, tdm::axis_dir::front};
    ASSERT_FALSE(invalid_cs.valid<float>());

    // Maya and UE coordinate systems should be valid
    ASSERT_TRUE(maya_cs.valid<float>());
    ASSERT_TRUE(ue_cs.valid<float>());
}

// Test change_of_basis function
TEST(TestCoordSys, ChangeOfBasis) {
    // Change of basis from Maya to UE
    const auto cob = tdm::change_of_basis<float>(maya_cs, ue_cs);

    // Apply to a vector: Maya (1, 2, 3) -> UE (3, -1, 2)
    const tdm::fvec3 maya_vec{1.0f, 2.0f, 3.0f};
    const tdm::fvec3 ue_vec = maya_vec * cob;

    ASSERT_NEAR(ue_vec[0], 3.0f, 0.001f);   // mz
    ASSERT_NEAR(ue_vec[1], -1.0f, 0.001f);  // -mx
    ASSERT_NEAR(ue_vec[2], 2.0f, 0.001f);   // my
}

// Test cached change_of_basis for batch operations
TEST(TestCoordSys, CachedChangeOfBasisBatchConversion) {
    // Pre-compute change of basis matrix once
    const auto cob = tdm::change_of_basis<float>(maya_cs, ue_cs);

    // Convert multiple positions using cached matrix
    const tdm::fvec3 positions[] = {
        {1.0f, 2.0f, 3.0f},
        {4.0f, 5.0f, 6.0f},
        {7.0f, 8.0f, 9.0f}
    };

    for (const auto& pos : positions) {
        const auto converted_cached = tdm::convert_position(pos, cob);
        const auto converted_direct = tdm::convert_position(pos, maya_cs, ue_cs);

        ASSERT_NEAR(converted_cached[0], converted_direct[0], 0.001f);
        ASSERT_NEAR(converted_cached[1], converted_direct[1], 0.001f);
        ASSERT_NEAR(converted_cached[2], converted_direct[2], 0.001f);
    }
}

// ================ Maya <-> UE Bidirectional Tests ================

TEST(TestCoordSys, MayaToUE_Position) {
    // Maya: X=left, Y=up, Z=front -> UE: X=front, Y=right, Z=up
    // Maya (mx, my, mz) -> UE (mz, -mx, my)
    const tdm::fvec3 maya_translation{1.0f, 2.0f, 3.0f};
    const auto ue_translation = tdm::convert_position(maya_translation, maya_cs, ue_cs);

    ASSERT_NEAR(ue_translation[0], 3.0f, 0.001f);   // mz
    ASSERT_NEAR(ue_translation[1], -1.0f, 0.001f);  // -mx
    ASSERT_NEAR(ue_translation[2], 2.0f, 0.001f);   // my
}

TEST(TestCoordSys, UEToMaya_Position) {
    const tdm::fvec3 ue_pos{1.0f, 3.0f, 2.0f};
    const auto maya_pos = tdm::convert_position(ue_pos, ue_cs, maya_cs);

    // UE (ux, uy, uz) -> Maya (-uy, uz, ux)
    ASSERT_NEAR(maya_pos[0], -3.0f, 0.001f);  // -uy
    ASSERT_NEAR(maya_pos[1], 2.0f, 0.001f);   // uz
    ASSERT_NEAR(maya_pos[2], 1.0f, 0.001f);   // ux
}

TEST(TestCoordSys, MayaToUE_Rotation) {
    const tdm::frad3 maya_rot{tdm::frad{tdm::fdeg{30.0f}}, tdm::frad{tdm::fdeg{-45.0f}}, tdm::frad{tdm::fdeg{60.0f}}};
    const auto ue_rot = tdm::convert_rotation<float>(maya_rot,
                                                     maya_cs,
                                                     tdm::rot_seq::xyz,
                                                     rot_sign_positive,
                                                     ue_cs,
                                                     tdm::rot_seq::xyz,
                                                     rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{ue_rot[0]}.value, -78.30f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{ue_rot[1]}.value, 51.29f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{ue_rot[2]}.value, -11.70f, 0.01f);
}

TEST(TestCoordSys, UEToMaya_Rotation) {
    const tdm::frad3 ue_rot{tdm::frad{tdm::fdeg{15.0f}}, tdm::frad{tdm::fdeg{-30.0f}}, tdm::frad{tdm::fdeg{45.0f}}};
    const auto maya_rot = tdm::convert_rotation<float>(ue_rot,
                                                       ue_cs,
                                                       tdm::rot_seq::xyz,
                                                       rot_sign_positive,
                                                       maya_cs,
                                                       tdm::rot_seq::xyz,
                                                       rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{maya_rot[0]}.value, -14.51f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{maya_rot[1]}.value, -50.76f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{maya_rot[2]}.value, -20.75f, 0.01f);
}

TEST(TestCoordSys, MayaToUE_Scale) {
    // Maya: X=left, Y=up, Z=front -> UE: X=front, Y=right, Z=up
    // Maya (mx, my, mz) -> UE (mz, mx, my) for scale (absolute values)
    const tdm::fvec3 maya_scale{1.0f, 2.0f, 3.0f};
    const auto ue_scale = tdm::convert_scale(maya_scale, maya_cs, ue_cs);

    ASSERT_NEAR(ue_scale[0], 3.0f, 0.001f);  // mz
    ASSERT_NEAR(ue_scale[1], 1.0f, 0.001f);  // mx
    ASSERT_NEAR(ue_scale[2], 2.0f, 0.001f);  // my
}

TEST(TestCoordSys, UEToMaya_Scale) {
    // UE (ux, uy, uz) -> Maya (uy, uz, ux) for scale (absolute values)
    const tdm::fvec3 ue_scale{1.0f, 3.0f, 2.0f};
    const auto maya_scale = tdm::convert_scale(ue_scale, ue_cs, maya_cs);

    ASSERT_NEAR(maya_scale[0], 3.0f, 0.001f);  // uy
    ASSERT_NEAR(maya_scale[1], 2.0f, 0.001f);  // uz
    ASSERT_NEAR(maya_scale[2], 1.0f, 0.001f);  // ux
}

TEST(TestCoordSys, MayaToUE_Direction) {
    const tdm::fvec3 maya_dir{0.0f, 1.0f, 0.0f};  // Up in Maya
    const auto ue_dir = tdm::convert_direction(maya_dir, maya_cs, ue_cs);

    // Maya (mx, my, mz) -> UE (mz, -mx, my)
    // Maya (0, 1, 0) -> UE (0, 0, 1)
    ASSERT_NEAR(ue_dir[0], 0.0f, 0.001f);  // mz
    ASSERT_NEAR(ue_dir[1], 0.0f, 0.001f);  // -mx
    ASSERT_NEAR(ue_dir[2], 1.0f, 0.001f);  // my
}

TEST(TestCoordSys, UEToMaya_Direction) {
    const tdm::fvec3 ue_dir{0.0f, 0.0f, 1.0f};  // Up in UE
    const auto maya_dir = tdm::convert_direction(ue_dir, ue_cs, maya_cs);

    // UE (ux, uy, uz) -> Maya (-uy, uz, ux)
    // UE (0, 0, 1) -> Maya (0, 1, 0)
    ASSERT_NEAR(maya_dir[0], 0.0f, 0.001f);  // -uy
    ASSERT_NEAR(maya_dir[1], 1.0f, 0.001f);  // uz
    ASSERT_NEAR(maya_dir[2], 0.0f, 0.001f);  // ux
}

// ================ Maya <-> Blender Bidirectional Tests ================

TEST(TestCoordSys, MayaToBlender_Position) {
    const tdm::fvec3 maya_pos{1.0f, 2.0f, 3.0f};
    const auto blender_pos = tdm::convert_position(maya_pos, maya_cs, blender_cs);

    // Round-trip
    const auto back_to_maya = tdm::convert_position(blender_pos, blender_cs, maya_cs);
    ASSERT_NEAR(back_to_maya[0], maya_pos[0], 0.001f);
    ASSERT_NEAR(back_to_maya[1], maya_pos[1], 0.001f);
    ASSERT_NEAR(back_to_maya[2], maya_pos[2], 0.001f);
}

TEST(TestCoordSys, BlenderToMaya_Position) {
    const tdm::fvec3 blender_pos{1.0f, 2.0f, 3.0f};
    const auto maya_pos = tdm::convert_position(blender_pos, blender_cs, maya_cs);

    // Round-trip
    const auto back_to_blender = tdm::convert_position(maya_pos, maya_cs, blender_cs);
    ASSERT_NEAR(back_to_blender[0], blender_pos[0], 0.001f);
    ASSERT_NEAR(back_to_blender[1], blender_pos[1], 0.001f);
    ASSERT_NEAR(back_to_blender[2], blender_pos[2], 0.001f);
}

TEST(TestCoordSys, MayaToBlender_Rotation) {
    const tdm::frad3 maya_rot{tdm::frad{tdm::fdeg{25.0f}}, tdm::frad{tdm::fdeg{-35.0f}}, tdm::frad{tdm::fdeg{50.0f}}};
    const auto blender_rot = tdm::convert_rotation<float>(maya_rot,
                                                          maya_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive,
                                                          blender_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{blender_rot[0]}.value, -59.36f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{blender_rot[1]}.value, 38.87f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{blender_rot[2]}.value, -47.45f, 0.01f);
}

TEST(TestCoordSys, BlenderToMaya_Rotation) {
    const tdm::frad3 blender_rot{tdm::frad{tdm::fdeg{40.0f}}, tdm::frad{tdm::fdeg{-20.0f}}, tdm::frad{tdm::fdeg{70.0f}}};
    const auto maya_rot = tdm::convert_rotation<float>(blender_rot,
                                                       blender_cs,
                                                       tdm::rot_seq::xyz,
                                                       rot_sign_positive,
                                                       maya_cs,
                                                       tdm::rot_seq::xyz,
                                                       rot_sign_positive);

    ASSERT_NEAR(tdm::fdeg{maya_rot[0]}.value, -83.22f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{maya_rot[1]}.value, 62.01f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{maya_rot[2]}.value, -46.78f, 0.01f);
}

TEST(TestCoordSys, MayaToBlender_Scale) {
    const tdm::fvec3 maya_scale{1.5f, 2.0f, 0.5f};
    const auto blender_scale = tdm::convert_scale(maya_scale, maya_cs, blender_cs);

    // Round-trip
    const auto back_to_maya = tdm::convert_scale(blender_scale, blender_cs, maya_cs);
    ASSERT_NEAR(back_to_maya[0], maya_scale[0], 0.001f);
    ASSERT_NEAR(back_to_maya[1], maya_scale[1], 0.001f);
    ASSERT_NEAR(back_to_maya[2], maya_scale[2], 0.001f);
}

TEST(TestCoordSys, BlenderToMaya_Scale) {
    const tdm::fvec3 blender_scale{2.0f, 1.5f, 3.0f};
    const auto maya_scale = tdm::convert_scale(blender_scale, blender_cs, maya_cs);

    // Round-trip
    const auto back_to_blender = tdm::convert_scale(maya_scale, maya_cs, blender_cs);
    ASSERT_NEAR(back_to_blender[0], blender_scale[0], 0.001f);
    ASSERT_NEAR(back_to_blender[1], blender_scale[1], 0.001f);
    ASSERT_NEAR(back_to_blender[2], blender_scale[2], 0.001f);
}

TEST(TestCoordSys, MayaToBlender_Direction) {
    const tdm::fvec3 maya_dir{0.577f, 0.577f, 0.577f};  // Normalized diagonal
    const auto blender_dir = tdm::convert_direction(maya_dir, maya_cs, blender_cs);

    // Round-trip
    const auto back_to_maya = tdm::convert_direction(blender_dir, blender_cs, maya_cs);
    ASSERT_NEAR(back_to_maya[0], maya_dir[0], 0.01f);
    ASSERT_NEAR(back_to_maya[1], maya_dir[1], 0.01f);
    ASSERT_NEAR(back_to_maya[2], maya_dir[2], 0.01f);
}

TEST(TestCoordSys, BlenderToMaya_Direction) {
    const tdm::fvec3 blender_dir{0.0f, 0.0f, 1.0f};  // Up in Blender
    const auto maya_dir = tdm::convert_direction(blender_dir, blender_cs, maya_cs);

    // Round-trip
    const auto back_to_blender = tdm::convert_direction(maya_dir, maya_cs, blender_cs);
    ASSERT_NEAR(back_to_blender[0], blender_dir[0], 0.001f);
    ASSERT_NEAR(back_to_blender[1], blender_dir[1], 0.001f);
    ASSERT_NEAR(back_to_blender[2], blender_dir[2], 0.001f);
}

// ================ Maya <-> Houdini Bidirectional Tests ================

TEST(TestCoordSys, MayaToHoudini_Position) {
    const tdm::fvec3 maya_pos{1.0f, 2.0f, 3.0f};
    const auto houdini_pos = tdm::convert_position(maya_pos, maya_cs, houdini_cs);

    // Round-trip
    const auto back_to_maya = tdm::convert_position(houdini_pos, houdini_cs, maya_cs);
    ASSERT_NEAR(back_to_maya[0], maya_pos[0], 0.001f);
    ASSERT_NEAR(back_to_maya[1], maya_pos[1], 0.001f);
    ASSERT_NEAR(back_to_maya[2], maya_pos[2], 0.001f);
}

TEST(TestCoordSys, HoudiniToMaya_Position) {
    const tdm::fvec3 houdini_pos{1.0f, 2.0f, 3.0f};
    const auto maya_pos = tdm::convert_position(houdini_pos, houdini_cs, maya_cs);

    // Round-trip
    const auto back_to_houdini = tdm::convert_position(maya_pos, maya_cs, houdini_cs);
    ASSERT_NEAR(back_to_houdini[0], houdini_pos[0], 0.001f);
    ASSERT_NEAR(back_to_houdini[1], houdini_pos[1], 0.001f);
    ASSERT_NEAR(back_to_houdini[2], houdini_pos[2], 0.001f);
}

TEST(TestCoordSys, MayaToHoudini_Rotation) {
    const tdm::frad3 maya_rot{tdm::frad{tdm::fdeg{15.0f}}, tdm::frad{tdm::fdeg{-25.0f}}, tdm::frad{tdm::fdeg{35.0f}}};
    const auto houdini_rot = tdm::convert_rotation<float>(maya_rot,
                                                          maya_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive,
                                                          houdini_cs,
                                                          tdm::rot_seq::xyz,
                                                          rot_sign_positive);

    // Maya and Houdini are both RH Y-up but with opposite X and Z directions,
    // so X and Z rotation angles negate while Y stays the same.
    ASSERT_NEAR(tdm::fdeg{houdini_rot[0]}.value, -15.0f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{houdini_rot[1]}.value, -25.0f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{houdini_rot[2]}.value, -35.0f, 0.01f);
}

TEST(TestCoordSys, HoudiniToMaya_Rotation) {
    const tdm::frad3 houdini_rot{tdm::frad{tdm::fdeg{55.0f}}, tdm::frad{tdm::fdeg{-15.0f}}, tdm::frad{tdm::fdeg{80.0f}}};
    const auto maya_rot = tdm::convert_rotation<float>(houdini_rot,
                                                       houdini_cs,
                                                       tdm::rot_seq::xyz,
                                                       rot_sign_positive,
                                                       maya_cs,
                                                       tdm::rot_seq::xyz,
                                                       rot_sign_positive);

    // Houdini and Maya are both RH Y-up but with opposite X and Z directions,
    // so X and Z rotation angles negate while Y stays the same.
    ASSERT_NEAR(tdm::fdeg{maya_rot[0]}.value, -55.0f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{maya_rot[1]}.value, -15.0f, 0.01f);
    ASSERT_NEAR(tdm::fdeg{maya_rot[2]}.value, -80.0f, 0.01f);
}

TEST(TestCoordSys, MayaToHoudini_Scale) {
    const tdm::fvec3 maya_scale{1.0f, 2.5f, 0.75f};
    const auto houdini_scale = tdm::convert_scale(maya_scale, maya_cs, houdini_cs);

    // Round-trip
    const auto back_to_maya = tdm::convert_scale(houdini_scale, houdini_cs, maya_cs);
    ASSERT_NEAR(back_to_maya[0], maya_scale[0], 0.001f);
    ASSERT_NEAR(back_to_maya[1], maya_scale[1], 0.001f);
    ASSERT_NEAR(back_to_maya[2], maya_scale[2], 0.001f);
}

TEST(TestCoordSys, HoudiniToMaya_Scale) {
    const tdm::fvec3 houdini_scale{3.0f, 1.0f, 2.0f};
    const auto maya_scale = tdm::convert_scale(houdini_scale, houdini_cs, maya_cs);

    // Round-trip
    const auto back_to_houdini = tdm::convert_scale(maya_scale, maya_cs, houdini_cs);
    ASSERT_NEAR(back_to_houdini[0], houdini_scale[0], 0.001f);
    ASSERT_NEAR(back_to_houdini[1], houdini_scale[1], 0.001f);
    ASSERT_NEAR(back_to_houdini[2], houdini_scale[2], 0.001f);
}

TEST(TestCoordSys, MayaToHoudini_Direction) {
    const tdm::fvec3 maya_dir{0.0f, 1.0f, 0.0f};  // Up in Maya
    const auto houdini_dir = tdm::convert_direction(maya_dir, maya_cs, houdini_cs);

    // Round-trip
    const auto back_to_maya = tdm::convert_direction(houdini_dir, houdini_cs, maya_cs);
    ASSERT_NEAR(back_to_maya[0], maya_dir[0], 0.001f);
    ASSERT_NEAR(back_to_maya[1], maya_dir[1], 0.001f);
    ASSERT_NEAR(back_to_maya[2], maya_dir[2], 0.001f);
}

TEST(TestCoordSys, HoudiniToMaya_Direction) {
    const tdm::fvec3 houdini_dir{0.0f, 1.0f, 0.0f};  // Up in Houdini
    const auto maya_dir = tdm::convert_direction(houdini_dir, houdini_cs, maya_cs);

    // Round-trip
    const auto back_to_houdini = tdm::convert_direction(maya_dir, maya_cs, houdini_cs);
    ASSERT_NEAR(back_to_houdini[0], houdini_dir[0], 0.001f);
    ASSERT_NEAR(back_to_houdini[1], houdini_dir[1], 0.001f);
    ASSERT_NEAR(back_to_houdini[2], houdini_dir[2], 0.001f);
}

// These tests verify that converted values have the correct physical meaning,
// not just that round-trips recover the original. Round-trip tests alone cannot
// detect a transpose error in the change-of-basis matrix, because for any
// orthogonal C: (v * C) * C^T = v, regardless of whether C is correct.
// Similarly, tests involving only diagonal bases (like OpenGL<->DirectX) cannot
// detect the bug because diagonal matrices equal their own transpose.

// Verify that the "up" direction maps correctly across all coordinate systems.
// "Up" is a universal physical direction, so it must be preserved regardless of
// which axis represents it in each system.
TEST(TestCoordSys, PhysicalUp_MayaToBlender) {
    // Maya: Y=up -> (0, 1, 0), Blender: Z=up -> (0, 0, 1)
    const tdm::fvec3 maya_up{0.0f, 1.0f, 0.0f};
    const auto blender_up = tdm::convert_direction(maya_up, maya_cs, blender_cs);
    ASSERT_NEAR(blender_up[0], 0.0f, 0.001f);
    ASSERT_NEAR(blender_up[1], 0.0f, 0.001f);
    ASSERT_NEAR(blender_up[2], 1.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalUp_MayaToHoudini) {
    // Maya: Y=up -> (0, 1, 0), Houdini: Y=up -> (0, 1, 0)
    const tdm::fvec3 maya_up{0.0f, 1.0f, 0.0f};
    const auto houdini_up = tdm::convert_direction(maya_up, maya_cs, houdini_cs);
    ASSERT_NEAR(houdini_up[0], 0.0f, 0.001f);
    ASSERT_NEAR(houdini_up[1], 1.0f, 0.001f);
    ASSERT_NEAR(houdini_up[2], 0.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalUp_UEToBlender) {
    // UE: Z=up -> (0, 0, 1), Blender: Z=up -> (0, 0, 1)
    const tdm::fvec3 ue_up{0.0f, 0.0f, 1.0f};
    const auto blender_up = tdm::convert_direction(ue_up, ue_cs, blender_cs);
    ASSERT_NEAR(blender_up[0], 0.0f, 0.001f);
    ASSERT_NEAR(blender_up[1], 0.0f, 0.001f);
    ASSERT_NEAR(blender_up[2], 1.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalUp_UEToHoudini) {
    // UE: Z=up -> (0, 0, 1), Houdini: Y=up -> (0, 1, 0)
    const tdm::fvec3 ue_up{0.0f, 0.0f, 1.0f};
    const auto houdini_up = tdm::convert_direction(ue_up, ue_cs, houdini_cs);
    ASSERT_NEAR(houdini_up[0], 0.0f, 0.001f);
    ASSERT_NEAR(houdini_up[1], 1.0f, 0.001f);
    ASSERT_NEAR(houdini_up[2], 0.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalUp_BlenderToHoudini) {
    // Blender: Z=up -> (0, 0, 1), Houdini: Y=up -> (0, 1, 0)
    const tdm::fvec3 blender_up{0.0f, 0.0f, 1.0f};
    const auto houdini_up = tdm::convert_direction(blender_up, blender_cs, houdini_cs);
    ASSERT_NEAR(houdini_up[0], 0.0f, 0.001f);
    ASSERT_NEAR(houdini_up[1], 1.0f, 0.001f);
    ASSERT_NEAR(houdini_up[2], 0.0f, 0.001f);
}

// Verify specific position values for conversions involving axis permutations.
// These tests check that each component physically means the same thing before
// and after conversion (e.g., "2 units up" stays "2 units up").
TEST(TestCoordSys, PhysicalPosition_MayaToBlender) {
    // Maya (1, 2, 3) = 1 left, 2 up, 3 front
    // Blender (X=right, Y=front, Z=up): right=-1, front=3, up=2 -> (-1, 3, 2)
    const tdm::fvec3 maya_pos{1.0f, 2.0f, 3.0f};
    const auto blender_pos = tdm::convert_position(maya_pos, maya_cs, blender_cs);
    ASSERT_NEAR(blender_pos[0], -1.0f, 0.001f);
    ASSERT_NEAR(blender_pos[1], 3.0f, 0.001f);
    ASSERT_NEAR(blender_pos[2], 2.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalPosition_MayaToHoudini) {
    // Maya (1, 2, 3) = 1 left, 2 up, 3 front
    // Houdini (X=right, Y=up, Z=back): right=-1, up=2, back=-3 -> (-1, 2, -3)
    const tdm::fvec3 maya_pos{1.0f, 2.0f, 3.0f};
    const auto houdini_pos = tdm::convert_position(maya_pos, maya_cs, houdini_cs);
    ASSERT_NEAR(houdini_pos[0], -1.0f, 0.001f);
    ASSERT_NEAR(houdini_pos[1], 2.0f, 0.001f);
    ASSERT_NEAR(houdini_pos[2], -3.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalPosition_UEToBlender) {
    // UE (1, 2, 3) = 1 front, 2 right, 3 up
    // Blender (X=right, Y=front, Z=up): right=2, front=1, up=3 -> (2, 1, 3)
    const tdm::fvec3 ue_pos{1.0f, 2.0f, 3.0f};
    const auto blender_pos = tdm::convert_position(ue_pos, ue_cs, blender_cs);
    ASSERT_NEAR(blender_pos[0], 2.0f, 0.001f);
    ASSERT_NEAR(blender_pos[1], 1.0f, 0.001f);
    ASSERT_NEAR(blender_pos[2], 3.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalPosition_UEToHoudini) {
    // UE (1, 2, 3) = 1 front, 2 right, 3 up
    // Houdini (X=right, Y=up, Z=back): right=2, up=3, back=-1 -> (2, 3, -1)
    const tdm::fvec3 ue_pos{1.0f, 2.0f, 3.0f};
    const auto houdini_pos = tdm::convert_position(ue_pos, ue_cs, houdini_cs);
    ASSERT_NEAR(houdini_pos[0], 2.0f, 0.001f);
    ASSERT_NEAR(houdini_pos[1], 3.0f, 0.001f);
    ASSERT_NEAR(houdini_pos[2], -1.0f, 0.001f);
}

TEST(TestCoordSys, PhysicalPosition_BlenderToHoudini) {
    // Blender (1, 2, 3) = 1 right, 2 front, 3 up
    // Houdini (X=right, Y=up, Z=back): right=1, up=3, back=-2 -> (1, 3, -2)
    const tdm::fvec3 blender_pos{1.0f, 2.0f, 3.0f};
    const auto houdini_pos = tdm::convert_position(blender_pos, blender_cs, houdini_cs);
    ASSERT_NEAR(houdini_pos[0], 1.0f, 0.001f);
    ASSERT_NEAR(houdini_pos[1], 3.0f, 0.001f);
    ASSERT_NEAR(houdini_pos[2], -2.0f, 0.001f);
}

// Converting from a Y-up system to a Z-up system should produce a positive Z value for an object that is "above ground".
TEST(TestCoordSys, YUpToZUp_PositiveHeight) {
    // Source: X=left, Y=up, Z=front (the DNA source system)
    // Destination: X=left, Y=back, Z=up (the user's target system)
    constexpr tdm::coord_sys src_cs{tdm::axis_dir::left, tdm::axis_dir::up, tdm::axis_dir::front};
    constexpr tdm::coord_sys dst_cs{tdm::axis_dir::left, tdm::axis_dir::back, tdm::axis_dir::up};

    // Y=107 in the source system
    const tdm::fvec3 src_pos{0.0f, 107.0f, 0.0f};
    const auto dst_pos = tdm::convert_position(src_pos, src_cs, dst_cs);

    // In the destination system, Z=up, so Z must be +107 (not -107)
    ASSERT_NEAR(dst_pos[0], 0.0f, 0.001f);
    ASSERT_NEAR(dst_pos[1], 0.0f, 0.001f);
    ASSERT_NEAR(dst_pos[2], 107.0f, 0.001f);
}

// ================ Coordinate System Handedness Tests ================
TEST(TestCoordSys, MayaHandedness) {
    // Maya coordinate system (right-handed)
    ASSERT_EQ(maya_cs.handedness<float>(), tdm::chirality::right);
}

TEST(TestCoordSys, UEHandedness) {
    // UE coordinate system (left-handed)
    ASSERT_EQ(ue_cs.handedness<float>(), tdm::chirality::left);
}

TEST(TestCoordSys, BlenderHandedness) {
    // Blender: X=right, Y=front (into screen), Z=up should be right-handed
    ASSERT_EQ(blender_cs.handedness<float>(), tdm::chirality::right);
}

TEST(TestCoordSys, HoudiniHandedness) {
    // Houdini: X=right, Y=up, Z=back (towards viewer) should be right-handed
    ASSERT_EQ(houdini_cs.handedness<float>(), tdm::chirality::right);
}

TEST(TestCoordSys, AllCoordSysValid) {
    // All defined coordinate systems should be valid
    ASSERT_TRUE(maya_cs.valid<float>());
    ASSERT_TRUE(ue_cs.valid<float>());
    ASSERT_TRUE(blender_cs.valid<float>());
    ASSERT_TRUE(houdini_cs.valid<float>());
}
