// Copyright Epic Games, Inc. All Rights Reserved.

#include "tdmtests/Defs.h"
#include "tdmtests/Helpers.h"

#include "tdm/TDM.h"

TEST(MatTestConstruction, DefaultConstructMat4) {
    tdm::mat4<int> m;
    tdm::vec4<int> v{};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
    ASSERT_EQ(m[2], v);
    ASSERT_EQ(m[3], v);
}

TEST(MatTestConstruction, DefaultConstructMat5) {
    tdm::mat<5, 5, int> m;
    tdm::vec<5, int> v{};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
    ASSERT_EQ(m[2], v);
    ASSERT_EQ(m[3], v);
    ASSERT_EQ(m[4], v);
}

TEST(MatTestConstruction, ConstructFromScalar) {
    tdm::mat4<int> m{4};
    tdm::vec4<int> v{4};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
    ASSERT_EQ(m[2], v);
    ASSERT_EQ(m[3], v);
}

TEST(MatTestConstruction, ConstructFromScalarValues) {
    tdm::mat4<int> m{1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16};
    ASSERT_EQ(m[0], tdm::vec4<int>(1, 2, 3, 4));
    ASSERT_EQ(m[1], tdm::vec4<int>(5, 6, 7, 8));
    ASSERT_EQ(m[2], tdm::vec4<int>(9, 10, 11, 12));
    ASSERT_EQ(m[3], tdm::vec4<int>(13, 14, 15, 16));
}

TEST(MatTestConstruction, ConstructFromVec2s) {
    tdm::vec2<int> v{2};
    tdm::mat2<int> m{v, v};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
}

TEST(MatTestConstruction, ConstructFromVec3s) {
    tdm::vec3<int> v{3};
    tdm::mat3<int> m{v, v, v};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
    ASSERT_EQ(m[2], v);
}

TEST(MatTestConstruction, ConstructFromVec4s) {
    tdm::vec4<int> v{4};
    tdm::mat4<int> m{v, v, v, v};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
    ASSERT_EQ(m[2], v);
    ASSERT_EQ(m[3], v);
}

TEST(MatTestConstruction, ConstructFromArbitraryVecs) {
    tdm::vec<5, int> v{5};
    tdm::mat<5, 5, int> m{v, v, v, v, v};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
    ASSERT_EQ(m[2], v);
    ASSERT_EQ(m[3], v);
    ASSERT_EQ(m[4], v);
}

TEST(MatTestConstruction, ConstructSquareFromRows) {
    tdm::vec4<int> v{1, 2, 3, 4};
    tdm::mat4<int> m = tdm::mat4<int>::from_rows(v, v, v, v);
    ASSERT_EQ(m[0], tdm::vec4<int>(1, 2, 3, 4));
    ASSERT_EQ(m[1], tdm::vec4<int>(1, 2, 3, 4));
    ASSERT_EQ(m[2], tdm::vec4<int>(1, 2, 3, 4));
    ASSERT_EQ(m[3], tdm::vec4<int>(1, 2, 3, 4));
}

TEST(MatTestConstruction, ConstructNonSquareFromRows) {
    tdm::vec2<int> v{1, 2};
    tdm::mat<4, 2, int> m = tdm::mat<4, 2, int>::from_rows(v, v, v, v);
    ASSERT_EQ(m[0], tdm::vec2<int>(1, 2));
    ASSERT_EQ(m[1], tdm::vec2<int>(1, 2));
    ASSERT_EQ(m[2], tdm::vec2<int>(1, 2));
    ASSERT_EQ(m[3], tdm::vec2<int>(1, 2));
}

TEST(MatTestConstruction, ConstructSquareFromCols) {
    tdm::vec4<int> v{1, 2, 3, 4};
    tdm::mat4<int> m = tdm::mat4<int>::from_columns(v, v, v, v);
    ASSERT_EQ(m[0], tdm::vec4<int>(1, 1, 1, 1));
    ASSERT_EQ(m[1], tdm::vec4<int>(2, 2, 2, 2));
    ASSERT_EQ(m[2], tdm::vec4<int>(3, 3, 3, 3));
    ASSERT_EQ(m[3], tdm::vec4<int>(4, 4, 4, 4));
}

TEST(MatTestConstruction, ConstructNonSquareFromCols) {
    tdm::vec4<int> v{1, 2, 3, 4};
    tdm::mat<4, 2, int> m = tdm::mat<4, 2, int>::from_columns(v, v);
    ASSERT_EQ(m[0], tdm::vec2<int>(1, 1));
    ASSERT_EQ(m[1], tdm::vec2<int>(2, 2));
    ASSERT_EQ(m[2], tdm::vec2<int>(3, 3));
    ASSERT_EQ(m[3], tdm::vec2<int>(4, 4));
}

TEST(MatTestConstruction, ConstructFromMat) {
    tdm::mat4<long int> m{tdm::mat4<int>{4}};
    tdm::vec4<long int> v{4};
    ASSERT_EQ(m[0], v);
    ASSERT_EQ(m[1], v);
    ASSERT_EQ(m[2], v);
    ASSERT_EQ(m[3], v);
}

TEST(MatTestConstruction, ConstructDiagonalMatFromScalar) {
    auto m = tdm::mat4<int>::diagonal(42);
    ASSERT_EQ(m[0], tdm::vec4<int>(42, 0, 0, 0));
    ASSERT_EQ(m[1], tdm::vec4<int>(0, 42, 0, 0));
    ASSERT_EQ(m[2], tdm::vec4<int>(0, 0, 42, 0));
    ASSERT_EQ(m[3], tdm::vec4<int>(0, 0, 0, 42));
}

TEST(MatTestConstruction, ConstructDiagonalMatFromScalarValues) {
    auto m = tdm::mat4<int>::diagonal(1, 2, 3, 4);
    ASSERT_EQ(m[0], tdm::vec4<int>(1, 0, 0, 0));
    ASSERT_EQ(m[1], tdm::vec4<int>(0, 2, 0, 0));
    ASSERT_EQ(m[2], tdm::vec4<int>(0, 0, 3, 0));
    ASSERT_EQ(m[3], tdm::vec4<int>(0, 0, 0, 4));
}

TEST(MatTestConstruction, ConstructDiagonalMatFromVec) {
    auto m = tdm::mat4<int>::diagonal(tdm::vec4<int>{42});
    ASSERT_EQ(m[0], tdm::vec4<int>(42, 0, 0, 0));
    ASSERT_EQ(m[1], tdm::vec4<int>(0, 42, 0, 0));
    ASSERT_EQ(m[2], tdm::vec4<int>(0, 0, 42, 0));
    ASSERT_EQ(m[3], tdm::vec4<int>(0, 0, 0, 42));
}

TEST(MatTestConstruction, ConstructIdentityMat) {
    auto m = tdm::mat4<int>::identity();
    ASSERT_EQ(m[0], tdm::vec4<int>(1, 0, 0, 0));
    ASSERT_EQ(m[1], tdm::vec4<int>(0, 1, 0, 0));
    ASSERT_EQ(m[2], tdm::vec4<int>(0, 0, 1, 0));
    ASSERT_EQ(m[3], tdm::vec4<int>(0, 0, 0, 1));
}

namespace {

class MatTestOperators : public ::testing::Test {
protected:
    void SetUp() override {
        v1 = tdm::vec4<int>{1, 2, 3, 4};
        v2 = tdm::vec4<int>{5, 6, 7, 8};
        v3 = tdm::vec4<int>{9, 10, 11, 12};
        v4 = tdm::vec4<int>{13, 14, 15, 16};

        m = tdm::mat4<int>{v1, v2, v3, v4};
        m_rev = tdm::mat4<int>{v4, v3, v2, v1};

        invertible = tdm::mat4<int>{
            tdm::vec4<int>{2, 3, 1,  5},
            tdm::vec4<int>{1, 0, 3,  1},
            tdm::vec4<int>{0, 2, -3, 2},
            tdm::vec4<int>{0, 2, 3,  1}
        };
    }

protected:
    tdm::vec4<int> v1;
    tdm::vec4<int> v2;
    tdm::vec4<int> v3;
    tdm::vec4<int> v4;
    tdm::mat4<int> m;
    tdm::mat4<int> m_rev;
    tdm::mat4<int> invertible;
};

}  // namespace

TEST_F(MatTestOperators, AssignMat) {
    tdm::vec4<long int> v{4};
    tdm::mat4<long int> lm;
    lm = tdm::mat4<int>{4};
    ASSERT_EQ(lm[0], v);
    ASSERT_EQ(lm[1], v);
    ASSERT_EQ(lm[2], v);
    ASSERT_EQ(lm[3], v);
}

TEST_F(MatTestOperators, TestSubscript) {
    ASSERT_EQ(m[0], v1);
    ASSERT_EQ(m[1], v2);
    ASSERT_EQ(m[2], v3);
    ASSERT_EQ(m[3], v4);
}

TEST_F(MatTestOperators, TestLookup) {
    ASSERT_EQ(m(0, 0), 1);
    ASSERT_EQ(m(3, 3), 16);
    ASSERT_EQ(m(1, 2), 7);
    ASSERT_EQ(m(2, 0), 9);
}

TEST_F(MatTestOperators, TestIncrement) {
    ++m;
    ASSERT_EQ(m[0], v1 + 1);
    ASSERT_EQ(m[1], v2 + 1);
    ASSERT_EQ(m[2], v3 + 1);
    ASSERT_EQ(m[3], v4 + 1);
}

TEST_F(MatTestOperators, TestDecrement) {
    --m;
    ASSERT_EQ(m[0], v1 - 1);
    ASSERT_EQ(m[1], v2 - 1);
    ASSERT_EQ(m[2], v3 - 1);
    ASSERT_EQ(m[3], v4 - 1);
}

TEST_F(MatTestOperators, TestCompoundAssignmentAddScalar) {
    m += 2;
    ASSERT_EQ(m[0], v1 + 2);
    ASSERT_EQ(m[1], v2 + 2);
    ASSERT_EQ(m[2], v3 + 2);
    ASSERT_EQ(m[3], v4 + 2);
}

TEST_F(MatTestOperators, TestCompoundAssignmentAddInitializer) {
    m += {v1, v2, v3, v4};
    ASSERT_EQ(m[0], v1 + v1);
    ASSERT_EQ(m[1], v2 + v2);
    ASSERT_EQ(m[2], v3 + v3);
    ASSERT_EQ(m[3], v4 + v4);
}

TEST_F(MatTestOperators, TestCompoundAssignmentAddMat) {
    m += m_rev;
    ASSERT_EQ(m[0], v1 + v4);
    ASSERT_EQ(m[1], v2 + v3);
    ASSERT_EQ(m[2], v3 + v2);
    ASSERT_EQ(m[3], v4 + v1);
}

TEST_F(MatTestOperators, TestCompoundAssignmentSubtractScalar) {
    m -= 2;
    ASSERT_EQ(m[0], v1 - 2);
    ASSERT_EQ(m[1], v2 - 2);
    ASSERT_EQ(m[2], v3 - 2);
    ASSERT_EQ(m[3], v4 - 2);
}

TEST_F(MatTestOperators, TestCompoundAssignmentSubtractInitializer) {
    m -= {v4, v3, v2, v1};
    ASSERT_EQ(m[0], v1 - v4);
    ASSERT_EQ(m[1], v2 - v3);
    ASSERT_EQ(m[2], v3 - v2);
    ASSERT_EQ(m[3], v4 - v1);
}

TEST_F(MatTestOperators, TestCompoundAssignmentSubtractMat) {
    m -= m_rev;
    ASSERT_EQ(m[0], v1 - v4);
    ASSERT_EQ(m[1], v2 - v3);
    ASSERT_EQ(m[2], v3 - v2);
    ASSERT_EQ(m[3], v4 - v1);
}

TEST_F(MatTestOperators, TestCompoundAssignmentMultiplyScalar) {
    m *= 2;
    ASSERT_EQ(m[0], v1 * 2);
    ASSERT_EQ(m[1], v2 * 2);
    ASSERT_EQ(m[2], v3 * 2);
    ASSERT_EQ(m[3], v4 * 2);
}

TEST_F(MatTestOperators, TestCompoundAssignmentMultiplyInitializer) {
    m *= {v4, v3, v2, v1};
    ASSERT_EQ(m[0], tdm::vec4<int>(50, 60, 70, 80));
    ASSERT_EQ(m[1], tdm::vec4<int>(162, 188, 214, 240));
    ASSERT_EQ(m[2], tdm::vec4<int>(274, 316, 358, 400));
    ASSERT_EQ(m[3], tdm::vec4<int>(386, 444, 502, 560));
}

TEST_F(MatTestOperators, TestCompoundAssignmentMultiplyMat) {
    m *= m_rev;
    ASSERT_EQ(m[0], tdm::vec4<int>(50, 60, 70, 80));
    ASSERT_EQ(m[1], tdm::vec4<int>(162, 188, 214, 240));
    ASSERT_EQ(m[2], tdm::vec4<int>(274, 316, 358, 400));
    ASSERT_EQ(m[3], tdm::vec4<int>(386, 444, 502, 560));
}

TEST_F(MatTestOperators, TestCompoundAssignmentDivideScalar) {
    m /= 2;
    ASSERT_EQ(m[0], v1 / 2);
    ASSERT_EQ(m[1], v2 / 2);
    ASSERT_EQ(m[2], v3 / 2);
    ASSERT_EQ(m[3], v4 / 2);
}

TEST_F(MatTestOperators, TestCompoundAssignmentDivideInitializer) {
    tdm::mat4<float> fm{
        tdm::vec4<float>{0.6f, 0.2f, 0.3f, 0.4f},
        tdm::vec4<float>{0.2f, 0.7f, 0.5f, 0.3f},
        tdm::vec4<float>{0.3f, 0.5f, 0.7f, 0.2f},
        tdm::vec4<float>{0.4f, 0.3f, 0.2f, 0.6f}
    };
    fm /= {fm[0], fm[1], fm[2], fm[3]};
    tdm::mat4<float> expected{
        tdm::vec4<float>{1.0f,          -5.96046e-08f, 2.38419e-07f, -2.38419e-07f},
        tdm::vec4<float>{-1.19209e-07f, 1.0f,          4.47035e-07f, -1.19209e-07f},
        tdm::vec4<float>{-1.19209e-07f, 2.98023e-07f,  1.0f,         -2.98023e-07f},
        tdm::vec4<float>{-1.19209e-07f, 0.0f,          2.98023e-07f, 1.0f         }
    };
    ASSERT_MAT_NEAR(fm, expected, 0.0001f);
}

TEST_F(MatTestOperators, TestCompoundAssignmentDivideMat) {
    tdm::mat4<float> fm{
        tdm::vec4<float>{0.6f, 0.2f, 0.3f, 0.4f},
        tdm::vec4<float>{0.2f, 0.7f, 0.5f, 0.3f},
        tdm::vec4<float>{0.3f, 0.5f, 0.7f, 0.2f},
        tdm::vec4<float>{0.4f, 0.3f, 0.2f, 0.6f}
    };
    tdm::mat4<float> divisor{fm};
    fm /= divisor;
    tdm::mat4<float> expected{
        tdm::vec4<float>{1.0f,          -5.96046e-08f, 2.38419e-07f, -2.38419e-07f},
        tdm::vec4<float>{-1.19209e-07f, 1.0f,          4.47035e-07f, -1.19209e-07f},
        tdm::vec4<float>{-1.19209e-07f, 2.98023e-07f,  1.0f,         -2.98023e-07f},
        tdm::vec4<float>{-1.19209e-07f, 0.0f,          2.98023e-07f, 1.0f         }
    };
    ASSERT_MAT_NEAR(fm, expected, 0.0001f);
}

TEST_F(MatTestOperators, TestUnaryPlus) {
    tdm::vec4<int> v{-4, -3, 5, 6};
    tdm::mat4<int> m1{v, v2, v3, v4};
    tdm::mat4<int> m2 = +m1;
    ASSERT_EQ(m2[0], v);
    ASSERT_EQ(m2[1], v2);
    ASSERT_EQ(m2[2], v3);
    ASSERT_EQ(m2[3], v4);
}

TEST_F(MatTestOperators, TestUnaryMinus) {
    tdm::mat4<int> m1 = -m;
    ASSERT_EQ(m1[0], -v1);
    ASSERT_EQ(m1[1], -v2);
    ASSERT_EQ(m1[2], -v3);
    ASSERT_EQ(m1[3], -v4);
}

TEST_F(MatTestOperators, TestEquality) {
    auto m1 = m;
    ASSERT_TRUE(m == m1);
    ASSERT_FALSE(m == m_rev);
}

TEST_F(MatTestOperators, TestNonEquality) {
    auto m1 = m;
    ASSERT_FALSE(m != m1);
    ASSERT_TRUE(m != m_rev);
}

TEST_F(MatTestOperators, TestAddScalarMat) {
    auto res = 2 + m;
    ASSERT_EQ(res[0], v1 + 2);
    ASSERT_EQ(res[1], v2 + 2);
    ASSERT_EQ(res[2], v3 + 2);
    ASSERT_EQ(res[3], v4 + 2);
}

TEST_F(MatTestOperators, TestAddMatScalar) {
    auto res = m + 2;
    ASSERT_EQ(res[0], v1 + 2);
    ASSERT_EQ(res[1], v2 + 2);
    ASSERT_EQ(res[2], v3 + 2);
    ASSERT_EQ(res[3], v4 + 2);
}

TEST_F(MatTestOperators, TestAddMatMat) {
    auto res = m + m_rev;
    ASSERT_EQ(res[0], v1 + v4);
    ASSERT_EQ(res[1], v2 + v3);
    ASSERT_EQ(res[2], v3 + v2);
    ASSERT_EQ(res[3], v4 + v1);
}

TEST_F(MatTestOperators, TestSubtractScalarMat) {
    auto res = 2 - m;
    ASSERT_EQ(res[0], 2 - v1);
    ASSERT_EQ(res[1], 2 - v2);
    ASSERT_EQ(res[2], 2 - v3);
    ASSERT_EQ(res[3], 2 - v4);
}

TEST_F(MatTestOperators, TestSubtractMatScalar) {
    auto res = m - 2;
    ASSERT_EQ(res[0], v1 - 2);
    ASSERT_EQ(res[1], v2 - 2);
    ASSERT_EQ(res[2], v3 - 2);
    ASSERT_EQ(res[3], v4 - 2);
}

TEST_F(MatTestOperators, TestSubtractMatMat) {
    auto res = m - m_rev;
    ASSERT_EQ(res[0], v1 - v4);
    ASSERT_EQ(res[1], v2 - v3);
    ASSERT_EQ(res[2], v3 - v2);
    ASSERT_EQ(res[3], v4 - v1);
}

TEST_F(MatTestOperators, TestMultiplyScalarMat) {
    auto res = 2 * m;
    ASSERT_EQ(res[0], v1 * 2);
    ASSERT_EQ(res[1], v2 * 2);
    ASSERT_EQ(res[2], v3 * 2);
    ASSERT_EQ(res[3], v4 * 2);
}

TEST_F(MatTestOperators, TestMultiplyMatScalar) {
    auto res = m * 2;
    ASSERT_EQ(res[0], v1 * 2);
    ASSERT_EQ(res[1], v2 * 2);
    ASSERT_EQ(res[2], v3 * 2);
    ASSERT_EQ(res[3], v4 * 2);
}

TEST_F(MatTestOperators, TestMultiplyMat4x4Mat4x4) {
    auto res = m * m_rev;
    ASSERT_EQ(res[0], tdm::vec4<int>(50, 60, 70, 80));
    ASSERT_EQ(res[1], tdm::vec4<int>(162, 188, 214, 240));
    ASSERT_EQ(res[2], tdm::vec4<int>(274, 316, 358, 400));
    ASSERT_EQ(res[3], tdm::vec4<int>(386, 444, 502, 560));
}

TEST_F(MatTestOperators, TestMultiplyMat4x3Mat3x4) {
    tdm::mat<4, 3, int> m4x3{1, 2, 3, 5, 6, 7, 9, 10, 11, 13, 14, 15};
    tdm::mat<3, 2, int> m3x2{3, 4, 7, 8, 11, 12};
    tdm::mat<4, 2, int> res = m4x3 * m3x2;
    ASSERT_EQ(res[0], tdm::vec2<int>(50, 56));
    ASSERT_EQ(res[1], tdm::vec2<int>(134, 152));
    ASSERT_EQ(res[2], tdm::vec2<int>(218, 248));
    ASSERT_EQ(res[3], tdm::vec2<int>(302, 344));
}

TEST_F(MatTestOperators, TestMultiplyMat4x4Vec) {
    tdm::vec4<int> v{2, 3, 4, 5};
    auto res = m * v;
    ASSERT_EQ(res, tdm::vec4<int>(40, 96, 152, 208));
}

TEST_F(MatTestOperators, TestMultiplyVecMat4x4) {
    tdm::vec4<int> v{2, 3, 4, 5};
    auto res = v * m;
    ASSERT_EQ(res, tdm::vec4<int>(118, 132, 146, 160));
}

TEST_F(MatTestOperators, TestMultiplyMat4x3Vec) {
    tdm::mat<4, 3, int> m4x3{1, 2, 3, 5, 6, 7, 9, 10, 11, 13, 14, 15};
    tdm::vec3<int> v{2, 3, 4};
    tdm::vec4<int> res = m4x3 * v;
    ASSERT_EQ(res, tdm::vec4<int>(20, 56, 92, 128));
}

TEST_F(MatTestOperators, TestMultiplyVecMat4x3) {
    tdm::vec4<int> v{2, 3, 4, 5};
    tdm::mat<4, 3, int> m4x3{1, 2, 3, 5, 6, 7, 9, 10, 11, 13, 14, 15};
    tdm::vec3<int> res = v * m4x3;
    ASSERT_EQ(res, tdm::vec3<int>(118, 132, 146));
}

TEST_F(MatTestOperators, TestDivideScalarMat) {
    auto res = 2 / m;
    ASSERT_EQ(res[0], 2 / v1);
    ASSERT_EQ(res[1], 2 / v2);
    ASSERT_EQ(res[2], 2 / v3);
    ASSERT_EQ(res[3], 2 / v4);
}

TEST_F(MatTestOperators, TestDivideMatScalar) {
    auto res = m / 2;
    ASSERT_EQ(res[0], v1 / 2);
    ASSERT_EQ(res[1], v2 / 2);
    ASSERT_EQ(res[2], v3 / 2);
    ASSERT_EQ(res[3], v4 / 2);
}

TEST_F(MatTestOperators, TestDivideMatMat) {
    const tdm::mat4<float> fm{
        tdm::vec4<float>{0.6f, 0.2f, 0.3f, 0.4f},
        tdm::vec4<float>{0.2f, 0.7f, 0.5f, 0.3f},
        tdm::vec4<float>{0.3f, 0.5f, 0.7f, 0.2f},
        tdm::vec4<float>{0.4f, 0.3f, 0.2f, 0.6f}
    };
    const tdm::mat4<float> divisor{fm[0], fm[1], fm[2], fm[3]};
    auto res = fm / divisor;
    tdm::mat4<float> expected{
        tdm::vec4<float>{1.0f,          -5.96046e-08f, 2.38419e-07f, -2.38419e-07f},
        tdm::vec4<float>{-1.19209e-07f, 1.0f,          4.47035e-07f, -1.19209e-07f},
        tdm::vec4<float>{-1.19209e-07f, 2.98023e-07f,  1.0f,         -2.98023e-07f},
        tdm::vec4<float>{-1.19209e-07f, 0.0f,          2.98023e-07f, 1.0f         }
    };
    ASSERT_MAT_NEAR(res, expected, 0.0001f);
}

TEST_F(MatTestOperators, TestDivideMatVec) {
    tdm::vec4<int> v{2, 3, 4, 5};
    auto res = invertible / v;
    ASSERT_EQ(res, tdm::vec4<int>(-176, -87, 20, 119));
}

TEST_F(MatTestOperators, TestDivideVecMat) {
    tdm::vec4<int> v{2, 3, 4, 5};
    auto res = v / invertible;
    ASSERT_EQ(res, tdm::vec4<int>(-5, 12, 9, 0));
}

namespace {

class MatTestMembers : public ::testing::Test {
protected:
    void SetUp() override {
        v1 = tdm::vec4<int>{1, 2, 3, 4};
        v2 = tdm::vec4<int>{5, 6, 7, 8};
        v3 = tdm::vec4<int>{9, 10, 11, 12};
        v4 = tdm::vec4<int>{13, 14, 15, 16};
        m = tdm::mat4<int>{v1, v2, v3, v4};
    }

protected:
    tdm::vec4<int> v1;
    tdm::vec4<int> v2;
    tdm::vec4<int> v3;
    tdm::vec4<int> v4;
    tdm::mat4<int> m;
};

}  // namespace

TEST_F(MatTestMembers, TestRow) {
    ASSERT_EQ(m.row(0), v1);
    ASSERT_EQ(m.row(1), v2);
    ASSERT_EQ(m.row(2), v3);
    ASSERT_EQ(m.row(3), v4);
}

TEST_F(MatTestMembers, TestCol) {
    ASSERT_EQ(m.column(0), tdm::vec4<int>(1, 5, 9, 13));
    ASSERT_EQ(m.column(1), tdm::vec4<int>(2, 6, 10, 14));
    ASSERT_EQ(m.column(2), tdm::vec4<int>(3, 7, 11, 15));
    ASSERT_EQ(m.column(3), tdm::vec4<int>(4, 8, 12, 16));
}

TEST_F(MatTestMembers, TestSubMatSquare) {
    tdm::mat2<int> sm2 = m.submat<2, 2>(1, 1);
    ASSERT_EQ(sm2[0], tdm::vec2<int>(6, 7));
    ASSERT_EQ(sm2[1], tdm::vec2<int>(10, 11));

    tdm::mat3<int> sm3 = m.submat<3, 3>(1, 0);
    ASSERT_EQ(sm3[0], tdm::vec3<int>(5, 6, 7));
    ASSERT_EQ(sm3[1], tdm::vec3<int>(9, 10, 11));
    ASSERT_EQ(sm3[2], tdm::vec3<int>(13, 14, 15));
}

TEST_F(MatTestMembers, TestSubMatNonSquare) {
    tdm::mat<2, 3, int> sm2 = m.submat<2, 3>(1, 1);
    ASSERT_EQ(sm2[0], tdm::vec3<int>(6, 7, 8));
    ASSERT_EQ(sm2[1], tdm::vec3<int>(10, 11, 12));

    tdm::mat<3, 4, int> sm3 = m.submat<3, 4>(1, 0);
    ASSERT_EQ(sm3[0], tdm::vec4<int>(5, 6, 7, 8));
    ASSERT_EQ(sm3[1], tdm::vec4<int>(9, 10, 11, 12));
    ASSERT_EQ(sm3[2], tdm::vec4<int>(13, 14, 15, 16));
}

TEST_F(MatTestMembers, TestApplyMat3) {
    std::size_t count{};
    tdm::mat3<int> m1{
        tdm::vec3<int>{1, 2, 3},
        tdm::vec3<int>{4, 5, 6},
        tdm::vec3<int>{7, 8, 9}
    };
    m1.apply([&](tdm::mat3<int>::row_type& row, tdm::dim_t /*unused*/) {
        ASSERT_EQ(row, m1[count]);
        ++count;
    });
    ASSERT_EQ(count, 3u);
}

TEST_F(MatTestMembers, TestApplyMat4) {
    std::size_t count{};
    m.apply([&](tdm::mat4<int>::row_type& row, tdm::dim_t /*unused*/) {
        ASSERT_EQ(row, m[count]);
        ++count;
    });
    ASSERT_EQ(count, 4u);
}

TEST_F(MatTestMembers, TestTranspose) {
    auto m_copy = m;
    m.transpose();
    ASSERT_EQ(m[0], m_copy.column(0));
    ASSERT_EQ(m[1], m_copy.column(1));
    ASSERT_EQ(m[2], m_copy.column(2));
    ASSERT_EQ(m[3], m_copy.column(3));
}
