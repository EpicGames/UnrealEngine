// Copyright Epic Games, Inc. All Rights Reserved.

#include "tdmtests/Defs.h"
#include "tdmtests/Helpers.h"

#include "tdm/TDM.h"

TEST(QuatTestConstruction, FromEulerAnglesXYZ1) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{30.0_fdeg}, tdm::frad{-45.0_fdeg}, tdm::frad{-15.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::xyz,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic XYZ: Rx * Ry * Rz (swapped from old zyx)
    ASSERT_NEAR(q.x, 0.1888237f, 1e-4f);
    ASSERT_NEAR(q.y, -0.3976926f, 1e-4f);
    ASSERT_NEAR(q.z, -0.018283f, 1e-4f);
    ASSERT_NEAR(q.w, 0.8976926f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesXZY1) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{30.0_fdeg}, tdm::frad{-45.0_fdeg}, tdm::frad{-15.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::xzy,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic XZY: Rx * Rz * Ry (swapped from old yzx)
    ASSERT_NEAR(q.x, 0.2853201f, 1e-4f);
    ASSERT_NEAR(q.y, -0.3976926f, 1e-4f);
    ASSERT_NEAR(q.z, -0.018283f, 1e-4f);
    ASSERT_NEAR(q.w, 0.8718364f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesYXZ1) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{30.0_fdeg}, tdm::frad{-45.0_fdeg}, tdm::frad{-15.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::yxz,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic YXZ: Ry * Rx * Rz (swapped from old zxy)
    ASSERT_NEAR(q.x, 0.1888237f, 1e-4f);
    ASSERT_NEAR(q.y, -0.3976926f, 1e-4f);
    ASSERT_NEAR(q.z, -0.2146799f, 1e-4f);
    ASSERT_NEAR(q.w, 0.8718364f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesYZX1) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{30.0_fdeg}, tdm::frad{-45.0_fdeg}, tdm::frad{-15.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::yzx,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic YZX: Ry * Rz * Rx (swapped from old xzy)
    ASSERT_NEAR(q.x, 0.1888237f, 1e-4f);
    ASSERT_NEAR(q.y, -0.3352703f, 1e-4f);
    ASSERT_NEAR(q.z, -0.2146799f, 1e-4f);
    ASSERT_NEAR(q.w, 0.8976926f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesZXY1) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{30.0_fdeg}, tdm::frad{-45.0_fdeg}, tdm::frad{-15.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::zxy,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic ZXY: Rz * Rx * Ry (swapped from old yxz)
    ASSERT_NEAR(q.x, 0.2853201f, 1e-4f);
    ASSERT_NEAR(q.y, -0.3352703f, 1e-4f);
    ASSERT_NEAR(q.z, -0.018283f, 1e-4f);
    ASSERT_NEAR(q.w, 0.8976926f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesZYX1) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{30.0_fdeg}, tdm::frad{-45.0_fdeg}, tdm::frad{-15.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::zyx,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic ZYX: Rz * Ry * Rx (swapped from old xyz)
    ASSERT_NEAR(q.x, 0.2853201f, 1e-4f);
    ASSERT_NEAR(q.y, -0.3352703f, 1e-4f);
    ASSERT_NEAR(q.z, -0.2146799f, 1e-4f);
    ASSERT_NEAR(q.w, 0.8718364f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesXYZ2) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{0.0_fdeg}, tdm::frad{-30.0_fdeg}, tdm::frad{-30.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::xyz,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic XYZ (swapped from old zyx)
    ASSERT_NEAR(q.x, -0.06698729f, 1e-4f);
    ASSERT_NEAR(q.y, -0.25f, 1e-4f);
    ASSERT_NEAR(q.z, -0.25f, 1e-4f);
    ASSERT_NEAR(q.w, 0.9330127f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesXZY2) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{0.0_fdeg}, tdm::frad{-30.0_fdeg}, tdm::frad{-30.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::xzy,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic XZY (swapped from old yzx)
    ASSERT_NEAR(q.x, 0.0669873f, 1e-4f);
    ASSERT_NEAR(q.y, -0.25f, 1e-4f);
    ASSERT_NEAR(q.z, -0.25f, 1e-4f);
    ASSERT_NEAR(q.w, 0.9330127f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesYXZ2) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{0.0_fdeg}, tdm::frad{-30.0_fdeg}, tdm::frad{-30.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::yxz,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic YXZ (swapped from old zxy)
    ASSERT_NEAR(q.x, -0.0669873f, 1e-4f);
    ASSERT_NEAR(q.y, -0.25f, 1e-4f);
    ASSERT_NEAR(q.z, -0.25f, 1e-4f);
    ASSERT_NEAR(q.w, 0.9330127f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesYZX2) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{0.0_fdeg}, tdm::frad{-30.0_fdeg}, tdm::frad{-30.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::yzx,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic YZX (swapped from old xzy)
    ASSERT_NEAR(q.x, -0.0669873f, 1e-4f);
    ASSERT_NEAR(q.y, -0.25f, 1e-4f);
    ASSERT_NEAR(q.z, -0.25f, 1e-4f);
    ASSERT_NEAR(q.w, 0.9330127f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesZXY2) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{0.0_fdeg}, tdm::frad{-30.0_fdeg}, tdm::frad{-30.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::zxy,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic ZXY (swapped from old yxz)
    ASSERT_NEAR(q.x, 0.0669873f, 1e-4f);
    ASSERT_NEAR(q.y, -0.25f, 1e-4f);
    ASSERT_NEAR(q.z, -0.25f, 1e-4f);
    ASSERT_NEAR(q.w, 0.9330127f, 1e-4f);
}

TEST(QuatTestConstruction, FromEulerAnglesZYX2) {
    using namespace tdm::ang_literals;
    const tdm::frad3 euler{tdm::frad{0.0_fdeg}, tdm::frad{-30.0_fdeg}, tdm::frad{-30.0_fdeg}};
    const tdm::fquat q{
        euler,
        tdm::rot_seq::zyx,
        tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive}
    };
    // Intrinsic ZYX (swapped from old xyz)
    ASSERT_NEAR(q.x, 0.0669873f, 1e-4f);
    ASSERT_NEAR(q.y, -0.25f, 1e-4f);
    ASSERT_NEAR(q.z, -0.25f, 1e-4f);
    ASSERT_NEAR(q.w, 0.9330127f, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesXYZ1) {
    using namespace tdm::ang_literals;
    // Intrinsic XYZ (swapped from old zyx input)
    const tdm::fquat q{0.1888237f, -0.3976926f, -0.018283f, 0.8976926f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::xyz, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-45.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-15.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesXZY1) {
    using namespace tdm::ang_literals;
    // Intrinsic XZY (swapped from old yzx input)
    const tdm::fquat q{0.2853201f, -0.3976926f, -0.018283f, 0.8718364f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::xzy, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-45.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-15.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesYXZ1) {
    using namespace tdm::ang_literals;
    // Intrinsic YXZ (swapped from old zxy input)
    const tdm::fquat q{0.1888237f, -0.3976926f, -0.2146799f, 0.8718364f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::yxz, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-45.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-15.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesYZX1) {
    using namespace tdm::ang_literals;
    // Intrinsic YZX (swapped from old xzy input)
    const tdm::fquat q{0.1888237f, -0.3352703f, -0.2146799f, 0.8976926f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::yzx, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-45.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-15.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesZXY1) {
    using namespace tdm::ang_literals;
    // Intrinsic ZXY (swapped from old yxz input)
    const tdm::fquat q{0.2853201f, -0.3352703f, -0.018283f, 0.8976926f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::zxy, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-45.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-15.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesZYX1) {
    using namespace tdm::ang_literals;
    // Intrinsic ZYX (swapped from old xyz input)
    const tdm::fquat q{0.2853201f, -0.3352703f, -0.2146799f, 0.8718364f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::zyx, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-45.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-15.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesXYZ2) {
    using namespace tdm::ang_literals;
    // Intrinsic XYZ (swapped from old zyx input)
    const tdm::fquat q{-0.06698729f, -0.25f, -0.25f, 0.9330127f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::xyz, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{0.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesXZY2) {
    using namespace tdm::ang_literals;
    // Intrinsic XZY (swapped from old yzx input)
    const tdm::fquat q{0.0669873f, -0.25f, -0.25f, 0.9330127f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::xzy, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{0.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesYXZ2) {
    using namespace tdm::ang_literals;
    // Intrinsic YXZ (swapped from old zxy input)
    const tdm::fquat q{-0.0669873f, -0.25f, -0.25f, 0.9330127f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::yxz, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{0.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesYZX2) {
    using namespace tdm::ang_literals;
    // Intrinsic YZX (swapped from old xzy input)
    const tdm::fquat q{-0.0669873f, -0.25f, -0.25f, 0.9330127f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::yzx, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{0.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesZXY2) {
    using namespace tdm::ang_literals;
    // Intrinsic ZXY (swapped from old yxz input)
    const tdm::fquat q{0.0669873f, -0.25f, -0.25f, 0.9330127f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::zxy, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{0.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, ToEulerAnglesZYX2) {
    using namespace tdm::ang_literals;
    // Intrinsic ZYX (swapped from old xyz input)
    const tdm::fquat q{0.0669873f, -0.25f, -0.25f, 0.9330127f};
    const tdm::frad3 result =
        q.euler(tdm::rot_seq::zyx, tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    ASSERT_NEAR(result[0].value, tdm::frad{0.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[1].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
    ASSERT_NEAR(result[2].value, tdm::frad{-30.0_fdeg}.value, 1e-4f);
}

TEST(QuatTestConstruction, Concatenation) {
    const tdm::fquat q1{0.1888237f, -0.3976926f, -0.018283f, 0.8976926f};
    const tdm::fquat q2{-0.06698729f, -0.24999999f, -0.24999999f, 0.93301270f};
    const tdm::fquat qc = q1 * q2;
    ASSERT_NEAR(qc.x, 0.210893303f, 1e-4f);
    ASSERT_NEAR(qc.y, -0.547044694f, 1e-4f);
    ASSERT_NEAR(qc.z, -0.315327674f, 1e-4f);
    ASSERT_NEAR(qc.w, 0.746213555f, 1e-4f);
}

// Test that all-negative rot_sign quaternion is conjugate to all-positive rot_sign quaternion
// negative_sign(theta) = positive_sign(-theta)
TEST(QuatTestConstruction, AllNegativeSignEqualsConjugateAllPositiveSign) {
    const tdm::frad3 angles{tdm::frad{tdm::fdeg{30.0f}}, tdm::frad{tdm::fdeg{-45.0f}}, tdm::frad{tdm::fdeg{60.0f}}};

    // Build quaternion with positive rotation direction convention
    const tdm::fquat quat_pos(angles,
                              tdm::rot_seq::xyz,
                              tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});

    // Build quaternion with positive convention using negated angles
    const tdm::frad3 negated_angles = -angles;
    const tdm::fquat quat_pos_negated(negated_angles,
                                      tdm::rot_seq::xyz,
                                      tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});

    // Build quaternion with negative rotation direction convention
    const tdm::fquat quat_neg(angles,
                              tdm::rot_seq::xyz,
                              tdm::rot_sign{tdm::rot_dir::negative, tdm::rot_dir::negative, tdm::rot_dir::negative});

    // negative_sign(theta) should equal positive_sign(-theta)
    ASSERT_NEAR(quat_neg.x, quat_pos_negated.x, 0.0001f);
    ASSERT_NEAR(quat_neg.y, quat_pos_negated.y, 0.0001f);
    ASSERT_NEAR(quat_neg.z, quat_pos_negated.z, 0.0001f);
    ASSERT_NEAR(quat_neg.w, quat_pos_negated.w, 0.0001f);
}

// Test that new dst_signs {+,-,+} with negative rot_sign quat = old dst_signs {-,+,-} with positive rot_sign quat
TEST(QuatTestConstruction, RotationSignAbleToProduceSameQuaternionWithDifferentInputSigns) {
    const tdm::frad3 extracted_angles{tdm::frad{tdm::fdeg{45.0f}}, tdm::frad{tdm::fdeg{30.0f}}, tdm::frad{tdm::fdeg{-60.0f}}};

    // Old scheme: apply {-,+,-} then build positive rotation direction quat
    const tdm::frad3 old_adjusted{-extracted_angles[0], extracted_angles[1], -extracted_angles[2]};
    const tdm::fquat quat_old(old_adjusted,
                              tdm::rot_seq::xyz,
                              tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});

    // New scheme: apply {+,-,+} then build negative rotation direction quat
    const tdm::frad3 new_adjusted{extracted_angles[0], -extracted_angles[1], extracted_angles[2]};
    const tdm::fquat quat_new(new_adjusted,
                              tdm::rot_seq::xyz,
                              tdm::rot_sign{tdm::rot_dir::negative, tdm::rot_dir::negative, tdm::rot_dir::negative});

    // Both should produce the same quaternion
    ASSERT_NEAR(quat_old.x, quat_new.x, 0.0001f);
    ASSERT_NEAR(quat_old.y, quat_new.y, 0.0001f);
    ASSERT_NEAR(quat_old.z, quat_new.z, 0.0001f);
    ASSERT_NEAR(quat_old.w, quat_new.w, 0.0001f);
}
