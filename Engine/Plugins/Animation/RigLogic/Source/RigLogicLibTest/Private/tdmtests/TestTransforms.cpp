// Copyright Epic Games, Inc. All Rights Reserved.

#include "tdmtests/Defs.h"
#include "tdmtests/Helpers.h"

#include "tdm/TDM.h"

TEST(TestTransforms, Rotate) {
    using namespace tdm::ang_literals;
    tdm::vec3<float> axis{0.0f, 0.0f, 1.0f};
    auto m = tdm::rotate(axis, tdm::frad{90.0_fdeg}, tdm::rot_dir::positive);
    tdm::vec4<float> v{1.0f, 0.0f, 0.0f, 1.0f};
    auto result = v * m;
    tdm::vec4<float> expected{0.0f, -1.0f, 0.0f, 1.0f};
    ASSERT_VEC_NEAR(result, expected, 0.0001f);
}

TEST(TestTransforms, RotateNegativeDirection) {
    using namespace tdm::ang_literals;
    tdm::vec3<float> axis{0.5f, 0.4f, 0.3f};
    auto m = tdm::rotate(axis, tdm::frad{45.0_fdeg}, tdm::rot_dir::negative);
    tdm::mat4<float> expected{0.853553f,
                              0.417157f,
                              -0.312132f,
                              0.0f,
                              -0.182843f,
                              0.800833f,
                              0.570294f,
                              0.0f,
                              0.487868f,
                              -0.429706f,
                              0.759828f,
                              0.0f,
                              0.0f,
                              0.0f,
                              0.0f,
                              1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, ScaleUniform) {
    auto m = tdm::mat4<float>::identity();
    m = tdm::scale(m, 3.0f);
    tdm::mat4<float> expected{3.0f, 0.0f, 0.0f, 0.0f, 0.0f, 3.0f, 0.0f, 0.0f, 0.0f, 0.0f, 3.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, ScaleNonUniform) {
    auto m = tdm::mat4<float>::identity();
    m = tdm::scale(m, tdm::vec3<float>{2.0f, 3.0f, 4.0f});
    tdm::mat4<float> expected{2.0f, 0.0f, 0.0f, 0.0f, 0.0f, 3.0f, 0.0f, 0.0f, 0.0f, 0.0f, 4.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, Translate) {
    auto m = tdm::mat4<float>::identity();
    m = tdm::translate(m, tdm::vec3<float>{2.0f, 3.0f, 4.0f});
    tdm::mat4<float> expected{1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 2.0f, 3.0f, 4.0f, 1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, CreateRotationMatrixFromXYZAngles) {
    using namespace tdm::ang_literals;
    tdm::mat4<float> m =
        tdm::rotate<float>(tdm::frad{90.0_fdeg},
                           tdm::frad{0.0_fdeg},
                           tdm::frad{90.0_fdeg},
                           tdm::rot_seq::xyz,
                           tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    tdm::mat4<float> expected{0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, CreateRotationMatrixFromXYZAnglesVector) {
    using namespace tdm::ang_literals;
    tdm::mat4<float> m =
        tdm::rotate<float>({tdm::frad{90.0_fdeg}, tdm::frad{0.0_fdeg}, tdm::frad{90.0_fdeg}},
                           tdm::rot_seq::xyz,
                           tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    tdm::mat4<float> expected{0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f, 0.0f, 1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, RotateByXYZAngles) {
    using namespace tdm::ang_literals;
    auto m = tdm::mat4<float>::identity();
    m = tdm::rotate<float>(m,
                           tdm::frad{45.0_fdeg},
                           tdm::frad{0.0_fdeg},
                           tdm::frad{30.0_fdeg},
                           tdm::rot_seq::xyz,
                           tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    tdm::mat4<float> expected{0.866025f,
                              0.5f,
                              0.0f,
                              0.0f,
                              -0.353553f,
                              0.612372f,
                              0.707107f,
                              0.0f,
                              0.353553f,
                              -0.612372f,
                              0.707107f,
                              0.0f,
                              0.0f,
                              0.0f,
                              0.0f,
                              1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, RotateByXYZAnglesVector) {
    using namespace tdm::ang_literals;
    auto m = tdm::mat4<float>::identity();
    m = tdm::rotate<float>(m,
                           {tdm::frad{45.0_fdeg}, tdm::frad{0.0_fdeg}, tdm::frad{30.0_fdeg}},
                           tdm::rot_seq::xyz,
                           tdm::rot_sign{tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive});
    tdm::mat4<float> expected{0.866025f,
                              0.5f,
                              0.0f,
                              0.0f,
                              -0.353553f,
                              0.612372f,
                              0.707107f,
                              0.0f,
                              0.353553f,
                              -0.612372f,
                              0.707107f,
                              0.0f,
                              0.0f,
                              0.0f,
                              0.0f,
                              1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}

TEST(TestTransforms, RotateByXYZAnglesNegativeDirection) {
    using namespace tdm::ang_literals;
    constexpr tdm::rot_sign rot_sign_negative{tdm::rot_dir::negative, tdm::rot_dir::negative, tdm::rot_dir::negative};
    auto m = tdm::mat4<float>::identity();
    m = tdm::rotate<float>(m,
                           tdm::frad{45.0_fdeg},
                           tdm::frad{0.0_fdeg},
                           tdm::frad{30.0_fdeg},
                           tdm::rot_seq::xyz,
                           rot_sign_negative);
    tdm::mat4<float> expected{0.8660254f,
                              -0.5f,
                              0.0f,
                              0.0f,
                              0.3535534f,
                              0.6123725f,
                              -0.7071068f,
                              0.0f,
                              0.3535534f,
                              0.6123725f,
                              0.7071068f,
                              0.0f,
                              0.0f,
                              0.0f,
                              0.0f,
                              1.0f};
    ASSERT_MAT_NEAR(m, expected, 0.0001f);
}
