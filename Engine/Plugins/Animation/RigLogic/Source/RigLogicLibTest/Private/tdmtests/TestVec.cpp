// Copyright Epic Games, Inc. All Rights Reserved.

#include "tdmtests/Defs.h"
#include "tdmtests/Helpers.h"

#include "tdm/TDM.h"

TEST(VecTestConstruction, DefaultConstructVec1) {
    tdm::vec<1, int> v;
    ASSERT_EQ(v[0], 0);
}

TEST(VecTestConstruction, DefaultConstructVec2) {
    tdm::vec2<int> v;
    ASSERT_EQ(v[0], 0);
    ASSERT_EQ(v[1], 0);
}

TEST(VecTestConstruction, DefaultConstructVec3) {
    tdm::vec3<int> v;
    ASSERT_EQ(v[0], 0);
    ASSERT_EQ(v[1], 0);
    ASSERT_EQ(v[2], 0);
}

TEST(VecTestConstruction, DefaultConstructVec4) {
    tdm::vec4<int> v;
    ASSERT_EQ(v[0], 0);
    ASSERT_EQ(v[1], 0);
    ASSERT_EQ(v[2], 0);
    ASSERT_EQ(v[3], 0);
}

TEST(VecTestConstruction, DefaultConstructVecArbitrary) {
    tdm::vec<5, int> v;
    ASSERT_EQ(v[0], 0);
    ASSERT_EQ(v[1], 0);
    ASSERT_EQ(v[2], 0);
    ASSERT_EQ(v[3], 0);
    ASSERT_EQ(v[4], 0);
}

TEST(VecTestConstruction, ConstructFromArgsVec1) {
    tdm::vec<1, int> v{1};
    ASSERT_EQ(v[0], 1);
}

TEST(VecTestConstruction, ConstructFromArgsVec2) {
    tdm::vec2<int> v{1, 2};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
}

TEST(VecTestConstruction, ConstructFromArgsVec3) {
    tdm::vec3<int> v{1, 2, 3};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
}

TEST(VecTestConstruction, ConstructFromArgsVec4) {
    tdm::vec4<int> v{1, 2, 3, 4};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
}

TEST(VecTestConstruction, ConstructFromArgsVecArbitrary) {
    tdm::vec<5, int> v{1, 2, 3, 4, 5};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
    ASSERT_EQ(v[4], 5);
}

TEST(VecTestConstruction, ConstructFromVec1) {
    tdm::vec<1, long int> v{tdm::vec<1, int>{1}};
    ASSERT_EQ(v[0], 1);
}

TEST(VecTestConstruction, ConstructFromVec2) {
    tdm::vec2<long int> v{
        tdm::vec2<int>{1, 2}
    };
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
}

TEST(VecTestConstruction, ConstructFromVec3) {
    tdm::vec3<long int> v{
        tdm::vec3<int>{1, 2, 3}
    };
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
}

TEST(VecTestConstruction, ConstructFromVec4) {
    tdm::vec4<long int> v{
        tdm::vec4<int>{1, 2, 3, 4}
    };
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
}

TEST(VecTestConstruction, ConstructFromVecArbitrary) {
    tdm::vec<5, long int> v{
        tdm::vec<5, int>{1, 2, 3, 4, 5}
    };
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
    ASSERT_EQ(v[4], 5);
}

TEST(VecTestConstruction, ConstructFromScalarVec1) {
    tdm::vec<1, int> v{42};
    ASSERT_EQ(v[0], 42);
}

TEST(VecTestConstruction, ConstructFromScalarVec2) {
    tdm::vec2<int> v{42};
    ASSERT_EQ(v[0], 42);
    ASSERT_EQ(v[1], 42);
}

TEST(VecTestConstruction, ConstructFromScalarVec3) {
    tdm::vec3<int> v{42};
    ASSERT_EQ(v[0], 42);
    ASSERT_EQ(v[1], 42);
    ASSERT_EQ(v[2], 42);
}

TEST(VecTestConstruction, ConstructFromScalarVec4) {
    tdm::vec4<int> v{42};
    ASSERT_EQ(v[0], 42);
    ASSERT_EQ(v[1], 42);
    ASSERT_EQ(v[2], 42);
    ASSERT_EQ(v[3], 42);
}

TEST(VecTestConstruction, ConstructFromScalarVecArbitrary) {
    tdm::vec<5, int> v{42};
    ASSERT_EQ(v[0], 42);
    ASSERT_EQ(v[1], 42);
    ASSERT_EQ(v[2], 42);
    ASSERT_EQ(v[3], 42);
    ASSERT_EQ(v[4], 42);
}

TEST(VecTestOperators, AssignVec1) {
    tdm::vec<1, long int> v;
    v = tdm::vec<1, int>{1};
    ASSERT_EQ(v[0], 1);
}

TEST(VecTestOperators, AssignVec2) {
    tdm::vec2<long int> v;
    v = tdm::vec2<int>{1, 2};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
}

TEST(VecTestOperators, AssignVec3) {
    tdm::vec3<long int> v;
    v = tdm::vec3<int>{1, 2, 3};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
}

TEST(VecTestOperators, AssignVec4) {
    tdm::vec4<long int> v;
    v = tdm::vec4<int>{1, 2, 3, 4};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
}

TEST(VecTestOperators, AssignVecArbitrary) {
    tdm::vec<5, long int> v1;
    const tdm::vec<5, int> v2{1, 2, 3, 4, 5};
    v1 = v2;
    ASSERT_EQ(v1[0], 1);
    ASSERT_EQ(v1[1], 2);
    ASSERT_EQ(v1[2], 3);
    ASSERT_EQ(v1[3], 4);
    ASSERT_EQ(v1[4], 5);
}

TEST(VecTestOperators, AssignArgsVec1) {
    tdm::fvec<1> v;
    v = {1.0f};
    ASSERT_EQ(v[0], 1.0f);
}

TEST(VecTestOperators, AssignArgsVec2) {
    tdm::fvec<2> v;
    v = {1.0f, 2.0f};
    ASSERT_EQ(v[0], 1.0f);
    ASSERT_EQ(v[1], 2.0f);
}

TEST(VecTestOperators, AssignArgsVec3) {
    tdm::fvec<3> v;
    v = {1.0f, 2.0f, 3.0f};
    ASSERT_EQ(v[0], 1.0f);
    ASSERT_EQ(v[1], 2.0f);
    ASSERT_EQ(v[2], 3.0f);
}

TEST(VecTestOperators, AssignArgsVec4) {
    tdm::fvec<4> v;
    v = {1.0f, 2.0f, 3.0f, 4.0f};
    ASSERT_EQ(v[0], 1.0f);
    ASSERT_EQ(v[1], 2.0f);
    ASSERT_EQ(v[2], 3.0f);
    ASSERT_EQ(v[3], 4.0f);
}

TEST(VecTestOperators, AssignArgsVec5) {
    tdm::fvec<5> v;
    v = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f};
    ASSERT_EQ(v[0], 1.0f);
    ASSERT_EQ(v[1], 2.0f);
    ASSERT_EQ(v[2], 3.0f);
    ASSERT_EQ(v[3], 4.0f);
    ASSERT_EQ(v[4], 5.0f);
}

TEST(VecTestOperators, TestSubscript) {
    tdm::vec4<int> v{1, 2, 3, 4};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
}

TEST(VecTestOperators, TestSubscriptModify) {
    tdm::vec4<int> v{1, 2, 3, 4};
    v[0] = 5;
    ASSERT_EQ(v[0], 5);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
}

TEST(VecTestOperators, TestSubscriptConst) {
    const tdm::vec4<int> v{1, 2, 3, 4};
    ASSERT_EQ(v[0], 1);
    ASSERT_EQ(v[1], 2);
    ASSERT_EQ(v[2], 3);
    ASSERT_EQ(v[3], 4);
}

TEST(VecTestOperators, TestIncrement) {
    tdm::vec4<int> v{1, 2, 3, 4};
    ++v;
    ASSERT_EQ(v, tdm::vec4<int>(2, 3, 4, 5));
}

TEST(VecTestOperators, TestDecrement) {
    tdm::vec4<int> v{1, 2, 3, 4};
    --v;
    ASSERT_EQ(v, tdm::vec4<int>(0, 1, 2, 3));
}

TEST(VecTestOperators, TestCompoundAssignmentAddScalar) {
    tdm::vec4<long int> v{1, 2, 3, 4};
    v += 2;
    ASSERT_EQ(v, tdm::vec4<long int>(3, 4, 5, 6));
}

TEST(VecTestOperators, TestCompoundAssignmentAddInitializer) {
    tdm::vec4<long int> v{1, 2, 3, 4};
    v += {1, 2, 3, 4};
    ASSERT_EQ(v, tdm::vec4<long int>(2, 4, 6, 8));
}

TEST(VecTestOperators, TestCompoundAssignmentAddVec) {
    tdm::vec4<long int> v1{1, 2, 3, 4};
    tdm::vec4<int> v2{1, 2, 3, 4};
    v1 += v2;
    ASSERT_EQ(v1, tdm::vec4<long int>(2, 4, 6, 8));
}

TEST(VecTestOperators, TestCompoundAssignmentSubtractScalar) {
    tdm::vec4<long int> v{5, 8, 11, 14};
    v -= 2;
    ASSERT_EQ(v, tdm::vec4<long int>(3, 6, 9, 12));
}

TEST(VecTestOperators, TestCompoundAssignmentSubtractInitializer) {
    tdm::vec4<long int> v{5, 8, 11, 14};
    v -= {1, 2, 3, 4};
    ASSERT_EQ(v, tdm::vec4<long int>(4, 6, 8, 10));
}

TEST(VecTestOperators, TestCompoundAssignmentSubtractVec) {
    tdm::vec4<long int> v1{5, 8, 11, 14};
    tdm::vec4<int> v2{1, 2, 3, 4};
    v1 -= v2;
    ASSERT_EQ(v1, tdm::vec4<long int>(4, 6, 8, 10));
}

TEST(VecTestOperators, TestCompoundAssignmentMultiplyScalar) {
    tdm::vec4<long int> v{1, 2, 3, 4};
    v *= 2;
    ASSERT_EQ(v, tdm::vec4<long int>(2, 4, 6, 8));
}

TEST(VecTestOperators, TestCompoundAssignmentMultiplyInitializer) {
    tdm::vec4<long int> v{1, 2, 3, 4};
    v *= {1, 2, 3, 4};
    ASSERT_EQ(v, tdm::vec4<long int>(1, 4, 9, 16));
}

TEST(VecTestOperators, TestCompoundAssignmentMultiplyVec) {
    tdm::vec4<long int> v1{1, 2, 3, 4};
    tdm::vec4<int> v2{4, 3, 2, 1};
    v1 *= v2;
    ASSERT_EQ(v1, tdm::vec4<long int>(4, 6, 6, 4));
}

TEST(VecTestOperators, TestCompoundAssignmentDivideScalar) {
    tdm::vec4<long int> v{8, 24, 36, 32};
    v /= 2;
    ASSERT_EQ(v, tdm::vec4<long int>(4, 12, 18, 16));
}

TEST(VecTestOperators, TestCompoundAssignmentDivideInitializer) {
    tdm::vec4<long int> v{8, 24, 36, 32};
    v /= {1, 2, 3, 4};
    ASSERT_EQ(v, tdm::vec4<long int>(8, 12, 12, 8));
}

TEST(VecTestOperators, TestCompoundAssignmentDivideVec) {
    tdm::vec4<long int> v1{8, 24, 36, 32};
    tdm::vec4<int> v2{4, 3, 2, 1};
    v1 /= v2;
    ASSERT_EQ(v1, tdm::vec4<long int>(2, 8, 18, 32));
}

TEST(VecTestOperators, TestAddVecVec) {
    tdm::vec4<int> v1{1, 2, 3, 4};
    tdm::vec4<int> v2{5, 6, 7, 8};
    auto res = v1 + v2;
    ASSERT_EQ(res, tdm::vec4<int>(6, 8, 10, 12));
}

TEST(VecTestOperators, TestAddScalarVec) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = 4 + v;
    ASSERT_EQ(res, tdm::vec4<int>(5, 6, 7, 8));
}

TEST(VecTestOperators, TestAddVecScalar) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = v + 7;
    ASSERT_EQ(res, tdm::vec4<int>(8, 9, 10, 11));
}

TEST(VecTestOperators, TestSubtractVecVec) {
    tdm::vec4<int> v1{1, 2, 3, 4};
    tdm::vec4<int> v2{8, 7, 6, 5};
    auto res = v1 - v2;
    ASSERT_EQ(res, tdm::vec4<int>(-7, -5, -3, -1));
}

TEST(VecTestOperators, TestSubtractScalarVec) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = 4 - v;
    ASSERT_EQ(res, tdm::vec4<int>(3, 2, 1, 0));
}

TEST(VecTestOperators, TestSubtractVecScalar) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = v - 7;
    ASSERT_EQ(res, tdm::vec4<int>(-6, -5, -4, -3));
}

TEST(VecTestOperators, TestMultiplyVecVec) {
    tdm::vec4<int> v1{1, 2, 3, 4};
    tdm::vec4<int> v2{5, 6, 7, 8};
    auto res = v1 * v2;
    ASSERT_EQ(res, tdm::vec4<int>(5, 12, 21, 32));
}

TEST(VecTestOperators, TestMultiplyScalarVec) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = 4 * v;
    ASSERT_EQ(res, tdm::vec4<int>(4, 8, 12, 16));
}

TEST(VecTestOperators, TestMultiplyVecScalar) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = v * -2;
    ASSERT_EQ(res, tdm::vec4<int>(-2, -4, -6, -8));
}

TEST(VecTestOperators, TestDivideVecVec) {
    tdm::vec4<int> v1{1, 2, 3, 4};
    tdm::vec4<int> v2{5, 6, 7, 8};
    auto res = v2 / v1;
    ASSERT_EQ(res, tdm::vec4<int>(5, 3, 2, 2));
}

TEST(VecTestOperators, TestDivideScalarVec) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = 10 / v;
    ASSERT_EQ(res, tdm::vec4<int>(10, 5, 3, 2));
}

TEST(VecTestOperators, TestDivideVecScalar) {
    tdm::vec4<int> v{1, 2, 3, 4};
    auto res = v / 2;
    ASSERT_EQ(res, tdm::vec4<int>(0, 1, 1, 2));
}

TEST(VecTestOperators, TestUnaryPlus) {
    tdm::vec4<int> v1{-4, -3, 5, 6};
    tdm::vec4<int> v2 = +v1;
    ASSERT_EQ(v1, v2);
}

TEST(VecTestOperators, TestUnaryMinus) {
    tdm::vec4<int> v1{-4, -3, 5, 6};
    tdm::vec4<int> v2 = -v1;
    ASSERT_EQ(v2, tdm::vec4<int>(4, 3, -5, -6));
}

TEST(VecTestOperators, TestEquality) {
    tdm::vec4<int> v1{4, 3, 5, 6};
    tdm::vec4<int> v2{4, 3, 5, 6};
    tdm::vec4<int> v3;
    ASSERT_TRUE(v1 == v2);
    ASSERT_FALSE(v1 == v3);
}

TEST(VecTestOperators, TestNonEquality) {
    tdm::vec4<int> v1{4, 3, 5, 6};
    tdm::vec4<int> v2{4, 3, 5, 6};
    tdm::vec4<int> v3;
    ASSERT_FALSE(v1 != v2);
    ASSERT_TRUE(v1 != v3);
}

TEST(VecTestMembers, TestApplyVec1) {
    tdm::vec<1, int> v{1};
    int count{};
    v.apply([&count](int& val, tdm::dim_t /*unused*/) {
        ASSERT_EQ(val, count + 1);
        ++count;
    });
    ASSERT_EQ(count, 1);
}

TEST(VecTestMembers, TestApplyVec2) {
    tdm::vec2<int> v{1, 2};
    int count{};
    v.apply([&count](int& val, tdm::dim_t /*unused*/) {
        ASSERT_EQ(val, count + 1);
        ++count;
    });
    ASSERT_EQ(count, 2);
}

TEST(VecTestMembers, TestApplyVec3) {
    tdm::vec3<int> v{1, 2, 3};
    int count{};
    v.apply([&count](int& val, tdm::dim_t /*unused*/) {
        ASSERT_EQ(val, count + 1);
        ++count;
    });
    ASSERT_EQ(count, 3);
}

TEST(VecTestMembers, TestApplyVec4) {
    tdm::vec4<int> v{1, 2, 3, 4};
    int count{};
    v.apply([&count](int& val, tdm::dim_t /*unused*/) {
        ASSERT_EQ(val, count + 1);
        ++count;
    });
    ASSERT_EQ(count, 4);
}

TEST(VecTestMembers, TestApplyVecArbirtary) {
    tdm::vec<5, int> v{1, 2, 3, 4, 5};
    int count{};
    v.apply([&count](int& val, tdm::dim_t /*unused*/) {
        ASSERT_EQ(val, count + 1);
        ++count;
    });
    ASSERT_EQ(count, 5);
}

TEST(VecTestMembers, TestNegate) {
    tdm::vec4<int> v1{-4, -3, 5, 6};
    v1.negate();
    ASSERT_EQ(v1, tdm::vec4<int>(4, 3, -5, -6));
}

TEST(VecTestMembers, TestLength) {
    tdm::vec4<float> v1{0.5f, -0.5f, -0.2f, 1.0f};
    ASSERT_NEAR(v1.length(), 1.2409, 0.0001);

    tdm::vec4<double> v2{9.0, 24.0, 37.0, 32.0};
    ASSERT_NEAR(v2.length(), 55.2268, 0.0001);

    v2 = {0.0, 0.0, 0.0, 0.0};
    ASSERT_NEAR(v2.length(), 0.0, 0.0001);
}

TEST(VecTestMembers, TestNormalize) {
    tdm::vec4<float> v1{0.5f, -0.5f, -0.2f, 1.0f};
    v1.normalize();
    ASSERT_VEC_NEAR(v1, tdm::vec4<float>(0.4029f, -0.4029f, -0.1611f, 0.8058f), 0.0001f);

    tdm::vec4<double> v2{9.0, 24.0, 37.0, 32.0};
    v2.normalize();
    ASSERT_VEC_NEAR(v2, tdm::vec4<double>(0.1629, 0.4345, 0.6699, 0.5794), 0.0001);

    v2 = {1.0, 0.0, 0.0, 0.0};
    v2.normalize();
    ASSERT_VEC_NEAR(v2, tdm::vec4<double>(1.0, 0.0, 0.0, 0.0), 0.0001);
}

TEST(VecTestMembers, TestSum) {
    tdm::vec4<int> v{1, 2, 3, 4};
    ASSERT_EQ(v.sum(), 10);
}
