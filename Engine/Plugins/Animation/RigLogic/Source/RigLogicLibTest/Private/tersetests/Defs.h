// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4061 4121 4365 4987 4668 4996 6330)
#endif
#ifdef __clang__
    #pragma clang diagnostic push
    #pragma clang diagnostic ignored "-Wundef"
#endif
#include <gtest/gtest.h>
#ifdef __clang__
    #pragma clang diagnostic pop
    // This cannot be applied to the scope of the header inclusion only, as the TEST macros will trigger it
    #pragma clang diagnostic ignored "-Wglobal-constructors"
#endif
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#ifndef INSTANTIATE_TEST_SUITE_P
    #define INSTANTIATE_TEST_SUITE_P INSTANTIATE_TEST_CASE_P
#endif

#ifndef TYPED_TEST_SUITE_P
    #define TYPED_TEST_SUITE_P TYPED_TEST_CASE_P
#endif

#ifndef REGISTER_TYPED_TEST_SUITE_P
    #define REGISTER_TYPED_TEST_SUITE_P REGISTER_TYPED_TEST_CASE_P
#endif

#ifndef INSTANTIATE_TYPED_TEST_SUITE_P
    #define INSTANTIATE_TYPED_TEST_SUITE_P INSTANTIATE_TYPED_TEST_CASE_P
#endif

#ifndef ASSERT_ELEMENTS_EQ
    #define ASSERT_ELEMENTS_EQ(result, expected, count)                                                                          \
        for (std::size_t i = 0ul; i < count; ++i) {                                                                              \
            ASSERT_EQ(result[i], expected[i]);                                                                                   \
        }
#endif

#ifndef EXPECT_ELEMENTS_EQ
    #define EXPECT_ELEMENTS_EQ(result, expected, count)                                                                          \
        for (std::size_t i = 0ul; i < count; ++i) {                                                                              \
            EXPECT_EQ(result[i], expected[i]);                                                                                   \
        }
#endif

#ifndef ASSERT_ELEMENTS_NEAR
    #define ASSERT_ELEMENTS_NEAR(result, expected, count, threshold)                                                             \
        for (std::size_t i = 0ul; i < count; ++i) {                                                                              \
            ASSERT_NEAR(result[i], expected[i], threshold);                                                                      \
        }
#endif

#ifndef EXPECT_ELEMENTS_NEAR
    #define EXPECT_ELEMENTS_NEAR(result, expected, count, threshold)                                                             \
        for (std::size_t i = 0ul; i < count; ++i) {                                                                              \
            EXPECT_NEAR(result[i], expected[i], threshold);                                                                      \
        }
#endif
