// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <algorithm>
#include <cassert>
#include <cstddef>
#include <cstdint>
#include <cstring>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

namespace tersetests {

struct FakeStream {

    FakeStream() :
        position{} {
    }

    void open() {
        position = 0ul;
    }

    void close() {
        position = 0ul;
    }

    std::uint64_t tell() {
        return position;
    }

    void seek(std::uint64_t position_) {
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic push
    #pragma GCC diagnostic ignored "-Wuseless-cast"
#endif
        position = static_cast<std::size_t>(position_);
#if !defined(__clang__) && defined(__GNUC__)
    #pragma GCC diagnostic pop
#endif
    }

    std::size_t read(char* buffer, std::size_t size) {
        const std::size_t bytesAvailable = data.size() - position;
        const std::size_t bytesToRead = std::min(bytesAvailable, size);
        if (bytesToRead == 0ul) {
            return 0ul;
        }
        std::memcpy(buffer, &data[position], bytesToRead);
        position += bytesToRead;
        return bytesToRead;
    }

    std::size_t write(const char* buffer, std::size_t size) {
        std::size_t available = data.size() - position;
        if (available < size) {
            data.resize(data.size() + (size - available));
        }
        std::memcpy(&data[position], buffer, size);
        position += size;
        return size;
    }

    std::uint64_t size() {
        return data.size();
    }

    std::vector<char> data;
    std::size_t position;
};

}  // namespace tersetests
