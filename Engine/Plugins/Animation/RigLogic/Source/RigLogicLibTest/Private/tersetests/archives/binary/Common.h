// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/types/ArchiveOffset.h"
#include "terse/types/Versioned.h"

#include <pma/TypeDefs.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <string>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

struct ComplexType {
    std::int8_t a;
    std::uint8_t b;
    std::int16_t c;
    std::uint16_t d;
    std::int32_t e;
    std::uint32_t f;

    template<class Archive>
    void serialize(Archive& archive) {
        archive(a, b, c, d, e, f);
    }
};

struct Child {
    pma::Vector<std::uint16_t> integers;
    pma::Vector<float> floats;

    explicit Child(pma::MemoryResource* memRes) :
        integers{memRes},
        floats{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(integers, floats);
    }
};

struct Root {
    std::string name;
    pma::Vector<Child> children;

    explicit Root(pma::MemoryResource* memRes) :
        children{memRes} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(name, children);
    }
};

struct OffsetUtilizer {
    terse::ArchiveOffset<std::uint32_t> layerStart;
    terse::ArchiveOffset<std::uint32_t> first;
    terse::ArchiveOffset<std::uint32_t> second;

    std::int32_t a;
    char b;
    std::string c;

    terse::ArchiveOffset<std::uint32_t>::Proxy layerMarker;
    terse::ArchiveOffset<std::uint32_t>::Proxy firstMarker;
    std::int32_t firstInt;
    terse::ArchiveOffset<std::uint32_t>::Proxy secondMarker;
    std::int32_t secondInt;

    OffsetUtilizer() :
        layerMarker{layerStart},
        firstMarker{first},
        secondMarker{second} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(layerStart, first, second, a, b, c, layerMarker, firstMarker, firstInt, secondMarker, secondInt);
    }
};

struct SerializableByFreeSerialize {
    std::uint32_t a;
    std::uint16_t b;
};

template<class TArchive>
void serialize(TArchive& archive, SerializableByFreeSerialize& target) {
    archive(target.a, target.b);
}

struct SerializableByFreeLoadSave {
    std::uint32_t a;
    std::uint16_t b;
};

template<class TArchive>
void load(TArchive& archive, SerializableByFreeLoadSave& dest) {
    archive(dest.a, dest.b);
}

template<class TArchive>
void save(TArchive& archive, const SerializableByFreeLoadSave& source) {
    archive(source.a, source.b);
}

struct UpgradedSerializableType {
    std::uint32_t a;
    std::uint16_t b;

    template<class Archive>
    void load(Archive& archive, terse::Version<1> /*unused*/) {
        archive(a);
    }

    template<class Archive>
    void load(Archive& archive, terse::Version<2> /*unused*/) {
        archive(a, b);
    }

    template<class Archive>
    void save(Archive& archive, terse::Version<1> /*unused*/) {
        archive(a);
    }

    template<class Archive>
    void save(Archive& archive, terse::Version<2> /*unused*/) {
        archive(a, b);
    }
};
