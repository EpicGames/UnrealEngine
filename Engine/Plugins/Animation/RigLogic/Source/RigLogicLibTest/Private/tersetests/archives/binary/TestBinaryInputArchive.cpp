// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"
#include "tersetests/FakeStream.h"
#include "tersetests/archives/binary/Common.h"

#include "terse/archives/binary/InputArchive.h"
#include "terse/types/Anchor.h"
#include "terse/types/ArchiveOffset.h"
#include "terse/types/ArchiveSize.h"
#include "terse/types/DynArray.h"
#include "terse/types/Versioned.h"

#include <pma/TypeDefs.h>
#include <pma/resources/AlignedMemoryResource.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <limits>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

TEST(BinaryInputArchiveTest, StreamExtractionOperator) {
    tersetests::FakeStream stream;
    unsigned char bytes[2ul * sizeof(char)] = {0x01, 0x02};
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    char first = {};
    char second = {};
    archive >> first >> second;

    ASSERT_EQ(first, 0x01);
    ASSERT_EQ(second, 0x02);
}

TEST(BinaryInputArchiveTest, ArchiveOffset) {
    tersetests::FakeStream stream;
    unsigned char bytes[33ul] = {
        0x00, 0x00, 0x00, 0x19,  // layerStart offset
        0x00, 0x00, 0x00, 0x19,  // first offset
        0x00, 0x00, 0x00, 0x1d,  // second offset
        0x00, 0x00, 0x04, 0xd2,  // a = 1234
        0x78,                    // b = 'x'
        0x00, 0x00, 0x00, 0x04,  // c size = 4
        0x61, 0x62, 0x63, 0x64,  // c = 'abcd'
        0x00, 0x00, 0x01, 0x00,  // firstInt = 256
        0x00, 0x00, 0x04, 0x00   // secondInt = 1024

    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    OffsetUtilizer dest;
    archive >> dest;

    ASSERT_EQ(dest.layerStart.proxy, &dest.layerMarker);
    ASSERT_EQ(dest.layerStart.value, 25ul);

    ASSERT_EQ(dest.first.proxy, &dest.firstMarker);
    ASSERT_EQ(dest.first.value, 25ul);

    ASSERT_EQ(dest.second.proxy, &dest.secondMarker);
    ASSERT_EQ(dest.second.value, 29ul);
}

struct IgnoredSection {
    std::uint32_t a;
    std::string b;
    std::uint16_t c;

    IgnoredSection() :
        a{},
        b{},
        c{} {
    }

    template<class Archive>
    void serialize(Archive& /*unused*/) {
        // This layer controls it's own serialization and decides
        // during runtime for some reason not to deserialize it's data,
        // although it is present in the stream
    }
};

struct SectionSkipper {
    terse::ArchiveOffset<std::uint32_t> afterValueOffset;
    IgnoredSection ignored;
    terse::ArchiveOffset<std::uint32_t>::Proxy afterValueMarker;
    std::uint32_t afterValue;

    SectionSkipper() :
        afterValueOffset{},
        ignored{},
        afterValueMarker{afterValueOffset},
        afterValue{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(afterValueOffset, ignored, afterValueMarker, afterValue);
    }
};

TEST(BinaryInputArchiveTest, ArchiveOffsetRepositionsStream) {
    tersetests::FakeStream stream;
    unsigned char bytes[22ul] = {
        0x00, 0x00, 0x00, 0x12,  // afterValue offset
        0x00, 0x00, 0x04, 0xd2,  // a = 1234
        0x00, 0x00, 0x00, 0x04,  // b size = 4
        0x61, 0x62, 0x63, 0x64,  // b = 'abcd'
        0x01, 0x00,              // c = 256
        0x00, 0x00, 0x04, 0x00   // afterValue = 1024

    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    SectionSkipper dest;
    archive >> dest;

    // Since `IgnoredSection` decided not to load it's data,
    // we rely on `afterValueMarker` to reposition the stream
    // so that it correctly points to the start of `afterValue`,
    // after the aborted reading of `ignored`. If it works,
    // `afterValue` should load it's expected value
    ASSERT_EQ(dest.afterValue, 1024u);
    // Sanity check to make sure ignored layer was really ignored
    ASSERT_EQ(dest.ignored.a, 0u);
    ASSERT_EQ(dest.ignored.b, "");
    ASSERT_EQ(dest.ignored.c, 0u);
}

struct Wrapped {
    std::uint32_t someValue;
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size;
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size2;
    terse::Anchor<std::uint32_t> base;
    IgnoredSection ignored;
    terse::ArchiveSize<std::uint32_t, std::uint32_t>::Proxy sizeMarker;

    Wrapped() :
        someValue{},
        size{},
        size2{},
        base{},
        ignored{},
        sizeMarker{size, base} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(someValue, size, size2, base, ignored, sizeMarker, terse::proxy(size2, base));
    }
};

TEST(BinaryInputArchiveTest, ArchiveSizeIsRelativeToOffset) {
    tersetests::FakeStream stream;
    unsigned char bytes[26ul] = {
        0x00, 0x00, 0x00, 0x11,  // someValue
        0x00, 0x00, 0x00, 0x0e,  // size
        0x00, 0x00, 0x00, 0x0e,  // size2
        0x00, 0x00, 0x04, 0xd2,  // a = 1234
        0x00, 0x00, 0x00, 0x04,  // b size = 4
        0x61, 0x62, 0x63, 0x64,  // b = 'abcd'
        0x01, 0x00               // c = 256
    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    Wrapped dest;
    archive >> dest;

    // Sanity check to make sure ignored layer was really ignored
    ASSERT_EQ(dest.ignored.a, 0u);
    ASSERT_EQ(dest.ignored.b, "");
    ASSERT_EQ(dest.ignored.c, 0u);
    // Verify that stream was positioned correctly to jump to the end when size marker was encountered
    ASSERT_EQ(stream.tell(), sizeof(bytes));
}

TEST(BinaryInputArchiveTest, LittleEndianDataDeserialization) {
    tersetests::FakeStream stream;
    unsigned char bytes[sizeof(std::int32_t)] = {0xd2, 0x04, 0x00, 0x00};  // 1234
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream, std::uint32_t, std::uint32_t, terse::Endianness::Little> archive(&stream);
    std::int32_t dest;
    archive(dest);

    ASSERT_EQ(dest, 1234);
}

TEST(BinaryInputArchiveTest, PrimitiveTypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char bytes[sizeof(std::int32_t)] = {0x00, 0x00, 0x04, 0xd2};  // 1234
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::int32_t dest;
    archive(dest);

    ASSERT_EQ(dest, 1234);
}

TEST(BinaryInputArchiveTest, ComplexTypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char bytes[14ul] = {
        0x7f,  // 127
        0xff,  // 255
        0x7f,
        0xff,  // 32767
        0xff,
        0xff,  // 65535
        0x7f,
        0xff,
        0xff,
        0xff,  // 2147483647
        0xff,
        0xff,
        0xff,
        0xff  // 4294967295
    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    ComplexType dest;
    archive(dest);

    ASSERT_EQ(dest.a, 127);
    ASSERT_EQ(dest.b, 255u);
    ASSERT_EQ(dest.c, 32767);
    ASSERT_EQ(dest.d, 65535u);
    ASSERT_EQ(dest.e, 2147483647);
    ASSERT_EQ(dest.f, 4294967295u);
}

TEST(BinaryInputArchiveTest, FreeSerialize) {
    tersetests::FakeStream stream;
    unsigned char bytes[6ul] = {
        0xff,
        0xff,
        0xff,
        0xff,  // 4294967295
        0xff,
        0xff  // 65535
    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    SerializableByFreeSerialize dest;
    archive(dest);

    ASSERT_EQ(dest.a, 4294967295u);
    ASSERT_EQ(dest.b, 65535u);
}

TEST(BinaryInputArchiveTest, FreeLoad) {
    tersetests::FakeStream stream;
    unsigned char bytes[6ul] = {
        0xff,
        0xff,
        0xff,
        0xff,  // 4294967295
        0xff,
        0xff  // 65535
    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    SerializableByFreeLoadSave dest;
    archive(dest);

    ASSERT_EQ(dest.a, 4294967295u);
    ASSERT_EQ(dest.b, 65535u);
}

TEST(BinaryInputArchiveTest, StringDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x08};  // 8
    unsigned char bytes[8ul] = {
        0x61,  // 'a'
        0x62,  // 'b'
        0x63,  // 'c'
        0x64,  // 'd'
        0x65,  // 'e'
        0x66,  // 'f'
        0x67,  // 'g'
        0x68   // 'h'
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::string dest;
    archive(dest);

    ASSERT_EQ(dest, "abcdefgh");
}

TEST(BinaryInputArchiveTest, BlobDeserialization) {
    tersetests::FakeStream stream;
    unsigned char bytes[8ul] = {
        0x61,  // 'a'
        0x62,  // 'b'
        0x63,  // 'c'
        0x64,  // 'd'
        0x65,  // 'e'
        0x66,  // 'f'
        0x67,  // 'g'
        0x68   // 'h'
    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    terse::Blob<char, pma::PolyAllocator<char>> dest;
    dest.setSize(4ul);  // This dictates the number of bytes that will be read
    archive(dest);

    ASSERT_EQ(dest.size(), 4ul);
    ASSERT_EQ(stream.tell(), 4ul);
    ASSERT_ELEMENTS_EQ(dest.data(), bytes, dest.size());
}

TEST(BinaryInputArchiveTest, VectorOfInt8TypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x05};  // 5
    unsigned char bytes[5ul] = {
        0x80,  // -128
        0x9c,  // -100
        0xff,  // -1
        0x0f,  // 15
        0x7f   // 127
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<std::int8_t> dest;
    archive(dest);

    std::vector<std::int8_t> expected{-128, -100, -1, 15, 127};
    ASSERT_EQ(dest, expected);
}

TEST(BinaryInputArchiveTest, VectorOfInt16TypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x05};  // 5
    unsigned char bytes[5ul * sizeof(std::uint16_t)] = {
        0x80,
        0x00,  // -32768
        0xff,
        0xf9,  // -7
        0x00,
        0x0f,  // 15
        0x00,
        0xff,  // 255
        0x7f,
        0xff  // 32767
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<std::int16_t> dest;
    archive(dest);

    std::vector<std::int16_t> expected{std::numeric_limits<std::int16_t>::min(),
                                       -7,
                                       15,
                                       255,
                                       std::numeric_limits<std::int16_t>::max()};
    ASSERT_EQ(dest, expected);
}

TEST(BinaryInputArchiveTest, VectorOfInt32TypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x08};  // 8
    unsigned char bytes[8ul * sizeof(std::int32_t)] = {
        0x80, 0x00, 0x00, 0x00,  // -2147483648
        0xff, 0xff, 0xff, 0xf0,  // -16
        0x00, 0x00, 0x00, 0x80,  // 128
        0x00, 0x00, 0x01, 0x00,  // 256
        0x00, 0x00, 0x10, 0x00,  // 4096
        0x00, 0x01, 0x00, 0x00,  // 65536
        0x00, 0x02, 0x00, 0x00,  // 131072
        0x7f, 0xff, 0xff, 0xff   // 2147483647
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<std::int32_t> dest;
    archive(dest);

    std::vector<std::int32_t> expected{std::numeric_limits<std::int32_t>::min(),
                                       -16,
                                       128,
                                       256,
                                       4096,
                                       65536,
                                       131072,
                                       std::numeric_limits<std::int32_t>::max()};
    ASSERT_EQ(dest, expected);
}

TEST(BinaryInputArchiveTest, VectorOfInt64TypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::int32_t)] = {0x00, 0x00, 0x00, 0x05};  // 5
    unsigned char bytes[5ul * sizeof(std::int64_t)] = {
        0x80, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00,  // -9223372036854775808
        0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0x00,  // -256
        0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff,  // -1
        0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0xff,  // 255
        0x7f, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff   // 9223372036854775807
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<std::int64_t> dest;
    archive(dest);

    std::vector<std::int64_t> expected{std::numeric_limits<std::int64_t>::min(),
                                       -256,
                                       -1,
                                       255,
                                       std::numeric_limits<std::int64_t>::max()};
    ASSERT_EQ(dest, expected);
}

TEST(BinaryInputArchiveTest, VectorOfFloatTypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x05};  // 5
    unsigned char bytes[5ul * sizeof(float)] = {
        0x3f, 0x99, 0x99, 0x9a,  // 1.2f
        0x40, 0x48, 0xf5, 0xc3,  // 3.14f
        0x3d, 0x4c, 0xcc, 0xcd,  // 0.05f
        0x3f, 0x3d, 0x70, 0xa4,  // 0.74f
        0x43, 0x2,  0xe6, 0x66,  // 130.9f
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<float> dest;
    archive(dest);

    std::vector<float> expected{1.2f, 3.14f, 0.05f, 0.74f, 130.9f};
    ASSERT_ELEMENTS_NEAR(dest, expected, expected.size(), 0.001f);
}

TEST(BinaryInputArchiveTest, VectorOfDoubleTypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x05};  // 5
    unsigned char bytes[5ul * sizeof(double)] = {
        0xc0, 0x24, 0x00, 0x00, 0x00, 0x00, 0x00, 0x00, 0x40, 0x09, 0x1e, 0xb8, 0x51, 0xeb, 0x85, 0x1f, 0x3f, 0xa9, 0x99, 0x99,
        0x99, 0x99, 0x99, 0x9a, 0x3f, 0xe7, 0xae, 0x14, 0x7a, 0xe1, 0x47, 0xae, 0x40, 0x60, 0x5c, 0xcc, 0xcc, 0xcc, 0xcc, 0xcd};
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<double> dest;
    archive(dest);

    std::vector<double> expected{-10.0, 3.14, 0.05, 0.74, 130.9};
    ASSERT_ELEMENTS_NEAR(dest, expected, expected.size(), 0.001);
}

TEST(BinaryInputArchiveTest, VectorOfComplexTypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x02};  // 2
    unsigned char bytes[28ul] = {
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff,  // 4294967295
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff   // 4294967295
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<ComplexType> dest;
    archive(dest);

    ASSERT_EQ(dest.size(), 2ul);
    for (const auto& ct : dest) {
        ASSERT_EQ(ct.a, 127);
        ASSERT_EQ(ct.b, 255u);
        ASSERT_EQ(ct.c, 32767);
        ASSERT_EQ(ct.d, 65535u);
        ASSERT_EQ(ct.e, 2147483647);
        ASSERT_EQ(ct.f, 4294967295u);
    }
}

TEST(BinaryInputArchiveTest, DynArrayOfPrimitiveStructsDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x02};  // 2
    unsigned char bytes[28ul] = {
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff,  // 4294967295
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff   // 4294967295
    };
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    terse::DynArray<ComplexType, pma::PolyAllocator<ComplexType>> dest;
    archive(dest);

    ASSERT_EQ(dest.size(), 2ul);
    for (const auto& ct : dest) {
        ASSERT_EQ(ct.a, 127);
        ASSERT_EQ(ct.b, 255u);
        ASSERT_EQ(ct.c, 32767);
        ASSERT_EQ(ct.d, 65535u);
        ASSERT_EQ(ct.e, 2147483647);
        ASSERT_EQ(ct.f, 4294967295u);
    }
}

struct INeedAMemoryResource {
    INeedAMemoryResource(pma::MemoryResource* memRes) :
        pMemRes{memRes},
        number{} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(number);
    }

    pma::MemoryResource* pMemRes;

    int number;
};

TEST(BinaryInputArchiveTest, VectorOfTypeConstructibleFromMemoryResourceDeserialization) {
    tersetests::FakeStream stream;
    unsigned char size[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x01};  // 1
    unsigned char bytes[sizeof(std::int32_t)] = {0x00, 0x00, 0x04, 0xd2};  // 1234
    stream.write(reinterpret_cast<char*>(size), sizeof(size));
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    pma::AlignedMemoryResource memRes;
    std::vector<INeedAMemoryResource, pma::PolyAllocator<INeedAMemoryResource>> dest{
        pma::PolyAllocator<INeedAMemoryResource>{&memRes}};
    archive(dest);

    ASSERT_EQ(dest.size(), 1ul);
    for (const auto& inamr : dest) {
        ASSERT_EQ(inamr.number, 1234);
        ASSERT_EQ(inamr.pMemRes, &memRes);
    }
}

TEST(BinaryInputArchiveTest, VectorOfStringDeserialization) {
    tersetests::FakeStream stream;
    unsigned char vectorSize[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x02};  // 2
    unsigned char stringSize[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x08};  // 8
    unsigned char stringBytes[8ul] = {
        0x61,  // 'a'
        0x62,  // 'b'
        0x63,  // 'c'
        0x64,  // 'd'
        0x65,  // 'e'
        0x66,  // 'f'
        0x67,  // 'g'
        0x68   // 'h'
    };
    stream.write(reinterpret_cast<char*>(vectorSize), sizeof(vectorSize));
    // Write first string
    stream.write(reinterpret_cast<char*>(stringSize), sizeof(stringSize));
    stream.write(reinterpret_cast<char*>(stringBytes), sizeof(stringBytes));
    // Write second string
    stream.write(reinterpret_cast<char*>(stringSize), sizeof(stringSize));
    stream.write(reinterpret_cast<char*>(stringBytes), sizeof(stringBytes));

    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<std::string> dest;
    archive(dest);

    ASSERT_EQ(dest.size(), 2ul);
    for (const auto& s : dest) {
        ASSERT_EQ(s, "abcdefgh");
    }
}

TEST(BinaryInputArchiveTest, VectorOfPairOfStringsDeserialization) {
    tersetests::FakeStream stream;
    unsigned char vectorSize[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x02};  // 2
    unsigned char stringSize[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x04};  // 4
    unsigned char keyBytes[4ul] = {0x61, 0x62, 0x63, 0x64};                      // "abcd"
    unsigned char valueBytes[4ul] = {0x65, 0x66, 0x67, 0x68};                    // "efgh"
    stream.write(reinterpret_cast<char*>(vectorSize), sizeof(vectorSize));
    // Write key
    stream.write(reinterpret_cast<char*>(stringSize), sizeof(stringSize));
    stream.write(reinterpret_cast<char*>(keyBytes), sizeof(keyBytes));
    // Write value
    stream.write(reinterpret_cast<char*>(stringSize), sizeof(stringSize));
    stream.write(reinterpret_cast<char*>(valueBytes), sizeof(valueBytes));
    // Write key
    stream.write(reinterpret_cast<char*>(stringSize), sizeof(stringSize));
    stream.write(reinterpret_cast<char*>(keyBytes), sizeof(keyBytes));
    // Write value
    stream.write(reinterpret_cast<char*>(stringSize), sizeof(stringSize));
    stream.write(reinterpret_cast<char*>(valueBytes), sizeof(valueBytes));

    stream.seek(0ul);

    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    std::vector<std::pair<std::string, std::string>> dest;
    archive(dest);

    ASSERT_EQ(dest.size(), 2ul);
    for (const auto& s : dest) {
        ASSERT_EQ(s.first, "abcd");
        ASSERT_EQ(s.second, "efgh");
    }
}

TEST(BinaryInputArchiveTest, VectorOfVectorOfPrimitiveTypeDeserialization) {
    tersetests::FakeStream stream;
    unsigned char outerVectorSize[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x02};  // 2
    unsigned char innerVectorSize[sizeof(std::uint32_t)] = {0x00, 0x00, 0x00, 0x04};  // 4
    unsigned char innerVectorBytes[4ul * sizeof(std::uint16_t)] = {
        0x00,
        0x04,  // 4
        0x00,
        0x10,  // 16
        0x01,
        0x00,  // 256
        0x10,
        0x00  // 4096
    };
    stream.write(reinterpret_cast<char*>(outerVectorSize), sizeof(outerVectorSize));
    // Write first inner vector
    stream.write(reinterpret_cast<char*>(innerVectorSize), sizeof(innerVectorSize));
    stream.write(reinterpret_cast<char*>(innerVectorBytes), sizeof(innerVectorBytes));
    // Write second inner vector
    stream.write(reinterpret_cast<char*>(innerVectorSize), sizeof(innerVectorSize));
    stream.write(reinterpret_cast<char*>(innerVectorBytes), sizeof(innerVectorBytes));
    stream.seek(0ul);

    pma::AlignedMemoryResource memRes;
    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    pma::Vector<pma::Vector<std::uint16_t>> outer{&memRes};
    archive(outer);

    // Make sure values and structure is what's expected
    ASSERT_EQ(outer.size(), 2ul);
    for (const auto& inner : outer) {
        ASSERT_EQ(inner.size(), 4ul);
        ASSERT_EQ(inner[0], 4);
        ASSERT_EQ(inner[1], 16);
        ASSERT_EQ(inner[2], 256);
        ASSERT_EQ(inner[3], 4096);
    }

    // Make sure the parent allocator is propagated to nested elements
    ASSERT_EQ(&memRes, outer.get_allocator().getMemoryResource());
    for (const auto& inner : outer) {
        ASSERT_EQ(&memRes, inner.get_allocator().getMemoryResource());
    }
}

TEST(BinaryInputArchiveTest, MixedNestedTypes) {
    tersetests::FakeStream stream;
    unsigned char bytes[68ul] = {
        0x00, 0x00, 0x00, 0x01,                                                  // 1
        0x00, 0x00, 0x00, 0x04,                                                  // 4
        0x72, 0x6f, 0x6f, 0x74,                                                  // "root"
        0x00, 0x00, 0x00, 0x02,                                                  // 2
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x00, 0x01, 0x00, 0x02, 0x00, 0x03,                                      // 1, 2, 3
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x3c, 0x23, 0xd7, 0x0a, 0x3f, 0x7d, 0x70, 0xa4, 0x40, 0x48, 0xf5, 0xc3,  // 0.01f, 0.99f, 3.14f
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x00, 0x01, 0x00, 0x02, 0x00, 0x03,                                      // 1, 2, 3
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x3c, 0x23, 0xd7, 0x0a, 0x3f, 0x7d, 0x70, 0xa4, 0x40, 0x48, 0xf5, 0xc3   // 0.01f, 0.99f, 3.14f
    };
    stream.write(reinterpret_cast<char*>(bytes), sizeof(bytes));
    stream.seek(0ul);

    pma::AlignedMemoryResource memRes;
    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    pma::Vector<Root> dest{&memRes};

    archive(dest);

    ASSERT_EQ(dest.size(), 1ul);
    for (const auto& root : dest) {
        ASSERT_EQ(root.name, "root");
        ASSERT_EQ(root.children.size(), 2ul);
        ASSERT_EQ(root.children.get_allocator().getMemoryResource(), &memRes);
        for (const auto& child : root.children) {
            ASSERT_EQ(child.integers.size(), 3ul);
            ASSERT_EQ(child.integers.get_allocator().getMemoryResource(), &memRes);
            ASSERT_EQ(child.integers[0], 1);
            ASSERT_EQ(child.integers[1], 2);
            ASSERT_EQ(child.integers[2], 3);
            ASSERT_EQ(child.floats.size(), 3ul);
            ASSERT_EQ(child.floats.get_allocator().getMemoryResource(), &memRes);
            ASSERT_EQ(child.floats[0], 0.01f);
            ASSERT_EQ(child.floats[1], 0.99f);
            ASSERT_EQ(child.floats[2], 3.14f);
        }
    }
}

TEST(BinaryInputArchiveTest, AttachUserData) {
    int userData = {};
    tersetests::FakeStream stream;
    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);
    ASSERT_EQ(archive.getUserData(), nullptr);
    archive.setUserData(&userData);
    ASSERT_EQ(archive.getUserData(), &userData);
}

TEST(BinaryInputArchiveTest, LoadVersionedType) {
    unsigned char bytesv1[] = {0x00, 0x00, 0x04, 0xd2};              // 1234
    unsigned char bytesv2[] = {0x00, 0x00, 0x04, 0xd2, 0x00, 0xff};  // 1234, 255

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<char*>(bytesv1), sizeof(bytesv1));
    stream.write(reinterpret_cast<char*>(bytesv2), sizeof(bytesv2));
    stream.seek(0ul);
    terse::BinaryInputArchive<tersetests::FakeStream> archive(&stream);

    UpgradedSerializableType destv1{};
    archive(terse::versioned(destv1, terse::Version<1>{}));
    ASSERT_EQ(destv1.a, 1234u);
    ASSERT_EQ(destv1.b, 0u);  // b is not loaded under version-1 serialization rules

    UpgradedSerializableType destv2{};
    archive(terse::versioned(destv2, terse::Version<2>{}));
    ASSERT_EQ(destv2.a, 1234u);
    ASSERT_EQ(destv2.b, 255u);
}
