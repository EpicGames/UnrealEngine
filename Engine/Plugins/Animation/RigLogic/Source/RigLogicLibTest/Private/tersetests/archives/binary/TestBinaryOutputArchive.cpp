// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"
#include "tersetests/FakeStream.h"
#include "tersetests/archives/binary/Common.h"

#include "terse/archives/binary/OutputArchive.h"
#include "terse/types/Anchor.h"
#include "terse/types/ArchiveOffset.h"
#include "terse/types/ArchiveSize.h"
#include "terse/types/DynArray.h"
#include "terse/types/Versioned.h"

#include <pma/TypeDefs.h>
#include <pma/resources/AlignedMemoryResource.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

TEST(BinaryOutputArchiveTest, StreamInsertionOperator) {
    std::int32_t source1 = 1234;
    std::int32_t source2 = 5678;

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive << source1 << source2;

    unsigned char expected[sizeof(std::int32_t) * 2ul] = {
        0x00,
        0x00,
        0x04,
        0xd2,  // 1234
        0x00,
        0x00,
        0x16,
        0x2e  // 5678
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, ArchiveOffset) {
    OffsetUtilizer source{};
    source.a = 1234;
    source.b = 'x';
    source.c = "abcd";
    source.firstInt = 256;
    source.secondInt = 1024;

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive << source;

    unsigned char expected[33ul] = {
        0x00, 0x00, 0x00, 0x19,  // layerStart offset
        0x00, 0x00, 0x00, 0x19,  // first offset
        0x00, 0x00, 0x00, 0x1d,  // second offset
        0x00, 0x00, 0x04, 0xd2,  // a = 1234
        0x78,                    // b = 'x'
        0x00, 0x00, 0x00, 0x04,  // c size = 4
        0x61, 0x62, 0x63, 0x64,  // c = 'abcd'
        0x00, 0x00, 0x01, 0x00,  // firstInt = 256
        0x00, 0x00, 0x04, 0x00   // secondInt = 1024

    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));
    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

struct OffsetSizeTest {
    std::uint32_t somePadding;
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size;
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size2;
    terse::Anchor<std::uint32_t> base;

    std::int32_t a;
    char b;
    std::string c;

    terse::ArchiveSize<std::uint32_t, std::uint32_t>::Proxy sizeMarker;

    OffsetSizeTest() :
        somePadding{},
        size{},
        size2{},
        base{},
        a{},
        b{},
        c{},
        sizeMarker{size, base} {
    }

    template<class Archive>
    void serialize(Archive& archive) {
        archive(somePadding, size, size2, base, a, b, c, sizeMarker, terse::proxy(size2, base));
    }
};

TEST(BinaryOutputArchiveTest, ArchiveSizeIsRelativeToOffset) {
    OffsetSizeTest source{};
    source.somePadding = 4u;
    source.a = 1234;
    source.b = 'x';
    source.c = "abcd";

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive << source;

    unsigned char expected[25ul] = {
        0x00, 0x00, 0x00, 0x04,  // somePadding
        0x00, 0x00, 0x00, 0x0d,  // size
        0x00, 0x00, 0x00, 0x0d,  // size2
        0x00, 0x00, 0x04, 0xd2,  // a = 1234
        0x78,                    // b = 'x'
        0x00, 0x00, 0x00, 0x04,  // c size = 4
        0x61, 0x62, 0x63, 0x64   // c = 'abcd'
    };

    ASSERT_EQ(stream.size(), sizeof(expected));
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));
    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
    ASSERT_EQ(source.size.value, 13ul);
    ASSERT_EQ(source.size2.value, 13ul);
}

TEST(BinaryOutputArchiveTest, LittleEndianDataSerialization) {
    std::int32_t source = 1234;

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream, std::uint32_t, std::uint32_t, terse::Endianness::Little> archive(&stream);
    archive(source);

    unsigned char expected[sizeof(std::int32_t)] = {0xd2, 0x04, 0x00, 0x00};  // 1234
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, PrimitiveTypeSerialization) {
    std::int32_t source = 1234;

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[sizeof(std::int32_t)] = {0x00, 0x00, 0x04, 0xd2};  // 1234
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, ComplexTypeSerialization) {
    ComplexType source{127, 255, 32767, 65535, 2147483647, 4294967295};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[14ul] = {
        0x7f,  // 127
        0xff,  // 255
        0x7f,
        0xff,  // 32767
        0xff,
        0xff,  // 65535
        0x7f,
        0xff,
        0xff,
        0xff,  // 2147483647
        0xff,
        0xff,
        0xff,
        0xff,  // 4294967295
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, FreeSerialize) {
    SerializableByFreeSerialize source{4294967295u, 65535u};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[6ul] = {
        0xff,
        0xff,
        0xff,
        0xff,  // 4294967295
        0xff,
        0xff  // 65535
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, FreeSave) {
    SerializableByFreeLoadSave source{4294967295u, 65535u};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[6ul] = {
        0xff,
        0xff,
        0xff,
        0xff,  // 4294967295
        0xff,
        0xff  // 65535
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, StringSerialization) {
    std::string source{"abcdefgh"};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[12ul] = {
        0x00,
        0x00,
        0x00,
        0x08,  // 8
        0x61,  // 'a'
        0x62,  // 'b'
        0x63,  // 'c'
        0x64,  // 'd'
        0x65,  // 'e'
        0x66,  // 'f'
        0x67,  // 'g'
        0x68   // 'h'
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, BlobSerialization) {
    terse::Blob<char, pma::PolyAllocator<char>> dest;
    dest.setSize(4ul);
    std::memcpy(dest.data(), "abcd", dest.size());

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(dest);

    unsigned char expected[4ul] = {
        0x61,  // 'a'
        0x62,  // 'b'
        0x63,  // 'c'
        0x64   // 'd'
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, VectorOfIntegerTypeSerialization) {
    std::vector<std::int32_t> source{16, 256, 4096, 65536};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[5ul * sizeof(std::int32_t)] = {
        0x00, 0x00, 0x00, 0x04,  // 4
        0x00, 0x00, 0x00, 0x10,  // 16
        0x00, 0x00, 0x01, 0x00,  // 256
        0x00, 0x00, 0x10, 0x00,  // 4096
        0x00, 0x01, 0x00, 0x00   // 65536
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, VectorOfFloatTypeSerialization) {
    std::vector<float> source{1.2f, 3.14f, 0.05f, 0.74f};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[5ul * sizeof(float)] = {
        0x00, 0x00, 0x00, 0x04,  // 4
        0x3f, 0x99, 0x99, 0x9a,  // 1.2f
        0x40, 0x48, 0xf5, 0xc3,  // 3.14f
        0x3d, 0x4c, 0xcc, 0xcd,  // 0.05f
        0x3f, 0x3d, 0x70, 0xa4   // 0.74f
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, VectorOfComplexTypeSerialization) {
    std::vector<ComplexType> source;
    source.push_back({127, 255, 32767, 65535, 2147483647, 4294967295});
    source.push_back({127, 255, 32767, 65535, 2147483647, 4294967295});

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[32ul] = {
        0x00, 0x00, 0x00, 0x02,  // 2
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff,  // 4294967295
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff   // 4294967295
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, DynArrayOfPrimitiveStructSerialization) {
    terse::DynArray<ComplexType, pma::PolyAllocator<ComplexType>> source;
    source.resize_uninitialized(2ul);
    source[0] = {127, 255, 32767, 65535, 2147483647, 4294967295};
    source[1] = {127, 255, 32767, 65535, 2147483647, 4294967295};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[32ul] = {
        0x00, 0x00, 0x00, 0x02,  // 2
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff,  // 4294967295
        0x7f,                    // 127
        0xff,                    // 255
        0x7f, 0xff,              // 32767
        0xff, 0xff,              // 65535
        0x7f, 0xff, 0xff, 0xff,  // 2147483647
        0xff, 0xff, 0xff, 0xff   // 4294967295
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, VectorOfStringSerialization) {
    std::vector<std::string> source{"abcdefgh", "abcdefgh"};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[28ul] = {
        0x00, 0x00, 0x00, 0x02,                          // 2
        0x00, 0x00, 0x00, 0x08,                          // 8
        0x61, 0x62, 0x63, 0x64, 0x65, 0x66, 0x67, 0x68,  // "abcdefgh"
        0x00, 0x00, 0x00, 0x08,                          // 8
        0x61, 0x62, 0x63, 0x64, 0x65, 0x66, 0x67, 0x68   // "abcdefgh"
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, VectorOfPairOfStringsSerialization) {
    using KeyValue = std::pair<std::string, std::string>;
    std::vector<KeyValue> source{{"abcd", "efgh"}, {"abcd", "efgh"}};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[36ul] = {
        0x00, 0x00, 0x00, 0x02,  // 2
        0x00, 0x00, 0x00, 0x04,  // 4
        0x61, 0x62, 0x63, 0x64,  // "abcd"
        0x00, 0x00, 0x00, 0x04,  // 4
        0x65, 0x66, 0x67, 0x68,  // "efgh"
        0x00, 0x00, 0x00, 0x04,  // 4
        0x61, 0x62, 0x63, 0x64,  // "abcd"
        0x00, 0x00, 0x00, 0x04,  // 4
        0x65, 0x66, 0x67, 0x68,  // "efgh"
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, VectorOfVectorOfPrimitiveTypeSerialization) {
    pma::Vector<std::uint16_t> values{4, 16, 256, 4096};
    std::vector<pma::Vector<std::uint16_t>> source{values, values};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[28ul] = {
        0x00, 0x00, 0x00, 0x02,                          // 2
        0x00, 0x00, 0x00, 0x04,                          // 4
        0x00, 0x04, 0x00, 0x10, 0x01, 0x00, 0x10, 0x00,  // 4, 16, 256, 4096
        0x00, 0x00, 0x00, 0x04,                          // 4
        0x00, 0x04, 0x00, 0x10, 0x01, 0x00, 0x10, 0x00   // 4, 16, 256, 4096
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, MixedNestedTypes) {
    pma::AlignedMemoryResource memRes;

    Root root{&memRes};
    root.name = "root";

    Child child1{&memRes};
    child1.integers.push_back(1);
    child1.integers.push_back(2);
    child1.integers.push_back(3);
    child1.floats.push_back(0.01f);
    child1.floats.push_back(0.99f);
    child1.floats.push_back(3.14f);
    Child child2{&memRes};
    child2.integers.push_back(1);
    child2.integers.push_back(2);
    child2.integers.push_back(3);
    child2.floats.push_back(0.01f);
    child2.floats.push_back(0.99f);
    child2.floats.push_back(3.14f);

    root.children.push_back(std::move(child1));
    root.children.push_back(std::move(child2));

    pma::Vector<Root> source{&memRes};
    source.push_back(std::move(root));

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(source);

    unsigned char expected[68ul] = {
        0x00, 0x00, 0x00, 0x01,                                                  // 1
        0x00, 0x00, 0x00, 0x04,                                                  // 4
        0x72, 0x6f, 0x6f, 0x74,                                                  // "root"
        0x00, 0x00, 0x00, 0x02,                                                  // 2
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x00, 0x01, 0x00, 0x02, 0x00, 0x03,                                      // 1, 2, 3
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x3c, 0x23, 0xd7, 0x0a, 0x3f, 0x7d, 0x70, 0xa4, 0x40, 0x48, 0xf5, 0xc3,  // 0.01f, 0.99f, 3.14f
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x00, 0x01, 0x00, 0x02, 0x00, 0x03,                                      // 1, 2, 3
        0x00, 0x00, 0x00, 0x03,                                                  // 3
        0x3c, 0x23, 0xd7, 0x0a, 0x3f, 0x7d, 0x70, 0xa4, 0x40, 0x48, 0xf5, 0xc3   // 0.01f, 0.99f, 3.14f
    };
    unsigned char bytes[sizeof(expected)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytes), sizeof(expected));

    ASSERT_ELEMENTS_EQ(bytes, expected, sizeof(expected));
}

TEST(BinaryOutputArchiveTest, AttachUserData) {
    int userData = {};
    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    ASSERT_EQ(archive.getUserData(), nullptr);
    archive.setUserData(&userData);
    ASSERT_EQ(archive.getUserData(), &userData);
}

TEST(BinaryOutputArchiveTest, SaveVersionedType) {
    UpgradedSerializableType source{1234u, 255u};

    tersetests::FakeStream stream;
    terse::BinaryOutputArchive<tersetests::FakeStream> archive(&stream);
    archive(terse::versioned(source, terse::Version<1>{}));
    archive(terse::versioned(source, terse::Version<2>{}));

    unsigned char expectedv1[] = {0x00, 0x00, 0x04, 0xd2};              // 1234
    unsigned char expectedv2[] = {0x00, 0x00, 0x04, 0xd2, 0x00, 0xff};  // 1234, 255

    unsigned char bytesv1[sizeof(expectedv1)];
    unsigned char bytesv2[sizeof(expectedv2)];
    stream.seek(0ul);
    stream.read(reinterpret_cast<char*>(bytesv1), sizeof(bytesv1));
    stream.read(reinterpret_cast<char*>(bytesv2), sizeof(bytesv2));

    ASSERT_ELEMENTS_EQ(bytesv1, expectedv1, sizeof(expectedv1));
    ASSERT_ELEMENTS_EQ(bytesv2, expectedv2, sizeof(expectedv2));
}
