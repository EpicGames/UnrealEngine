// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "terse/types/Transparent.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

struct JSONStruct {
    std::int8_t a;
    std::uint8_t b;
    std::int16_t c;
    std::uint16_t d;
    std::int32_t e;
    std::uint32_t f;

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("a");
        archive(a);
        archive.label("b");
        archive(b);
        archive.label("c");
        archive(c);
        archive.label("d");
        archive(d);
        archive.label("e");
        archive(e);
        archive.label("f");
        archive(f);
    }
};

inline bool operator==(const JSONStruct& lhs, const JSONStruct& rhs) {
    return (lhs.a == rhs.a) && (lhs.b == rhs.b) && (lhs.c == rhs.c) && (lhs.d == rhs.d) && (lhs.e == rhs.e) && (lhs.f == rhs.f);
}

struct NestedJSONStruct {
    JSONStruct a;
    JSONStruct b;
    std::vector<std::uint16_t> c;

    struct NestedVector {
        std::vector<std::int32_t> cc;

        template<class Archive>
        void serialize(Archive& archive) {
            archive.label("nested array in struct");
            archive(cc);
        }
    };

    NestedVector d;

    template<class Archive>
    void serialize(Archive& archive) {
        archive.label("first");
        archive(a);
        archive.label("another");
        archive(b);
        archive.label("array");
        archive(c);
        archive.label("array_in_struct");
        archive(d);
    }
};

inline bool operator==(const NestedJSONStruct::NestedVector& lhs, const NestedJSONStruct::NestedVector& rhs) {
    return (lhs.cc == rhs.cc);
}

inline bool operator==(const NestedJSONStruct& lhs, const NestedJSONStruct& rhs) {
    return (lhs.a == rhs.a) && (lhs.b == rhs.b) && (lhs.c == rhs.c) && (lhs.d == rhs.d);
}

struct InnerOpaque {
    std::uint32_t a;
    std::uint32_t b;

    template<typename Archive>
    void serialize(Archive& archive) {
        archive.label("a");
        archive(a);
        archive.label("b");
        archive(b);
    }
};

struct OuterTransparent {
    InnerOpaque inner;

    template<typename Archive>
    void serialize(Archive& archive) {
        archive(terse::transparent(inner));
    }
};
