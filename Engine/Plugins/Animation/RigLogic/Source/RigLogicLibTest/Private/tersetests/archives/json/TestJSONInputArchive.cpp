// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"
#include "tersetests/FakeStream.h"
#include "tersetests/archives/json/Common.h"

#include "terse/archives/json/InputArchive.h"

#include <pma/TypeDefs.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

TEST(JSONInputArchiveTest, InputPrimitiveStruct) {
    const char data[] = "{\n"
                        "    \"a\": 127,\n"
                        "    \"b\": 255,\n"
                        "    \"c\": 32767,\n"
                        "    \"d\": 65535,\n"
                        "    \"e\": 2147483647,\n"
                        "    \"f\": 4294967295\n"
                        "}";
    JSONStruct expected = {127, 255, 32767, 65535, 2147483647, 4294967295};

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    JSONStruct result;
    archive >> result;

    ASSERT_EQ(result, expected);
    ASSERT_TRUE(archive.isOk());
}

TEST(JSONInputArchiveTest, InputEmptyVector) {
    const char data[] = "[]";
    std::vector<int> expected;

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    std::vector<int> result;
    archive >> result;

    ASSERT_EQ(result, expected);
    ASSERT_TRUE(archive.isOk());
}

TEST(JSONInputArchiveTest, InputVectorOfPrimitives) {
    const char data[] = "[-10, 0, 1, 10, 10000]";
    std::vector<int> expected{-10, 0, 1, 10, 10000};

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    std::vector<int> result;
    archive >> result;

    ASSERT_EQ(result, expected);
    ASSERT_TRUE(archive.isOk());
}

TEST(JSONInputArchiveTest, InputBlob) {
    const char data[] = "\"AQIDBP8=\"";
    const char expected[] = "\x1\x2\x3\x4\xFF";

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    terse::Blob<char, pma::PolyAllocator<char>> dest;
    dest.setSize(5ul);

    archive >> dest;

    ASSERT_EQ(dest.size(), sizeof(expected) - 1);
    ASSERT_ELEMENTS_EQ(dest.data(), expected, sizeof(expected) - 1);
    ASSERT_TRUE(archive.isOk());
}

TEST(JSONInputArchiveTest, InputCharArray) {
    const char data[] = "[\"A\", \"B\", \"C\"]";
    std::array<char, 3> expected{'A', 'B', 'C'};

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    std::array<char, 3> result;
    archive >> result;

    ASSERT_EQ(result, expected);
    ASSERT_TRUE(archive.isOk());
}

TEST(JSONInputArchiveTest, InputVectorOfStructs) {
    const char data[] = "["
                        "{\n"
                        "    \"a\": 127,\n"
                        "    \"b\": 255,\n"
                        "    \"c\": 32767,\n"
                        "    \"d\": 65535,\n"
                        "    \"e\": 2147483647,\n"
                        "    \"f\": 4294967295\n"
                        "}, {\n"
                        "    \"a\": 127,\n"
                        "    \"b\": 255,\n"
                        "    \"c\": 32767,\n"
                        "    \"d\": 65535,\n"
                        "    \"e\": 2147483647,\n"
                        "    \"f\": 4294967295\n"
                        "}, {\n"
                        "    \"a\": 127,\n"
                        "    \"b\": 255,\n"
                        "    \"c\": 32767,\n"
                        "    \"d\": 65535,\n"
                        "    \"e\": 2147483647,\n"
                        "    \"f\": 4294967295\n"
                        "}"
                        "]";
    JSONStruct strukt = {127, 255, 32767, 65535, 2147483647, 4294967295};
    std::vector<JSONStruct> expected{strukt, strukt, strukt};

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    std::vector<JSONStruct> result;
    archive >> result;

    ASSERT_EQ(result, expected);
    ASSERT_TRUE(archive.isOk());
}

TEST(JSONInputArchiveTest, InputNestedStructs) {
    const char data[] = "{\n"
                        "    \"first\": {\n"
                        "        \"a\": 127,\n"
                        "        \"b\": 255,\n"
                        "        \"c\": 32767,\n"
                        "        \"d\": 65535,\n"
                        "        \"e\": 2147483647,\n"
                        "        \"f\": 4294967295\n"
                        "    },\n"
                        "    \"another\": {\n"
                        "        \"a\": 127,\n"
                        "        \"b\": 255,\n"
                        "        \"c\": 32767,\n"
                        "        \"d\": 65535,\n"
                        "        \"e\": 2147483647,\n"
                        "        \"f\": 4294967295\n"
                        "    },\n"
                        "    \"array\": [10, 20, 30],\n"
                        "    \"array_in_struct\": {\n"
                        "        \"nested array in struct\": [-20, -1, 0, 1, 20]\n"
                        "    }\n"
                        "}";
    JSONStruct strukt = {127, 255, 32767, 65535, 2147483647, 4294967295};
    NestedJSONStruct expected{strukt, strukt, {10, 20, 30}, {{-20, -1, 0, 1, 20}}};

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    NestedJSONStruct result;
    archive >> result;

    ASSERT_EQ(result, expected);
    ASSERT_TRUE(archive.isOk());
}

TEST(JSONInputArchiveTest, TransparentStruct) {
    const char data[] = "{\n"
                        "    \"a\": 10,\n"
                        "    \"b\": 20\n"
                        "}";

    tersetests::FakeStream stream;
    stream.write(reinterpret_cast<const char*>(&data), sizeof(data));
    stream.seek(0ul);

    terse::JSONInputArchive<tersetests::FakeStream> archive{&stream};
    OuterTransparent result;
    archive >> result;

    ASSERT_EQ(result.inner.a, 10u);
    ASSERT_EQ(result.inner.b, 20u);
    ASSERT_TRUE(archive.isOk());
}
