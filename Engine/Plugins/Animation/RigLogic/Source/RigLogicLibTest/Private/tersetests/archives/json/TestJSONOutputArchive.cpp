// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"
#include "tersetests/FakeStream.h"
#include "tersetests/archives/json/Common.h"

#include "terse/archives/json/OutputArchive.h"

#include <pma/TypeDefs.h>

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

TEST(JSONOutputArchiveTest, OutputPrimitiveStruct) {
    JSONStruct source = {127, 255, 32767, 65535, 2147483647, 4294967295};

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char* expected = "{\n"
                           "    \"a\": 127,\n"
                           "    \"b\": 255,\n"
                           "    \"c\": 32767,\n"
                           "    \"d\": 65535,\n"
                           "    \"e\": 2147483647,\n"
                           "    \"f\": 4294967295\n"
                           "}";
    const std::size_t expectedSize = std::strlen(expected);

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(JSONOutputArchiveTest, OutputVectorOfPrimitives) {
    std::vector<int> source{-10, 0, 1, 10, 10000};

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char* expected = "[-10, 0, 1, 10, 10000]";
    const std::size_t expectedSize = std::strlen(expected);

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(JSONOutputArchiveTest, OutputBlob) {
    terse::Blob<char, pma::PolyAllocator<char>> source;
    source.setSize(5ul);
    std::memcpy(source.data(), "\x1\x2\x3\x4\xFF", source.size());

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char expected[] = "\"AQIDBP8=\"";
    const std::size_t expectedSize = std::strlen(expected);

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(JSONOutputArchiveTest, OutputCharArray) {
    std::array<char, 3> source{'A', 'B', 'C'};

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char expected[] = "[\"A\", \"B\", \"C\"]";
    const std::size_t expectedSize = std::strlen(expected);

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(JSONOutputArchiveTest, OutputVectorOfStructs) {
    JSONStruct strukt = {127, 255, 32767, 65535, 2147483647, 4294967295};
    std::vector<JSONStruct> source{strukt, strukt, strukt};

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char* expected = "["
                           "{\n"
                           "    \"a\": 127,\n"
                           "    \"b\": 255,\n"
                           "    \"c\": 32767,\n"
                           "    \"d\": 65535,\n"
                           "    \"e\": 2147483647,\n"
                           "    \"f\": 4294967295\n"
                           "}, {\n"
                           "    \"a\": 127,\n"
                           "    \"b\": 255,\n"
                           "    \"c\": 32767,\n"
                           "    \"d\": 65535,\n"
                           "    \"e\": 2147483647,\n"
                           "    \"f\": 4294967295\n"
                           "}, {\n"
                           "    \"a\": 127,\n"
                           "    \"b\": 255,\n"
                           "    \"c\": 32767,\n"
                           "    \"d\": 65535,\n"
                           "    \"e\": 2147483647,\n"
                           "    \"f\": 4294967295\n"
                           "}"
                           "]";
    const std::size_t expectedSize = std::strlen(expected);

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(JSONOutputArchiveTest, OutputNestedStructs) {
    JSONStruct strukt = {127, 255, 32767, 65535, 2147483647, 4294967295};
    NestedJSONStruct source{strukt, strukt, {10, 20, 30}, {{-20, -1, 0, 1, 20}}};

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char* expected = "{\n"
                           "    \"first\": {\n"
                           "        \"a\": 127,\n"
                           "        \"b\": 255,\n"
                           "        \"c\": 32767,\n"
                           "        \"d\": 65535,\n"
                           "        \"e\": 2147483647,\n"
                           "        \"f\": 4294967295\n"
                           "    },\n"
                           "    \"another\": {\n"
                           "        \"a\": 127,\n"
                           "        \"b\": 255,\n"
                           "        \"c\": 32767,\n"
                           "        \"d\": 65535,\n"
                           "        \"e\": 2147483647,\n"
                           "        \"f\": 4294967295\n"
                           "    },\n"
                           "    \"array\": [10, 20, 30],\n"
                           "    \"array_in_struct\": {\n"
                           "        \"nested array in struct\": [-20, -1, 0, 1, 20]\n"
                           "    }\n"
                           "}";

    const std::size_t expectedSize = std::strlen(expected);
    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(JSONOutputArchiveTest, OutputEmptyVector) {
    std::vector<int> source;

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char* expected = "[]";
    const std::size_t expectedSize = std::strlen(expected);

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(JSONOutputArchiveTest, TransparentStruct) {
    OuterTransparent source{{10u, 20u}};

    tersetests::FakeStream stream;
    terse::JSONOutputArchive<tersetests::FakeStream> archive{&stream, 4u};
    archive << source;
    archive.sync();

    const char* expected = "{\n"
                           "    \"a\": 10,\n"
                           "    \"b\": 20\n"
                           "}";
    const std::size_t expectedSize = std::strlen(expected);

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}
