// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"

#include "terse/types/ArchiveOffset.h"

struct ArchiveOffsetMoveTest {
    terse::ArchiveOffset<std::uint32_t> offset1;
    terse::ArchiveOffset<std::uint32_t> offset2;

    terse::ArchiveOffset<std::uint32_t>::Proxy proxy1;

    ArchiveOffsetMoveTest() :
        offset1{},
        offset2{},
        proxy1{offset1} {
    }

    ~ArchiveOffsetMoveTest() = default;

    ArchiveOffsetMoveTest(const ArchiveOffsetMoveTest&) = delete;
    ArchiveOffsetMoveTest& operator=(const ArchiveOffsetMoveTest&) = delete;

    ArchiveOffsetMoveTest(ArchiveOffsetMoveTest&& rhs) :
        offset1{},
        offset2{},
        proxy1{offset1} {
        std::swap(offset1, rhs.offset1);
        std::swap(offset2, rhs.offset2);
        std::swap(proxy1, rhs.proxy1);
    }

    ArchiveOffsetMoveTest& operator=(ArchiveOffsetMoveTest&& rhs) {
        std::swap(offset1, rhs.offset1);
        std::swap(offset2, rhs.offset2);
        std::swap(proxy1, rhs.proxy1);
        return *this;
    }

    void fakeSerialize() {
        // Create temporary proxy, which is destroyed after serialization
        terse::proxy(offset2);
    }
};

TEST(ArchiveOffsetTest, SwapInstances) {
    ArchiveOffsetMoveTest instance1{};
    ArchiveOffsetMoveTest instance2{};

    std::swap(instance1, instance2);

    ASSERT_EQ(instance1.offset1.proxy, &instance1.proxy1);
    ASSERT_EQ(instance1.offset2.proxy, nullptr);
    ASSERT_EQ(instance1.proxy1.target, &instance1.offset1);

    ASSERT_EQ(instance2.offset1.proxy, &instance2.proxy1);
    ASSERT_EQ(instance2.offset2.proxy, nullptr);
    ASSERT_EQ(instance2.proxy1.target, &instance2.offset1);
}

TEST(ArchiveOffsetTest, MoveInstances) {
    std::vector<ArchiveOffsetMoveTest> instances(2ul);

    instances.push_back({});
    instances.push_back({});
    instances.push_back({});
    instances.push_back({});
    instances.push_back({});

    ASSERT_EQ(instances[1].offset1.proxy, &instances[1].proxy1);
    ASSERT_EQ(instances[1].offset2.proxy, nullptr);
    ASSERT_EQ(instances[1].proxy1.target, &instances[1].offset1);

    ASSERT_EQ(instances[2].offset1.proxy, &instances[2].proxy1);
    ASSERT_EQ(instances[2].offset2.proxy, nullptr);
    ASSERT_EQ(instances[2].proxy1.target, &instances[2].offset1);
}

TEST(ArchiveOffsetTest, MoveAfterTemporaryProxy) {
    ArchiveOffsetMoveTest instance1{};
    // This will cause it's offset2 member to have an association with a deleted proxy
    instance1.fakeSerialize();
    // If proxy cleanup is correctly implemented, association should be broken after temporary proxy is destroyed
    ArchiveOffsetMoveTest instance2 = std::move(instance1);
}
