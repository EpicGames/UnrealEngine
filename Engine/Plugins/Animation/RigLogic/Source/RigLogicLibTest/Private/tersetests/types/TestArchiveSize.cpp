// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"

#include "terse/types/ArchiveSize.h"

struct ArchiveSizeMoveTest {
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size1;
    terse::ArchiveSize<std::uint32_t, std::uint32_t> size2;

    terse::Anchor<std::uint32_t> anchor1;
    terse::ArchiveSize<std::uint32_t, std::uint32_t>::Proxy proxy1;

    ArchiveSizeMoveTest() :
        size1{},
        size2{},
        proxy1{size1, anchor1} {
    }

    ~ArchiveSizeMoveTest() = default;

    ArchiveSizeMoveTest(const ArchiveSizeMoveTest&) = delete;
    ArchiveSizeMoveTest& operator=(const ArchiveSizeMoveTest&) = delete;

    ArchiveSizeMoveTest(ArchiveSizeMoveTest&& rhs) :
        size1{},
        size2{},
        proxy1{size1, anchor1} {
        std::swap(size1, rhs.size1);
        std::swap(size2, rhs.size2);
        std::swap(anchor1, rhs.anchor1);
        std::swap(proxy1, rhs.proxy1);
    }

    ArchiveSizeMoveTest& operator=(ArchiveSizeMoveTest&& rhs) {
        std::swap(size1, rhs.size1);
        std::swap(size2, rhs.size2);
        std::swap(anchor1, rhs.anchor1);
        std::swap(proxy1, rhs.proxy1);
        return *this;
    }

    void fakeSerialize() {
        // Create temporary proxy, which is destroyed after serialization
        terse::Anchor<std::uint32_t> anchor2{};
        terse::proxy(size2, anchor2);
    }
};

TEST(ArchiveSizeTest, SwapInstances) {
    ArchiveSizeMoveTest instance1{};
    ArchiveSizeMoveTest instance2{};

    std::swap(instance1, instance2);

    ASSERT_EQ(instance1.size1.proxy, &instance1.proxy1);
    ASSERT_EQ(instance1.size2.proxy, nullptr);
    ASSERT_EQ(instance1.proxy1.target, &instance1.size1);
    ASSERT_EQ(instance1.proxy1.base, &instance1.anchor1);

    ASSERT_EQ(instance2.size1.proxy, &instance2.proxy1);
    ASSERT_EQ(instance2.size2.proxy, nullptr);
    ASSERT_EQ(instance2.proxy1.target, &instance2.size1);
    ASSERT_EQ(instance2.proxy1.base, &instance2.anchor1);
}

TEST(ArchiveSizeTest, MoveInstances) {
    std::vector<ArchiveSizeMoveTest> instances(2ul);

    instances.push_back({});
    instances.push_back({});
    instances.push_back({});
    instances.push_back({});
    instances.push_back({});

    ASSERT_EQ(instances[1].size1.proxy, &instances[1].proxy1);
    ASSERT_EQ(instances[1].size2.proxy, nullptr);
    ASSERT_EQ(instances[1].proxy1.target, &instances[1].size1);
    ASSERT_EQ(instances[1].proxy1.base, &instances[1].anchor1);

    ASSERT_EQ(instances[2].size1.proxy, &instances[2].proxy1);
    ASSERT_EQ(instances[2].size2.proxy, nullptr);
    ASSERT_EQ(instances[2].proxy1.target, &instances[2].size1);
    ASSERT_EQ(instances[2].proxy1.base, &instances[2].anchor1);
}

TEST(ArchiveSizeTest, MoveAfterTemporaryProxy) {
    ArchiveSizeMoveTest instance1{};
    // This will cause it's size2 member to have an association with a deleted proxy and anchor
    instance1.fakeSerialize();
    // If proxy cleanup is correctly implemented, both associations should be broken after temporary proxy is destroyed
    ArchiveSizeMoveTest instance2 = std::move(instance1);
}
