// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"
#include "tersetests/FakeStream.h"

#include "terse/types/CharInputStreamBuf.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <ostream>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

TEST(CharInputStreamBufTest, InputString) {
    tersetests::FakeStream stream;
    const char data[] = "a space delimited\n string";
    stream.write(reinterpret_cast<const char*>(&data), std::strlen(data));
    stream.seek(0ul);

    terse::CharInputStreamBuf<tersetests::FakeStream> streamBuf{&stream};
    std::istream inStream{&streamBuf};

    // Whitespace is used as delimiter by std::istream in all cases (including with std::noskipws
    // which keeps only leading whitespace, not delimiters)
    std::string result;

    inStream >> result;
    ASSERT_EQ(result, "a");

    inStream >> result;
    ASSERT_EQ(result, "space");

    inStream >> result;
    ASSERT_EQ(result, "delimited");

    inStream >> result;
    ASSERT_EQ(result, "string");
}

TEST(CharInputStreamBufTest, InputMixedDataTypes) {
    tersetests::FakeStream stream;
    const char data[] = "42 word 3.14 \n last.";
    stream.write(reinterpret_cast<const char*>(&data), std::strlen(data));
    stream.seek(0ul);

    terse::CharInputStreamBuf<tersetests::FakeStream> streamBuf{&stream};
    std::istream inStream{&streamBuf};

    std::string strResult;
    int intResult = {};
    float floatResult = {};

    inStream >> intResult;
    ASSERT_EQ(intResult, 42);

    inStream >> strResult;
    ASSERT_EQ(strResult, "word");

    inStream >> floatResult;
    ASSERT_EQ(floatResult, 3.14f);

    inStream >> strResult;
    ASSERT_EQ(strResult, "last.");
}
