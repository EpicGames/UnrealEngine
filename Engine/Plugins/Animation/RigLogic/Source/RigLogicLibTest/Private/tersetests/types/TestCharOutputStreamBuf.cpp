// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"
#include "tersetests/FakeStream.h"

#include "terse/types/CharOutputStreamBuf.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdint>
#include <ostream>
#include <string>
#include <vector>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

TEST(CharOutputStreamBufTest, OutputSingleString) {
    tersetests::FakeStream stream;
    terse::CharOutputStreamBuf<tersetests::FakeStream> streamBuf{&stream};
    std::ostream outStream{&streamBuf};

    const char* expected = "a single string";
    const std::size_t expectedSize = std::strlen(expected);
    outStream << expected;
    outStream << std::flush;

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(CharOutputStreamBufTest, OutputMultipleStrings) {
    tersetests::FakeStream stream;
    terse::CharOutputStreamBuf<tersetests::FakeStream> streamBuf{&stream};
    std::ostream outStream{&streamBuf};

    const char* expected = "a single string";
    const std::size_t expectedSize = std::strlen(expected);
    outStream << "a" << " " << "single" << " string";
    outStream << std::flush;

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}

TEST(CharOutputStreamBufTest, OutputMixedDataTypes) {
    tersetests::FakeStream stream;
    terse::CharOutputStreamBuf<tersetests::FakeStream> streamBuf{&stream};

    std::ostream outStream{&streamBuf};

    const char* expected = "42 is less than 100 \n but greater than life. 3.14";
    const std::size_t expectedSize = std::strlen(expected);
    outStream << 42 << " " << "is" << " " << "less" << " " << "than" << " " << 100U << " \n"
              << " but " << "greater" << " than life. " << 3.14f;
    outStream << std::flush;

    ASSERT_EQ(stream.size(), expectedSize);
    ASSERT_ELEMENTS_EQ(stream.data.data(), expected, expectedSize);
}
