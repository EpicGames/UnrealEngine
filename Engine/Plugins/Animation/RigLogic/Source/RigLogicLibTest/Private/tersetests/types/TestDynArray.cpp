// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"

#include "terse/types/DynArray.h"

#include <pma/PolyAllocator.h>
#include <pma/TypeDefs.h>
#include <pma/resources/AlignedMemoryResource.h>

TEST(DynArrayTest, CreateEmpty) {
    terse::DynArray<int, pma::PolyAllocator<int>> arr;
    ASSERT_TRUE(arr.empty());
    ASSERT_EQ(arr.size(), 0ul);
    ASSERT_EQ(arr.data(), nullptr);
}

TEST(DynArrayTest, CreateUninitialized) {
    terse::DynArray<int, pma::PolyAllocator<int>> arr{10ul};
    ASSERT_FALSE(arr.empty());
    ASSERT_EQ(arr.size(), 10ul);
    ASSERT_NE(arr.data(), nullptr);
}

TEST(DynArrayTest, CreateInitialized) {
    terse::DynArray<int, pma::PolyAllocator<int>> arr{10ul, 2};
    ASSERT_FALSE(arr.empty());
    ASSERT_EQ(arr.size(), 10ul);
    ASSERT_NE(arr.data(), nullptr);

    for (std::size_t i{}; i < arr.size(); ++i) {
        ASSERT_EQ(arr[i], 2);
    }
}

TEST(DynArrayTest, CreateFromRange) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, values + 10ul};
    ASSERT_FALSE(arr.empty());
    ASSERT_EQ(arr.size(), 10ul);
    ASSERT_NE(arr.data(), nullptr);
    ASSERT_NE(arr.data(), values);

    for (std::size_t i{}; i < arr.size(); ++i) {
        ASSERT_EQ(arr[i], values[i]);
    }
}

TEST(DynArrayTest, CreateFromEmptyRange) {
    pma::Vector<int> values;
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values.begin(), values.end()};
    ASSERT_TRUE(arr.empty());
}

TEST(DynArrayTest, CreateFromPointerSize) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};
    ASSERT_FALSE(arr.empty());
    ASSERT_EQ(arr.size(), 10ul);
    ASSERT_NE(arr.data(), nullptr);
    ASSERT_NE(arr.data(), values);

    for (std::size_t i{}; i < arr.size(); ++i) {
        ASSERT_EQ(arr[i], values[i]);
    }
}

TEST(DynArrayTest, CopyConstruct) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};
    terse::DynArray<int, pma::PolyAllocator<int>> arrCopy = arr;

    ASSERT_EQ(arr.size(), 10ul);
    ASSERT_EQ(arr.size(), arrCopy.size());
    for (std::size_t i{}; i < arr.size(); ++i) {
        ASSERT_EQ(arr[i], values[i]);
        ASSERT_EQ(arr[i], arrCopy[i]);
    }

    ASSERT_NE(arr.data(), arrCopy.data());
}

TEST(DynArrayTest, CopyAssign) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};
    terse::DynArray<int, pma::PolyAllocator<int>> arrCopy;
    arrCopy = arr;

    ASSERT_EQ(arr.size(), 10ul);
    ASSERT_EQ(arr.size(), arrCopy.size());
    for (std::size_t i{}; i < arr.size(); ++i) {
        ASSERT_EQ(arr[i], values[i]);
        ASSERT_EQ(arr[i], arrCopy[i]);
    }

    ASSERT_NE(arr.data(), arrCopy.data());
}

TEST(DynArrayTest, MoveConstruct) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};
    terse::DynArray<int, pma::PolyAllocator<int>> arrCopy = std::move(arr);

// Hide these assertions from clang static analyzer, as move operations for DynArrays are
// well defined, but clang doesn't know that, so it will always warn about accessing
// moved-from objects.
#ifndef __clang_analyzer__
    ASSERT_TRUE(arr.empty());
    ASSERT_EQ(arr.size(), 0ul);
    ASSERT_EQ(arr.data(), nullptr);
#endif  // __clang_analyzer__

    ASSERT_EQ(arrCopy.size(), 10ul);
    for (std::size_t i{}; i < arrCopy.size(); ++i) {
        ASSERT_EQ(arrCopy[i], values[i]);
    }

    ASSERT_NE(arrCopy.data(), values);
}

TEST(DynArrayTest, MoveAssign) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};
    terse::DynArray<int, pma::PolyAllocator<int>> arrCopy;
    arrCopy = std::move(arr);

// Hide these assertions from clang static analyzer, as move operations for DynArrays are
// well defined, but clang doesn't know that, so it will always warn about accessing
// moved-from objects.
#ifndef __clang_analyzer__
    ASSERT_TRUE(arr.empty());
    ASSERT_EQ(arr.size(), 0ul);
    ASSERT_EQ(arr.data(), nullptr);
#endif  // __clang_analyzer__

    ASSERT_EQ(arrCopy.size(), 10ul);
    for (std::size_t i{}; i < arrCopy.size(); ++i) {
        ASSERT_EQ(arrCopy[i], values[i]);
    }

    ASSERT_NE(arrCopy.data(), values);
}

TEST(DynArrayTest, SubscriptOperator) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};

    auto raw = arr.data();
    for (std::size_t i{}; i < 10ul; ++i) {
        ASSERT_EQ(arr[i], raw[i]);
    }
}

TEST(DynArrayTest, ResizeToSmaller) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};

    ASSERT_EQ(arr.size(), 10ul);

    auto preResize = arr.data();
    arr.resize(5ul);
    // Ensure pointer does not change, just the size is reduced
    ASSERT_EQ(preResize, arr.data());
    ASSERT_EQ(arr.size(), 5ul);

    for (std::size_t i{}; i < 5ul; ++i) {
        ASSERT_EQ(arr[i], values[i]);
    }
}

TEST(DynArrayTest, ResizeToLarger) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};

    ASSERT_EQ(arr.size(), 10ul);

    auto preResize = arr.data();
    arr.resize(15ul, 42);
    // Ensure pointer has changed
    ASSERT_NE(preResize, arr.data());
    ASSERT_EQ(arr.size(), 15ul);
    // Ensure existing data has been migrated
    for (std::size_t i{}; i < 10ul; ++i) {
        ASSERT_EQ(arr[i], values[i]);
    }
    // Ensure rest of the storage was filled with given value
    for (std::size_t i = 10ul; i < 15ul; ++i) {
        ASSERT_EQ(arr[i], 42);
    }
}

TEST(DynArrayTest, ResizeUninitializedToSmaller) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};

    ASSERT_EQ(arr.size(), 10ul);

    auto preResize = arr.data();
    arr.resize_uninitialized(5ul);
    // Ensure pointer does not change, just the size is reduced
    ASSERT_EQ(preResize, arr.data());
    ASSERT_EQ(arr.size(), 5ul);

    for (std::size_t i{}; i < 5ul; ++i) {
        ASSERT_EQ(arr[i], values[i]);
    }
}

TEST(DynArrayTest, ResizeUninitializedToLarger) {
    int values[10] = {1, 2, 3, 4, 5, 6, 7, 8, 9, 10};
    terse::DynArray<int, pma::PolyAllocator<int>> arr{values, 10ul};

    ASSERT_EQ(arr.size(), 10ul);

    auto preResize = arr.data();
    arr.resize_uninitialized(15ul);
    // Ensure pointer has changed
    ASSERT_NE(preResize, arr.data());
    ASSERT_EQ(arr.size(), 15ul);
    // Ensure existing data has been migrated
    for (std::size_t i{}; i < 10ul; ++i) {
        ASSERT_EQ(arr[i], values[i]);
    }
    // Rest of the storage was not initialized, no way to test
}
