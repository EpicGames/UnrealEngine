// Copyright Epic Games, Inc. All Rights Reserved.

#include "tersetests/Defs.h"

#include "terse/utils/Base64.h"

namespace {

class Base64Test : public ::testing::TestWithParam<std::size_t> {};

}  // namespace

TEST_P(Base64Test, EncodeDecodeSize) {
    auto roundUp = [](std::size_t number, std::size_t multiple) { return ((number + multiple - 1) / multiple) * multiple; };

    const auto dataSize = GetParam();
    // Calculates padded size
    const auto encodedSize = terse::base64encode(dataSize);
    // Decoded size does not take padding into account, so result will always assume data was not padded
    // (and will require a slightly larger buffer if padding was present)
    // Original data length -> Decoded length
    // 1 -> 3
    // 2 -> 3
    // 3 -> 3
    // 4 -> 6
    // 5 -> 6
    // 6 -> 6
    // 7 -> 9
    // 8 -> 9
    const auto decodedSize = terse::base64decode(encodedSize);
    ASSERT_EQ(roundUp(dataSize, 3ul), decodedSize);
}

INSTANTIATE_TEST_SUITE_P(Base64Test, Base64Test, ::testing::Values(0ul, 1ul, 2ul, 3ul, 4ul, 5ul, 6ul, 7ul, 8ul));
