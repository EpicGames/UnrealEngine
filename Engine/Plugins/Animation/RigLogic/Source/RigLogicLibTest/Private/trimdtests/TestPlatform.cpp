// Copyright Epic Games, Inc. All Rights Reserved.

#include "trimdtests/Defs.h"

#include "trimd/TRiMD.h"

TEST(CPUFeatureDetection, Platform) {
    trimd::CPUFeatures features = trimd::getCPUFeatures();
    if (features.NEON) {
        // If NEON is detected, none of the x86 CPU features should be available
        ASSERT_FALSE(features.SSE);
        ASSERT_FALSE(features.SSE2);
        ASSERT_FALSE(features.SSE3);
        ASSERT_FALSE(features.SSSE3);
        ASSERT_FALSE(features.SSE41);
        ASSERT_FALSE(features.SSE42);
        ASSERT_FALSE(features.AVX);
        ASSERT_FALSE(features.F16C);
    } else {
        // If NEON is not detected, x86 features might be found (depending on hardware)
        // But FP16 should not be available without NEON
        ASSERT_FALSE(features.FP16);
    }
}
