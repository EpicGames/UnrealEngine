// Copyright Epic Games, Inc. All Rights Reserved.

#include "trimdtests/Defs.h"

#include "trimd/TRiMD.h"

using T128Types = ::testing::Types<trimd::scalar::F128
#ifdef TRIMD_ENABLE_SSE
                                   ,
                                   trimd::sse::F128
#endif  // TRIMD_ENABLE_SSE
#ifdef TRIMD_ENABLE_NEON
                                   ,
                                   trimd::neon::F128
#endif  // TRIMD_ENABLE_NEON
                                   >;

#ifdef TRIMD_ENABLE_SSE
bool equal(const trimd::sse::F128& lhs, const trimd::sse::F128& rhs) {
    return (std::memcmp(&lhs.data, &rhs.data, sizeof(lhs.data)) == 0);
}

#endif  // TRIMD_ENABLE_SSE

#ifdef TRIMD_ENABLE_NEON
bool equal(const trimd::neon::F128& lhs, const trimd::neon::F128& rhs) {
    return (std::memcmp(&lhs.data, &rhs.data, sizeof(lhs.data)) == 0);
}

#endif  // TRIMD_ENABLE_NEON

template<typename TF128>
static TF128 frombits(uint32_t bits0, uint32_t bits1, uint32_t bits2, uint32_t bits3) {
    return TF128{trimd::bitcast<float>(bits0),
                 trimd::bitcast<float>(bits1),
                 trimd::bitcast<float>(bits2),
                 trimd::bitcast<float>(bits3)};
}

static bool equal(const trimd::scalar::F128& lhs, const trimd::scalar::F128& rhs) {
    return (std::memcmp(lhs.data.data(), rhs.data.data(), sizeof(float) * trimd::scalar::F128::size()) == 0);
}

template<typename T>
class T128Test : public ::testing::Test {
protected:
    using T128 = T;
};

TYPED_TEST_SUITE(T128Test, T128Types, );

TYPED_TEST(T128Test, CheckSize) {
    ASSERT_EQ(TestFixture::T128::size(), 4ul);
}

TYPED_TEST(T128Test, Equality) {
    using F128 = typename TestFixture::T128;
    F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v3{1.5f, 2.0f, 3.0f, 4.0f};
    F128 v4{1.0f, 2.5f, 3.0f, 4.0f};
    F128 v5{1.0f, 2.0f, 3.5f, 4.0f};
    F128 v6{1.0f, 2.0f, 3.0f, 4.5f};

    F128 m12 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F128 m13 = frombits<F128>(0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F128 m14 = frombits<F128>(0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F128 m15 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu);
    F128 m16 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u);

    ASSERT_TRUE(equal(v1 == v2, m12));
    ASSERT_TRUE(equal(v1 == v3, m13));
    ASSERT_TRUE(equal(v1 == v4, m14));
    ASSERT_TRUE(equal(v1 == v5, m15));
    ASSERT_TRUE(equal(v1 == v6, m16));
}

TYPED_TEST(T128Test, Inequality) {
    using F128 = typename TestFixture::T128;
    F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v3{1.5f, 2.0f, 3.0f, 4.0f};
    F128 v4{1.0f, 2.5f, 3.0f, 4.0f};
    F128 v5{1.0f, 2.0f, 3.5f, 4.0f};
    F128 v6{1.0f, 2.0f, 3.0f, 4.5f};

    F128 m12 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 m13 = frombits<F128>(0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 m14 = frombits<F128>(0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u);
    F128 m15 = frombits<F128>(0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u);
    F128 m16 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(v1 != v2, m12));
    ASSERT_TRUE(equal(v1 != v3, m13));
    ASSERT_TRUE(equal(v1 != v4, m14));
    ASSERT_TRUE(equal(v1 != v5, m15));
    ASSERT_TRUE(equal(v1 != v6, m16));
}

TYPED_TEST(T128Test, LessThan) {
    using F128 = typename TestFixture::T128;
    F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v3{2.0f, 3.0f, 4.0f, 5.0f};
    F128 v4{0.5f, 1.5f, 2.5f, 3.5f};

    F128 m12 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 m13 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F128 m14 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);

    ASSERT_TRUE(equal(v1 < v2, m12));
    ASSERT_TRUE(equal(v1 < v3, m13));
    ASSERT_TRUE(equal(v1 < v4, m14));
}

TYPED_TEST(T128Test, LessThanOrEqual) {
    using F128 = typename TestFixture::T128;
    F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v3{2.0f, 3.0f, 4.0f, 5.0f};
    F128 v4{0.5f, 1.5f, 2.5f, 3.5f};

    F128 m12 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F128 m13 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F128 m14 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);

    ASSERT_TRUE(equal(v1 <= v2, m12));
    ASSERT_TRUE(equal(v1 <= v3, m13));
    ASSERT_TRUE(equal(v1 <= v4, m14));
}

TYPED_TEST(T128Test, GreaterThan) {
    using F128 = typename TestFixture::T128;
    F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v3{2.0f, 3.0f, 4.0f, 5.0f};
    F128 v4{0.5f, 1.5f, 2.5f, 3.5f};

    F128 m12 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 m13 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 m14 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(v1 > v2, m12));
    ASSERT_TRUE(equal(v1 > v3, m13));
    ASSERT_TRUE(equal(v1 > v4, m14));
}

TYPED_TEST(T128Test, GreaterThanOrEqual) {
    using F128 = typename TestFixture::T128;
    F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    F128 v3{2.0f, 3.0f, 4.0f, 5.0f};
    F128 v4{0.5f, 1.5f, 2.5f, 3.5f};

    F128 m12 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F128 m13 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 m14 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(v1 >= v2, m12));
    ASSERT_TRUE(equal(v1 >= v3, m13));
    ASSERT_TRUE(equal(v1 >= v4, m14));
}

TYPED_TEST(T128Test, BitwiseAND) {
    using F128 = typename TestFixture::T128;
    F128 v{1.0f, 2.0f, 3.0f, 4.0f};

    F128 m1 = frombits<F128>(0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 m2 = frombits<F128>(0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u);
    F128 m3 = frombits<F128>(0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u);
    F128 m4 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu);

    F128 vm1{1.0f, 0.0f, 0.0f, 0.0f};
    F128 vm2{0.0f, 2.0f, 0.0f, 0.0f};
    F128 vm3{0.0f, 0.0f, 3.0f, 0.0f};
    F128 vm4{0.0f, 0.0f, 0.0f, 4.0f};

    ASSERT_TRUE(equal(v & m1, vm1));
    ASSERT_TRUE(equal(v & m2, vm2));
    ASSERT_TRUE(equal(v & m3, vm3));
    ASSERT_TRUE(equal(v & m4, vm4));
}

TYPED_TEST(T128Test, BitwiseOR) {
    using F128 = typename TestFixture::T128;
    F128 v1{0.0f, 2.0f, 0.0f, 4.0f};
    F128 v2{1.0f, 0.0f, 3.0f, 0.0f};

    F128 v12{1.0f, 2.0f, 3.0f, 4.0f};

    ASSERT_TRUE(equal(v1 | v2, v12));
}

TYPED_TEST(T128Test, BitwiseXOR) {
    using F128 = typename TestFixture::T128;
    F128 v1{0.0f, 2.0f, 0.0f, 4.0f};
    F128 v2{0.0f, 0.0f, 3.0f, 4.0f};

    F128 v12{0.0f, 2.0f, 3.0f, 0.0f};

    ASSERT_TRUE(equal(v1 ^ v2, v12));
}

TYPED_TEST(T128Test, BitwiseNOT) {
    using F128 = typename TestFixture::T128;
    F128 v1 = frombits<F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F128 v2 = frombits<F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(~v1, v2));
    ASSERT_TRUE(equal(~v2, v1));
}

TYPED_TEST(T128Test, ConstructFromArgs) {
    typename TestFixture::T128 v{1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 expected{1.0f, 2.0f, 3.0f, 4.0f};
    ASSERT_TRUE(equal(v, expected));
}

TYPED_TEST(T128Test, ConstructFromSingleValue) {
    typename TestFixture::T128 v{42.0f};
    typename TestFixture::T128 expected{42.0f, 42.0f, 42.0f, 42.0f};
    ASSERT_TRUE(equal(v, expected));
}

TYPED_TEST(T128Test, FromAlignedSource) {
    alignas(TestFixture::T128::alignment()) const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f};
    auto v = TestFixture::T128::fromAlignedSource(expected);

    alignas(TestFixture::T128::alignment()) float result[TestFixture::T128::size()];
    v.alignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T128::size());
}

TYPED_TEST(T128Test, AlignedLoadStore) {
    alignas(TestFixture::T128::alignment()) const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v;
    v.alignedLoad(expected);

    alignas(TestFixture::T128::alignment()) float result[TestFixture::T128::size()];
    v.alignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T128::size());
}

TYPED_TEST(T128Test, FromUnalignedSource) {
    const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f};
    auto v = TestFixture::T128::fromUnalignedSource(expected);

    float result[TestFixture::T128::size()];
    v.unalignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T128::size());
}

TYPED_TEST(T128Test, UnalignedLoadStore) {
    const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v;
    v.unalignedLoad(expected);

    float result[TestFixture::T128::size()];
    v.unalignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T128::size());
}

TYPED_TEST(T128Test, LoadSingleValue) {
    const float source[] = {42.0f, 43.0f, 44.0f, 45.0f};
    auto v = TestFixture::T128::loadSingleValue(source);
    typename TestFixture::T128 expected{42.0f, 0.0f, 0.0f, 0.0f};
    ASSERT_TRUE(equal(v, expected));
}

TYPED_TEST(T128Test, Sum) {
    typename TestFixture::T128 v{1.0f, 2.0f, 3.0f, 4.0f};
    ASSERT_EQ(v.sum(), 10.0f);
}

TYPED_TEST(T128Test, CompoundAssignmentAdd) {
    typename TestFixture::T128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v2{3.0f, 4.0f, 5.0f, 6.0f};
    typename TestFixture::T128 expected{4.0f, 6.0f, 8.0f, 10.0f};
    v1 += v2;
    ASSERT_TRUE(equal(v1, expected));
}

TYPED_TEST(T128Test, CompoundAssignmentSub) {
    typename TestFixture::T128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v2{3.0f, 4.0f, 5.0f, 6.0f};
    typename TestFixture::T128 expected{-2.0f, -2.0f, -2.0f, -2.0f};
    v1 -= v2;
    ASSERT_TRUE(equal(v1, expected));
}

TYPED_TEST(T128Test, CompoundAssignmentMul) {
    typename TestFixture::T128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v2{3.0f, 4.0f, 5.0f, 6.0f};
    typename TestFixture::T128 expected{3.0f, 8.0f, 15.0f, 24.0f};
    v1 *= v2;
    ASSERT_TRUE(equal(v1, expected));
}

TYPED_TEST(T128Test, CompoundAssignmentDiv) {
    typename TestFixture::T128 v1{4.0f, 3.0f, 9.0f, 12.0f};
    typename TestFixture::T128 v2{1.0f, 2.0f, 3.0f, 3.0f};
    float expected[TestFixture::T128::size()] = {4.0f, 1.5f, 3.0f, 4.0f};
    v1 /= v2;

    float result[TestFixture::T128::size()];
    v1.unalignedStore(result);

    ASSERT_ELEMENTS_NEAR(result, expected, TestFixture::T128::size(), 0.0001f);
}

TYPED_TEST(T128Test, OperatorAdd) {
    typename TestFixture::T128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v2{3.0f, 4.0f, 5.0f, 6.0f};
    typename TestFixture::T128 expected{4.0f, 6.0f, 8.0f, 10.0f};
    auto v3 = v1 + v2;
    ASSERT_TRUE(equal(v3, expected));
}

TYPED_TEST(T128Test, OperatorSub) {
    typename TestFixture::T128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v2{3.0f, 4.0f, 5.0f, 6.0f};
    typename TestFixture::T128 expected{-2.0f, -2.0f, -2.0f, -2.0f};
    auto v3 = v1 - v2;
    ASSERT_TRUE(equal(v3, expected));
}

TYPED_TEST(T128Test, OperatorMul) {
    typename TestFixture::T128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    typename TestFixture::T128 v2{3.0f, 4.0f, 5.0f, 6.0f};
    typename TestFixture::T128 expected{3.0f, 8.0f, 15.0f, 24.0f};
    auto v3 = v1 * v2;
    ASSERT_TRUE(equal(v3, expected));
}

TYPED_TEST(T128Test, OperatorDiv) {
    typename TestFixture::T128 v1{4.0f, 3.0f, 9.0f, 12.0f};
    typename TestFixture::T128 v2{1.0f, 2.0f, 3.0f, 3.0f};
    float expected[TestFixture::T128::size()] = {4.0f, 1.5f, 3.0f, 4.0f};
    auto v3 = v1 / v2;

    float result[TestFixture::T128::size()];
    v3.unalignedStore(result);

    ASSERT_ELEMENTS_NEAR(result, expected, TestFixture::T128::size(), 0.0001f);
}

TEST(T128Test, TransposeSquareScalar) {
    trimd::scalar::F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::scalar::F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::scalar::F128 v3{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::scalar::F128 v4{1.0f, 2.0f, 3.0f, 4.0f};

    trimd::scalar::transpose(v1, v2, v3, v4);

    trimd::scalar::F128 e1{1.0f, 1.0f, 1.0f, 1.0f};
    trimd::scalar::F128 e2{2.0f, 2.0f, 2.0f, 2.0f};
    trimd::scalar::F128 e3{3.0f, 3.0f, 3.0f, 3.0f};
    trimd::scalar::F128 e4{4.0f, 4.0f, 4.0f, 4.0f};

    ASSERT_TRUE(equal(v1, e1));
    ASSERT_TRUE(equal(v2, e2));
    ASSERT_TRUE(equal(v3, e3));
    ASSERT_TRUE(equal(v4, e4));
}

TEST(T128Test, AbsScalar) {
    trimd::scalar::F128 v{-1.0f, 2.0f, -3.0f, 0.0f};
    v = trimd::scalar::abs(v);
    trimd::scalar::F128 e{1.0f, 2.0f, 3.0f, 0.0f};
    ASSERT_TRUE(equal(v, e));
}

TEST(T128Test, AndNotScalar) {
    trimd::scalar::F128 v{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::scalar::F128 mask1 = frombits<trimd::scalar::F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    trimd::scalar::F128 mask2 = frombits<trimd::scalar::F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    trimd::scalar::F128 result1 = trimd::scalar::andnot(mask1, v);
    trimd::scalar::F128 result2 = trimd::scalar::andnot(mask2, v);
    trimd::scalar::F128 e1{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::scalar::F128 e2{0.0f, 0.0f, 0.0f, 0.0f};
    ASSERT_TRUE(equal(result1, e1));
    ASSERT_TRUE(equal(result2, e2));
}

TEST(T128Test, RsqrtScalar) {
    trimd::scalar::F128 v{1.0f, 2.0f, 3.0f, 9.0f};
    v = trimd::scalar::rsqrt(v);
    trimd::scalar::F128 e{1.0f, 0.70710678f, 0.57735026f, 0.33333333f};
#ifdef TRIMD_ENABLE_FAST_INVERSE_SQRT
    static constexpr float threshold = 0.0004f;
#else
    static constexpr float threshold = 0.0002f;
#endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
    ASSERT_TRUE(near(v, e, threshold));
}

#ifdef TRIMD_ENABLE_SSE
TEST(T128Test, TransposeSquareSSE) {
    trimd::sse::F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::sse::F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::sse::F128 v3{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::sse::F128 v4{1.0f, 2.0f, 3.0f, 4.0f};

    trimd::sse::transpose(v1, v2, v3, v4);

    trimd::sse::F128 e1{1.0f, 1.0f, 1.0f, 1.0f};
    trimd::sse::F128 e2{2.0f, 2.0f, 2.0f, 2.0f};
    trimd::sse::F128 e3{3.0f, 3.0f, 3.0f, 3.0f};
    trimd::sse::F128 e4{4.0f, 4.0f, 4.0f, 4.0f};

    ASSERT_TRUE(equal(v1, e1));
    ASSERT_TRUE(equal(v2, e2));
    ASSERT_TRUE(equal(v3, e3));
    ASSERT_TRUE(equal(v4, e4));
}

TEST(T128Test, AbsSSE) {
    trimd::sse::F128 v{-1.0f, 2.0f, -3.0f, 0.0f};
    v = trimd::sse::abs(v);
    trimd::sse::F128 e{1.0f, 2.0f, 3.0f, 0.0f};
    ASSERT_TRUE(equal(v, e));
}

TEST(T128Test, AndNotSSE) {
    trimd::sse::F128 v{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::sse::F128 mask1 = frombits<trimd::sse::F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    trimd::sse::F128 mask2 = frombits<trimd::sse::F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    trimd::sse::F128 result1 = trimd::sse::andnot(mask1, v);
    trimd::sse::F128 result2 = trimd::sse::andnot(mask2, v);
    trimd::sse::F128 e1{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::sse::F128 e2{0.0f, 0.0f, 0.0f, 0.0f};
    ASSERT_TRUE(equal(result1, e1));
    ASSERT_TRUE(equal(result2, e2));
}

TEST(T128Test, RsqrtSSE) {
    trimd::sse::F128 v{1.0f, 2.0f, 3.0f, 9.0f};
    v = trimd::sse::rsqrt(v);
    trimd::sse::F128 e{1.0f, 0.70710678f, 0.57735026f, 0.33333333f};
    #ifdef TRIMD_ENABLE_FAST_INVERSE_SQRT
    static constexpr float threshold = 0.0004f;
    #else
    static constexpr float threshold = 0.0003f;
    #endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
    ASSERT_TRUE(near(v, e, threshold));
}

    #ifdef TRIMD_ENABLE_F16C
TEST(T128Test, LoadAlignedHalfFloatsSSE) {
    alignas(trimd::sse::F128::alignment()) const std::uint16_t halfFloats[] = {15360, 16384, 16896, 17408};
    trimd::sse::F128 expected{1.0f, 2.0f, 3.0f, 4.0f};

    auto v = trimd::sse::F128::fromAlignedSource(halfFloats);
    ASSERT_TRUE(equal(v, expected));

    trimd::sse::F128 v2;
    v2.alignedLoad(halfFloats);
    ASSERT_TRUE(equal(v2, expected));
}

TEST(T128Test, LoadUnalignedHalfFloatsSSE) {
    const std::uint16_t halfFloats[] = {15360, 16384, 16896, 17408};
    trimd::sse::F128 expected{1.0f, 2.0f, 3.0f, 4.0f};

    auto v = trimd::sse::F128::fromUnalignedSource(halfFloats);
    ASSERT_TRUE(equal(v, expected));

    trimd::sse::F128 v2;
    v2.unalignedLoad(halfFloats);
    ASSERT_TRUE(equal(v2, expected));
}

TEST(T128Test, StoreAlignedHalfFloatsSSE) {
    const std::uint16_t expected[] = {15360, 16384, 16896, 17408};

    alignas(trimd::sse::F128::alignment()) std::uint16_t halfFloats[4ul] = {};
    const trimd::sse::F128 v{1.0f, 2.0f, 3.0f, 4.0f};
    v.alignedStore(halfFloats);

    ASSERT_ELEMENTS_EQ(halfFloats, expected, 4ul);
}

TEST(T128Test, StoreUnalignedHalfFloatsSSE) {
    const std::uint16_t expected[] = {15360, 16384, 16896, 17408};

    std::uint16_t halfFloats[4ul] = {};
    const trimd::sse::F128 v{1.0f, 2.0f, 3.0f, 4.0f};
    v.unalignedStore(halfFloats);

    ASSERT_ELEMENTS_EQ(halfFloats, expected, 4ul);
}
    #endif  // TRIMD_ENABLE_F16C
#endif      // TRIMD_ENABLE_SSE

#ifdef TRIMD_ENABLE_NEON
TEST(T128Test, TransposeSquareNEON) {
    trimd::neon::F128 v1{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::neon::F128 v2{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::neon::F128 v3{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::neon::F128 v4{1.0f, 2.0f, 3.0f, 4.0f};

    trimd::neon::transpose(v1, v2, v3, v4);

    trimd::neon::F128 e1{1.0f, 1.0f, 1.0f, 1.0f};
    trimd::neon::F128 e2{2.0f, 2.0f, 2.0f, 2.0f};
    trimd::neon::F128 e3{3.0f, 3.0f, 3.0f, 3.0f};
    trimd::neon::F128 e4{4.0f, 4.0f, 4.0f, 4.0f};

    ASSERT_TRUE(equal(v1, e1));
    ASSERT_TRUE(equal(v2, e2));
    ASSERT_TRUE(equal(v3, e3));
    ASSERT_TRUE(equal(v4, e4));
}

TEST(T128Test, AbsNEON) {
    trimd::neon::F128 v{-1.0f, 2.0f, -3.0f, 0.0f};
    v = trimd::neon::abs(v);
    trimd::neon::F128 e{1.0f, 2.0f, 3.0f, 0.0f};
    ASSERT_TRUE(equal(v, e));
}

TEST(T128Test, AndNotNEON) {
    trimd::neon::F128 v{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::neon::F128 mask1 = frombits<trimd::neon::F128>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    trimd::neon::F128 mask2 = frombits<trimd::neon::F128>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    trimd::neon::F128 result1 = trimd::neon::andnot(mask1, v);
    trimd::neon::F128 result2 = trimd::neon::andnot(mask2, v);
    trimd::neon::F128 e1{1.0f, 2.0f, 3.0f, 4.0f};
    trimd::neon::F128 e2{0.0f, 0.0f, 0.0f, 0.0f};
    ASSERT_TRUE(equal(result1, e1));
    ASSERT_TRUE(equal(result2, e2));
}

TEST(T128Test, RsqrtNEON) {
    trimd::neon::F128 v{1.0f, 2.0f, 3.0f, 9.0f};
    v = trimd::neon::rsqrt(v);
    trimd::neon::F128 e{1.0f, 0.70710678f, 0.57735026f, 0.33333333f};
    #ifdef TRIMD_ENABLE_FAST_INVERSE_SQRT
    static constexpr float threshold = 0.0004f;
    #else
    static constexpr float threshold = 0.0002f;
    #endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
    ASSERT_TRUE(near(v, e, threshold));
}

    #ifdef TRIMD_ENABLE_NEON_FP16
TEST(T128Test, LoadAlignedHalfFloatsNEON) {
    alignas(trimd::neon::F128::alignment()) const std::uint16_t halfFloats[] = {15360, 16384, 16896, 17408};
    trimd::neon::F128 expected{1.0f, 2.0f, 3.0f, 4.0f};

    auto v = trimd::neon::F128::fromAlignedSource(halfFloats);
    ASSERT_TRUE(equal(v, expected));

    trimd::neon::F128 v2;
    v2.alignedLoad(halfFloats);
    ASSERT_TRUE(equal(v2, expected));
}

TEST(T128Test, LoadUnalignedHalfFloatsNEON) {
    const std::uint16_t halfFloats[] = {15360, 16384, 16896, 17408};
    trimd::neon::F128 expected{1.0f, 2.0f, 3.0f, 4.0f};

    auto v = trimd::neon::F128::fromUnalignedSource(halfFloats);
    ASSERT_TRUE(equal(v, expected));

    trimd::neon::F128 v2;
    v2.unalignedLoad(halfFloats);
    ASSERT_TRUE(equal(v2, expected));
}

TEST(T128Test, StoreAlignedHalfFloatsNEON) {
    const std::uint16_t expected[] = {15360, 16384, 16896, 17408};

    alignas(trimd::neon::F128::alignment()) std::uint16_t halfFloats[4ul] = {};
    const trimd::neon::F128 v{1.0f, 2.0f, 3.0f, 4.0f};
    v.alignedStore(halfFloats);

    ASSERT_ELEMENTS_EQ(halfFloats, expected, 4ul);
}

TEST(T128Test, StoreUnalignedHalfFloatsNEON) {
    const std::uint16_t expected[] = {15360, 16384, 16896, 17408};

    std::uint16_t halfFloats[4ul] = {};
    const trimd::neon::F128 v{1.0f, 2.0f, 3.0f, 4.0f};
    v.unalignedStore(halfFloats);

    ASSERT_ELEMENTS_EQ(halfFloats, expected, 4ul);
}
    #endif  // TRIMD_ENABLE_NEON_FP16
#endif      // TRIMD_ENABLE_NEON
