// Copyright Epic Games, Inc. All Rights Reserved.

#include "trimdtests/Defs.h"

#include "trimd/TRiMD.h"

#if defined(TRIMD_ENABLE_AVX) && defined(TRIMD_ENABLE_SSE)
using T256Types = ::testing::Types<trimd::scalar::F256, trimd::sse::F256, trimd::avx::F256>;

#elif defined(TRIMD_ENABLE_AVX)
using T256Types = ::testing::Types<trimd::scalar::F256, trimd::avx::F256>;

#elif defined(TRIMD_ENABLE_SSE)
using T256Types = ::testing::Types<trimd::scalar::F256, trimd::sse::F256>;

#else
using T256Types = ::testing::Types<trimd::scalar::F256>;
#endif  // TRIMD_ENABLE_SSE

template<typename TF256>
static TF256 frombits(uint32_t bits0,
                      uint32_t bits1,
                      uint32_t bits2,
                      uint32_t bits3,
                      uint32_t bits4,
                      uint32_t bits5,
                      uint32_t bits6,
                      uint32_t bits7) {
    return TF256{
        trimd::bitcast<float>(bits0),
        trimd::bitcast<float>(bits1),
        trimd::bitcast<float>(bits2),
        trimd::bitcast<float>(bits3),
        trimd::bitcast<float>(bits4),
        trimd::bitcast<float>(bits5),
        trimd::bitcast<float>(bits6),
        trimd::bitcast<float>(bits7),
    };
}

#ifdef TRIMD_ENABLE_AVX
bool equal(const trimd::avx::F256& lhs, const trimd::avx::F256& rhs) {
    return (std::memcmp(&lhs.data, &rhs.data, sizeof(lhs.data)) == 0);
}

#endif  // TRIMD_ENABLE_AVX

#ifdef TRIMD_ENABLE_SSE
static bool equal(const trimd::fallback::T256<trimd::sse::F128>& lhs, const trimd::fallback::T256<trimd::sse::F128>& rhs) {
    const bool data1eq = (std::memcmp(&lhs.data1.data, &rhs.data1.data, sizeof(float) * decltype(lhs.data1)::size()) == 0);
    const bool data2eq = (std::memcmp(&lhs.data2.data, &rhs.data2.data, sizeof(float) * decltype(lhs.data2)::size()) == 0);
    return data1eq && data2eq;
}

#endif  // TRIMD_ENABLE_SSE

static bool equal(const trimd::fallback::T256<trimd::scalar::F128>& lhs, const trimd::fallback::T256<trimd::scalar::F128>& rhs) {
    const bool data1eq =
        (std::memcmp(lhs.data1.data.data(), rhs.data1.data.data(), sizeof(float) * decltype(lhs.data1)::size()) == 0);
    const bool data2eq =
        (std::memcmp(lhs.data2.data.data(), rhs.data2.data.data(), sizeof(float) * decltype(lhs.data2)::size()) == 0);
    return data1eq && data2eq;
}

template<typename T>
class T256Test : public ::testing::Test {
protected:
    using T256 = T;
};

TYPED_TEST_SUITE(T256Test, T256Types, );

TYPED_TEST(T256Test, CheckSize) {
    ASSERT_EQ(TestFixture::T256::size(), 8ul);
}

TYPED_TEST(T256Test, Equality) {
    using F256 = typename TestFixture::T256;

    F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v3{1.5f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v4{1.0f, 2.5f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v5{1.0f, 2.0f, 3.5f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v6{1.0f, 2.0f, 3.0f, 4.5f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v7{1.0f, 2.0f, 3.0f, 4.0f, 5.5f, 6.0f, 7.0f, 8.0f};
    F256 v8{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.5f, 7.0f, 8.0f};
    F256 v9{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.5f, 8.0f};
    F256 v10{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.5f};

    F256 m12 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m13 =
        frombits<F256>(0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m14 =
        frombits<F256>(0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m15 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m16 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m17 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m18 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m19 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u, 0xFFFFFFFFu);
    F256 m110 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0x00000000u);

    ASSERT_TRUE(equal(v1 == v2, m12));
    ASSERT_TRUE(equal(v1 == v3, m13));
    ASSERT_TRUE(equal(v1 == v4, m14));
    ASSERT_TRUE(equal(v1 == v5, m15));
    ASSERT_TRUE(equal(v1 == v6, m16));
    ASSERT_TRUE(equal(v1 == v7, m17));
    ASSERT_TRUE(equal(v1 == v8, m18));
    ASSERT_TRUE(equal(v1 == v9, m19));
    ASSERT_TRUE(equal(v1 == v10, m110));
}

TYPED_TEST(T256Test, Inequality) {
    using F256 = typename TestFixture::T256;

    F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v3{1.5f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v4{1.0f, 2.5f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v5{1.0f, 2.0f, 3.5f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v6{1.0f, 2.0f, 3.0f, 4.5f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v7{1.0f, 2.0f, 3.0f, 4.0f, 5.5f, 6.0f, 7.0f, 8.0f};
    F256 v8{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.5f, 7.0f, 8.0f};
    F256 v9{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.5f, 8.0f};
    F256 v10{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.5f};

    F256 m12 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m13 =
        frombits<F256>(0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m14 =
        frombits<F256>(0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m15 =
        frombits<F256>(0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m16 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m17 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m18 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u);
    F256 m19 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u);
    F256 m110 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(v1 != v2, m12));
    ASSERT_TRUE(equal(v1 != v3, m13));
    ASSERT_TRUE(equal(v1 != v4, m14));
    ASSERT_TRUE(equal(v1 != v5, m15));
    ASSERT_TRUE(equal(v1 != v6, m16));
    ASSERT_TRUE(equal(v1 != v7, m17));
    ASSERT_TRUE(equal(v1 != v8, m18));
    ASSERT_TRUE(equal(v1 != v9, m19));
    ASSERT_TRUE(equal(v1 != v10, m110));
}

TYPED_TEST(T256Test, LessThan) {
    using F256 = typename TestFixture::T256;

    F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v3{2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f};
    F256 v4{0.5f, 1.5f, 2.5f, 3.5f, 4.5f, 5.5f, 6.5f, 7.5f};

    F256 m12 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m13 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m14 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);

    ASSERT_TRUE(equal(v1 < v2, m12));
    ASSERT_TRUE(equal(v1 < v3, m13));
    ASSERT_TRUE(equal(v1 < v4, m14));
}

TYPED_TEST(T256Test, LessThanOrEqual) {
    using F256 = typename TestFixture::T256;

    F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v3{2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f};
    F256 v4{0.5f, 1.5f, 2.5f, 3.5f, 4.5f, 5.5f, 6.5f, 7.5f};

    F256 m12 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m13 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m14 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);

    ASSERT_TRUE(equal(v1 <= v2, m12));
    ASSERT_TRUE(equal(v1 <= v3, m13));
    ASSERT_TRUE(equal(v1 <= v4, m14));
}

TYPED_TEST(T256Test, GreaterThan) {
    using F256 = typename TestFixture::T256;

    F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v3{2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f};
    F256 v4{0.5f, 1.5f, 2.5f, 3.5f, 4.5f, 5.5f, 6.5f, 7.5f};

    F256 m12 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m13 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m14 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(v1 > v2, m12));
    ASSERT_TRUE(equal(v1 > v3, m13));
    ASSERT_TRUE(equal(v1 > v4, m14));
}

TYPED_TEST(T256Test, GreaterThanOrEqual) {
    using F256 = typename TestFixture::T256;

    F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    F256 v3{2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f};
    F256 v4{0.5f, 1.5f, 2.5f, 3.5f, 4.5f, 5.5f, 6.5f, 7.5f};

    F256 m12 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);
    F256 m13 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m14 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(v1 >= v2, m12));
    ASSERT_TRUE(equal(v1 >= v3, m13));
    ASSERT_TRUE(equal(v1 >= v4, m14));
}

TYPED_TEST(T256Test, BitwiseAND) {
    using F256 = typename TestFixture::T256;
    F256 v{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};

    F256 m1 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m2 =
        frombits<F256>(0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m3 =
        frombits<F256>(0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m4 =
        frombits<F256>(0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m5 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m6 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 m7 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u, 0x00000000u);
    F256 m8 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu, 0x00000000u);
    F256 m9 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0xFFFFFFFFu);

    F256 vm1{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    F256 vm2{1.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    F256 vm3{0.0f, 2.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    F256 vm4{0.0f, 0.0f, 3.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    F256 vm5{0.0f, 0.0f, 0.0f, 4.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    F256 vm6{0.0f, 0.0f, 0.0f, 0.0f, 5.0f, 0.0f, 0.0f, 0.0f};
    F256 vm7{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 6.0f, 0.0f, 0.0f};
    F256 vm8{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 7.0f, 0.0f};
    F256 vm9{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 8.0f};

    ASSERT_TRUE(equal(v & m1, vm1));
    ASSERT_TRUE(equal(v & m2, vm2));
    ASSERT_TRUE(equal(v & m3, vm3));
    ASSERT_TRUE(equal(v & m4, vm4));
    ASSERT_TRUE(equal(v & m5, vm5));
    ASSERT_TRUE(equal(v & m6, vm6));
    ASSERT_TRUE(equal(v & m7, vm7));
    ASSERT_TRUE(equal(v & m8, vm8));
    ASSERT_TRUE(equal(v & m9, vm9));
}

TYPED_TEST(T256Test, BitwiseOR) {
    using F256 = typename TestFixture::T256;
    F256 v1{0.0f, 2.0f, 0.0f, 4.0f, 0.0f, 6.0f, 0.0f, 8.0f};
    F256 v2{1.0f, 0.0f, 3.0f, 0.0f, 5.0f, 0.0f, 7.0f, 0.0f};

    F256 v12{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};

    ASSERT_TRUE(equal(v1 | v2, v12));
}

TYPED_TEST(T256Test, BitwiseXOR) {
    using F256 = typename TestFixture::T256;
    F256 v1{0.0f, 2.0f, 0.0f, 4.0f, 0.0f, 6.0f, 0.0f, 8.0f};
    F256 v2{0.0f, 0.0f, 3.0f, 0.0f, 5.0f, 0.0f, 7.0f, 8.0f};

    F256 v12{0.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 0.0f};

    ASSERT_TRUE(equal(v1 ^ v2, v12));
}

TYPED_TEST(T256Test, BitwiseNOT) {
    using F256 = typename TestFixture::T256;
    F256 v1 =
        frombits<F256>(0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u, 0x00000000u);
    F256 v2 =
        frombits<F256>(0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu, 0xFFFFFFFFu);

    ASSERT_TRUE(equal(~v1, v2));
    ASSERT_TRUE(equal(~v2, v1));
}

TYPED_TEST(T256Test, ConstructFromArgs) {
    typename TestFixture::T256 v{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 expected{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    ASSERT_TRUE(equal(v, expected));
}

TYPED_TEST(T256Test, ConstructFromSingleValue) {
    typename TestFixture::T256 v{42.0f};
    typename TestFixture::T256 expected{42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f, 42.0f};
    ASSERT_TRUE(equal(v, expected));
}

TYPED_TEST(T256Test, FromAlignedSource) {
    alignas(TestFixture::T256::alignment()) const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    auto v = TestFixture::T256::fromAlignedSource(expected);

    alignas(TestFixture::T256::alignment()) float result[TestFixture::T256::size()];
    v.alignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T256::size());
}

TYPED_TEST(T256Test, AlignedLoadStore) {
    alignas(TestFixture::T256::alignment()) const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v;
    v.alignedLoad(expected);

    alignas(TestFixture::T256::alignment()) float result[TestFixture::T256::size()];
    v.alignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T256::size());
}

TYPED_TEST(T256Test, FromUnalignedSource) {
    const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    auto v = TestFixture::T256::fromUnalignedSource(expected);

    float result[TestFixture::T256::size()];
    v.unalignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T256::size());
}

TYPED_TEST(T256Test, UnalignedLoadStore) {
    const float expected[] = {1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v;
    v.unalignedLoad(expected);

    float result[TestFixture::T256::size()];
    v.unalignedStore(result);

    ASSERT_ELEMENTS_EQ(result, expected, TestFixture::T256::size());
}

TYPED_TEST(T256Test, LoadSingleValue) {
    const float source[] = {42.0f, 43.0f, 44.0f, 45.0f, 46.0f, 47.0f, 48.0f, 49.0f};
    auto v = TestFixture::T256::loadSingleValue(source);
    typename TestFixture::T256 expected{42.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    ASSERT_TRUE(equal(v, expected));
}

TYPED_TEST(T256Test, Sum) {
    typename TestFixture::T256 v{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    ASSERT_EQ(v.sum(), 36.0f);
}

TYPED_TEST(T256Test, CompoundAssignmentAdd) {
    typename TestFixture::T256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v2{3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f};
    typename TestFixture::T256 expected{4.0f, 6.0f, 8.0f, 10.0f, 12.0f, 14.0f, 16.0f, 18.0f};
    v1 += v2;
    ASSERT_TRUE(equal(v1, expected));
}

TYPED_TEST(T256Test, CompoundAssignmentSub) {
    typename TestFixture::T256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v2{3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f};
    typename TestFixture::T256 expected{-2.0f, -2.0f, -2.0f, -2.0f, -2.0f, -2.0f, -2.0f, -2.0f};
    v1 -= v2;
    ASSERT_TRUE(equal(v1, expected));
}

TYPED_TEST(T256Test, CompoundAssignmentMul) {
    typename TestFixture::T256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v2{3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f};
    typename TestFixture::T256 expected{3.0f, 8.0f, 15.0f, 24.0f, 35.0f, 48.0f, 63.0f, 80.0f};
    v1 *= v2;
    ASSERT_TRUE(equal(v1, expected));
}

TYPED_TEST(T256Test, CompoundAssignmentDiv) {
    typename TestFixture::T256 v1{4.0f, 3.0f, 9.0f, 12.0f, 4.0f, 3.0f, 9.0f, 12.0f};
    typename TestFixture::T256 v2{1.0f, 2.0f, 3.0f, 3.0f, 1.0f, 2.0f, 3.0f, 3.0f};
    float expected[TestFixture::T256::size()] = {4.0f, 1.5f, 3.0f, 4.0f, 4.0f, 1.5f, 3.0f, 4.0f};
    v1 /= v2;

    float result[TestFixture::T256::size()];
    v1.unalignedStore(result);

    ASSERT_ELEMENTS_NEAR(result, expected, TestFixture::T256::size(), 0.0001f);
}

TYPED_TEST(T256Test, OperatorAdd) {
    typename TestFixture::T256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v2{3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f};
    typename TestFixture::T256 expected{4.0f, 6.0f, 8.0f, 10.0f, 12.0f, 14.0f, 16.0f, 18.0f};
    auto v3 = v1 + v2;
    ASSERT_TRUE(equal(v3, expected));
}

TYPED_TEST(T256Test, OperatorSub) {
    typename TestFixture::T256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v2{3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f};
    typename TestFixture::T256 expected{-2.0f, -2.0f, -2.0f, -2.0f, -2.0f, -2.0f, -2.0f, -2.0f};
    auto v3 = v1 - v2;
    ASSERT_TRUE(equal(v3, expected));
}

TYPED_TEST(T256Test, OperatorMul) {
    typename TestFixture::T256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    typename TestFixture::T256 v2{3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f, 9.0f, 10.0f};
    typename TestFixture::T256 expected{3.0f, 8.0f, 15.0f, 24.0f, 35.0f, 48.0f, 63.0f, 80.0f};
    auto v3 = v1 * v2;
    ASSERT_TRUE(equal(v3, expected));
}

TYPED_TEST(T256Test, OperatorDiv) {
    typename TestFixture::T256 v1{4.0f, 3.0f, 9.0f, 12.0f, 4.0f, 3.0f, 9.0f, 12.0f};
    typename TestFixture::T256 v2{1.0f, 2.0f, 3.0f, 3.0f, 1.0f, 2.0f, 3.0f, 3.0f};
    float expected[TestFixture::T256::size()] = {4.0f, 1.5f, 3.0f, 4.0f, 4.0f, 1.5f, 3.0f, 4.0f};
    auto v3 = v1 / v2;

    float result[TestFixture::T256::size()];
    v3.unalignedStore(result);

    ASSERT_ELEMENTS_NEAR(result, expected, TestFixture::T256::size(), 0.0001f);
}

TEST(T256Test, TransposeSquareScalar) {
    trimd::scalar::F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 v3{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 v4{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 v5{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 v6{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 v7{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 v8{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};

    trimd::scalar::transpose(v1, v2, v3, v4, v5, v6, v7, v8);

    trimd::scalar::F256 e1{1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
    trimd::scalar::F256 e2{2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f};
    trimd::scalar::F256 e3{3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f};
    trimd::scalar::F256 e4{4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f};
    trimd::scalar::F256 e5{5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f};
    trimd::scalar::F256 e6{6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f};
    trimd::scalar::F256 e7{7.0f, 7.0f, 7.0f, 7.0f, 7.0f, 7.0f, 7.0f, 7.0f};
    trimd::scalar::F256 e8{8.0f, 8.0f, 8.0f, 8.0f, 8.0f, 8.0f, 8.0f, 8.0f};

    ASSERT_TRUE(equal(v1, e1));
    ASSERT_TRUE(equal(v2, e2));
    ASSERT_TRUE(equal(v3, e3));
    ASSERT_TRUE(equal(v4, e4));
    ASSERT_TRUE(equal(v5, e5));
    ASSERT_TRUE(equal(v6, e6));
    ASSERT_TRUE(equal(v7, e7));
    ASSERT_TRUE(equal(v8, e8));
}

TEST(T256Test, AbsScalar) {
    trimd::scalar::F256 v{-1.0f, 2.0f, -3.0f, 0.0f, -1.0f, 2.0f, -3.0f, 0.0f};
    v = trimd::scalar::abs(v);
    trimd::scalar::F256 e{1.0f, 2.0f, 3.0f, 0.0f, 1.0f, 2.0f, 3.0f, 0.0f};
    ASSERT_TRUE(equal(v, e));
}

TEST(T256Test, AndNotScalar) {
    trimd::scalar::F256 v{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 mask1 = frombits<trimd::scalar::F256>(0x00000000u,
                                                              0x00000000u,
                                                              0x00000000u,
                                                              0x00000000u,
                                                              0x00000000u,
                                                              0x00000000u,
                                                              0x00000000u,
                                                              0x00000000u);
    trimd::scalar::F256 mask2 = frombits<trimd::scalar::F256>(0xFFFFFFFFu,
                                                              0xFFFFFFFFu,
                                                              0xFFFFFFFFu,
                                                              0xFFFFFFFFu,
                                                              0xFFFFFFFFu,
                                                              0xFFFFFFFFu,
                                                              0xFFFFFFFFu,
                                                              0xFFFFFFFFu);
    trimd::scalar::F256 result1 = trimd::scalar::andnot(mask1, v);
    trimd::scalar::F256 result2 = trimd::scalar::andnot(mask2, v);
    trimd::scalar::F256 e1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::scalar::F256 e2{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    ASSERT_TRUE(equal(result1, e1));
    ASSERT_TRUE(equal(result2, e2));
}

TEST(T256Test, RsqrtScalar) {
    trimd::scalar::F256 v{1.0f, 2.0f, 3.0f, 9.0f, 9.0f, 3.0f, 2.0f, 1.0};
    v = trimd::scalar::rsqrt(v);
    trimd::scalar::F256 e{1.0f, 0.70710678f, 0.57735026f, 0.33333333f, 0.33333333f, 0.57735026f, 0.70710678f, 1.0f};
#ifdef TRIMD_ENABLE_FAST_INVERSE_SQRT
    static constexpr float threshold = 0.0004f;
#else
    static constexpr float threshold = 0.0002f;
#endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
    ASSERT_TRUE(near(v, e, threshold));
}

#ifdef TRIMD_ENABLE_AVX
TEST(T256Test, TransposeSquareAVX) {
    trimd::avx::F256 v1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 v2{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 v3{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 v4{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 v5{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 v6{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 v7{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 v8{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};

    trimd::avx::transpose(v1, v2, v3, v4, v5, v6, v7, v8);

    trimd::avx::F256 e1{1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f};
    trimd::avx::F256 e2{2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f, 2.0f};
    trimd::avx::F256 e3{3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f, 3.0f};
    trimd::avx::F256 e4{4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f, 4.0f};
    trimd::avx::F256 e5{5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f, 5.0f};
    trimd::avx::F256 e6{6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f, 6.0f};
    trimd::avx::F256 e7{7.0f, 7.0f, 7.0f, 7.0f, 7.0f, 7.0f, 7.0f, 7.0f};
    trimd::avx::F256 e8{8.0f, 8.0f, 8.0f, 8.0f, 8.0f, 8.0f, 8.0f, 8.0f};

    ASSERT_TRUE(equal(v1, e1));
    ASSERT_TRUE(equal(v2, e2));
    ASSERT_TRUE(equal(v3, e3));
    ASSERT_TRUE(equal(v4, e4));
    ASSERT_TRUE(equal(v5, e5));
    ASSERT_TRUE(equal(v6, e6));
    ASSERT_TRUE(equal(v7, e7));
    ASSERT_TRUE(equal(v8, e8));
}

TEST(T256Test, AbsAVX) {
    trimd::avx::F256 v{-1.0f, 2.0f, -3.0f, 0.0f, -1.0f, 2.0f, -3.0f, 0.0f};
    v = trimd::avx::abs(v);
    trimd::avx::F256 e{1.0f, 2.0f, 3.0f, 0.0f, 1.0f, 2.0f, 3.0f, 0.0f};
    ASSERT_TRUE(equal(v, e));
}

TEST(T256Test, AndNotAVX) {
    trimd::avx::F256 v{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 mask1 = frombits<trimd::avx::F256>(0x00000000u,
                                                        0x00000000u,
                                                        0x00000000u,
                                                        0x00000000u,
                                                        0x00000000u,
                                                        0x00000000u,
                                                        0x00000000u,
                                                        0x00000000u);
    trimd::avx::F256 mask2 = frombits<trimd::avx::F256>(0xFFFFFFFFu,
                                                        0xFFFFFFFFu,
                                                        0xFFFFFFFFu,
                                                        0xFFFFFFFFu,
                                                        0xFFFFFFFFu,
                                                        0xFFFFFFFFu,
                                                        0xFFFFFFFFu,
                                                        0xFFFFFFFFu);
    trimd::avx::F256 result1 = trimd::avx::andnot(mask1, v);
    trimd::avx::F256 result2 = trimd::avx::andnot(mask2, v);
    trimd::avx::F256 e1{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    trimd::avx::F256 e2{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f};
    ASSERT_TRUE(equal(result1, e1));
    ASSERT_TRUE(equal(result2, e2));
}

TEST(T256Test, RsqrtAVX) {
    trimd::avx::F256 v{1.0f, 2.0f, 3.0f, 9.0f, 9.0f, 3.0f, 2.0f, 1.0};
    v = trimd::avx::rsqrt(v);
    trimd::avx::F256 e{1.0f, 0.70710678f, 0.57735026f, 0.33333333f, 0.33333333f, 0.57735026f, 0.70710678f, 1.0f};
    #ifdef TRIMD_ENABLE_FAST_INVERSE_SQRT
    static constexpr float threshold = 0.0004f;
    #else
    static constexpr float threshold = 0.0003f;
    #endif  // TRIMD_ENABLE_FAST_INVERSE_SQRT
    ASSERT_TRUE(near(v, e, threshold));
}
#endif  // TRIMD_ENABLE_AVX

#ifdef TRIMD_ENABLE_F16C

    #if defined(TRIMD_ENABLE_AVX) && defined(TRIMD_ENABLE_SSE)
using T256HFTypes = ::testing::Types<trimd::sse::F256, trimd::avx::F256>;

    #elif defined(TRIMD_ENABLE_AVX)
using T256HFTypes = ::testing::Types<trimd::avx::F256>;

    #elif defined(TRIMD_ENABLE_SSE)
using T256HFTypes = ::testing::Types<trimd::sse::F256>;

    #else
using T256HFTypes = ::testing::Types<>;
    #endif  // TRIMD_ENABLE_SSE

template<typename T>
class T256HFTest : public ::testing::Test {
protected:
    using T256 = T;
};

TYPED_TEST_SUITE(T256HFTest, T256HFTypes, );

TYPED_TEST(T256HFTest, LoadAlignedHalfFloats) {
    using F256 = typename TestFixture::T256;
    alignas(F256::alignment()) const std::uint16_t halfFloats[] = {15360, 16384, 16896, 17408, 17664, 17920, 18176, 18432};
    F256 expected{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};

    auto v = F256::fromAlignedSource(halfFloats);
    ASSERT_TRUE(equal(v, expected));

    F256 v2;
    v2.alignedLoad(halfFloats);
    ASSERT_TRUE(equal(v2, expected));
}

TYPED_TEST(T256HFTest, LoadUnalignedHalfFloats) {
    using F256 = typename TestFixture::T256;
    const std::uint16_t halfFloats[] = {15360, 16384, 16896, 17408, 17664, 17920, 18176, 18432};
    F256 expected{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};

    auto v = F256::fromUnalignedSource(halfFloats);
    ASSERT_TRUE(equal(v, expected));

    F256 v2;
    v2.unalignedLoad(halfFloats);
    ASSERT_TRUE(equal(v2, expected));
}

TYPED_TEST(T256HFTest, StoreAlignedHalfFloats) {
    using F256 = typename TestFixture::T256;
    const std::uint16_t expected[] = {15360, 16384, 16896, 17408, 17664, 17920, 18176, 18432};

    alignas(F256::alignment()) std::uint16_t halfFloats[8ul] = {};
    const F256 v{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    v.alignedStore(halfFloats);

    ASSERT_ELEMENTS_EQ(halfFloats, expected, 8ul);
}

TYPED_TEST(T256HFTest, StoreUnalignedHalfFloats) {
    using F256 = typename TestFixture::T256;
    const std::uint16_t expected[] = {15360, 16384, 16896, 17408, 17664, 17920, 18176, 18432};

    std::uint16_t halfFloats[8ul] = {};
    const F256 v{1.0f, 2.0f, 3.0f, 4.0f, 5.0f, 6.0f, 7.0f, 8.0f};
    v.unalignedStore(halfFloats);

    ASSERT_ELEMENTS_EQ(halfFloats, expected, 8ul);
}

#endif  // TRIMD_ENABLE_F16C
