// Copyright Epic Games, Inc. All Rights Reserved.

#include "triotests/TestStreams.h"

TYPED_TEST(StreamTest, OpenExistingFileForRead) {
    TestFixture::CreateTestFile("test", 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, OpenExistingFileForWrite) {
    TestFixture::CreateTestFile("test", 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, OpenExistingFileForReadWrite) {
    TestFixture::CreateTestFile("test", 4u);

    auto stream =
        StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::ReadWrite);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, OpenNonExistingFileForRead) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_IS(TestFixture::TStream::OpenError);
}

TYPED_TEST(StreamTest, OpenNonExistingFileForWrite) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, OpenNonExistingFileForReadWrite) {
    auto stream =
        StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::ReadWrite);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, OpenEmptyFile) {
    TestFixture::CreateTestFile();

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, OpenAlreadyOpenFile) {
    TestFixture::CreateTestFile("test", 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_IS(TestFixture::TStream::AlreadyOpenError);
}

TYPED_TEST(StreamTest, ReadSuccess) {
    static const char fixture[4ul] = {'t', 'e', 's', 't'};
    TestFixture::CreateTestFile(fixture, 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    // Read all data
    char buffer[8ul] = {};
    ASSERT_EQ(stream->read(buffer, 4ul), 4ul);
    ASSERT_STATUS_OK();
    ASSERT_ELEMENTS_EQ(buffer, fixture, 4ul);

    // End of stream reached
    ASSERT_EQ(stream->read(buffer, 1ul), 0ul);
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, ReadMoreThanBytesAvailable) {
    static const char fixture[4ul] = {'t', 'e', 's', 't'};
    TestFixture::CreateTestFile(fixture, 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    // Read all data
    char buffer[8ul] = {};
    ASSERT_EQ(stream->read(buffer, 8ul), 4ul);
    ASSERT_STATUS_OK();
    ASSERT_ELEMENTS_EQ(buffer, fixture, 4ul);
}

TYPED_TEST(StreamTest, ReadFromEmptyFile) {
    TestFixture::CreateTestFile();

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    char buffer[2ul] = {};
    ASSERT_EQ(stream->read(buffer, 1ul), 0ul);
    ASSERT_STATUS_OK();
}

TYPED_TEST(StreamTest, ReadFromUnopenedStream) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    char buffer[2ul] = {};
    ASSERT_EQ(stream->read(buffer, 1ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::ReadError);
}

TYPED_TEST(StreamTest, ReadFromWriteOnlyStream) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    char buffer[2ul] = {};
    ASSERT_EQ(stream->read(buffer, 1ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::ReadError);
}

TYPED_TEST(StreamTest, ReadIntoNullBuffer) {
    TestFixture::CreateTestFile();

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    char* destination = nullptr;
    ASSERT_EQ(stream->read(destination, 1ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::ReadError);
}

TYPED_TEST(StreamTest, ReadIntoNullWritable) {
    TestFixture::CreateTestFile();

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    trio::Writable* destination = nullptr;
    ASSERT_EQ(stream->read(destination, 1ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::ReadError);
}

TYPED_TEST(StreamTest, SeekSuccess) {
    TestFixture::CreateTestFile("test", 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), 0ul);

    stream->seek(2ul);
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), 2ul);

    char buffer[2ul] = {};
    ASSERT_EQ(stream->read(buffer, 2ul), 2ul);
    ASSERT_EQ(stream->tell(), 4ul);

    const char expected[] = {'s', 't'};
    ASSERT_ELEMENTS_EQ(buffer, expected, 2ul);
}

TYPED_TEST(StreamTest, SeekToEOF) {
    TestFixture::CreateTestFile("test", 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), 0ul);

    stream->seek(4ul);
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), 4ul);
}

TYPED_TEST(StreamTest, SeekOutOfBounds) {
    TestFixture::CreateTestFile("test", 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), 0ul);

    stream->seek(5ul);
    ASSERT_STATUS_IS(TestFixture::TStream::SeekError);
    ASSERT_EQ(stream->tell(), 0ul);
}

TYPED_TEST(StreamTest, SeekUnopenedStream) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->seek(1ul);
    ASSERT_STATUS_IS(TestFixture::TStream::SeekError);
}

TYPED_TEST(StreamTest, WriteSuccess) {
    static const char fixture[] = {'t', 'e', 's', 't'};
    static const char expected[] = {'h', 'e', 'l', 'l', 'o'};
    TestFixture::CreateTestFile(fixture, 4u);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    // Write data
    ASSERT_EQ(stream->write(expected, 5ul), 5ul);
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->size(), 5ul);
    ASSERT_EQ(stream->tell(), 5ul);
    stream->close();
    ASSERT_STATUS_OK();

    // Verify data
    TestFixture::CompareTestFile(expected, 5u);
}

TYPED_TEST(StreamTest, WriteIntoEmptyFile) {
    static const char fixture[] = {'t', 'e', 's', 't'};
    TestFixture::CreateTestFile();

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    ASSERT_EQ(stream->write(fixture, 4ul), 4ul);
    ASSERT_STATUS_OK();

    stream->close();
    ASSERT_STATUS_OK();

    // Verify data
    TestFixture::CompareTestFile(fixture, 4u);
}

TYPED_TEST(StreamTest, WriteIntoUnopenedStream) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    ASSERT_EQ(stream->write("test", 4ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::WriteError);
}

TYPED_TEST(StreamTest, WriteIntoReadOnlyStream) {
    TestFixture::CreateTestFile("test", 4ul);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    ASSERT_EQ(stream->write("hello", 5ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::WriteError);
}

TYPED_TEST(StreamTest, WriteFromNullBuffer) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    const char* source = nullptr;
    ASSERT_EQ(stream->write(source, 1ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::WriteError);
}

TYPED_TEST(StreamTest, WriteFromNullReadable) {
    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    trio::Readable* source = nullptr;
    ASSERT_EQ(stream->write(source, 1ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::WriteError);
}

TYPED_TEST(StreamTest, ReadAfterClose) {
    TestFixture::CreateTestFile("test", 4ul);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    stream->close();
    ASSERT_STATUS_OK();

    char buffer[2ul] = {};
    ASSERT_EQ(stream->read(buffer, 1ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::ReadError);
}

TYPED_TEST(StreamTest, WriteAfterClose) {
    TestFixture::CreateTestFile("test", 4ul);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    stream->close();
    ASSERT_STATUS_OK();

    ASSERT_EQ(stream->write("hello", 5ul), 0ul);
    ASSERT_STATUS_IS(TestFixture::TStream::WriteError);
}

TYPED_TEST(StreamTest, SeekAfterClose) {
    TestFixture::CreateTestFile("test", 4ul);

    auto stream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    stream->close();
    ASSERT_STATUS_OK();

    stream->seek(2ul);
    ASSERT_STATUS_IS(TestFixture::TStream::SeekError);
}
