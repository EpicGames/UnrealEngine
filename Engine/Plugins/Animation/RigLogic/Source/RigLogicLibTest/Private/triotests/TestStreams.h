// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "triotests/Defs.h"

#include "status/Provider.h"
#include "trio/streams/FileStream.h"
#include "trio/streams/MemoryMappedFileStream.h"

#ifdef _MSC_VER
    #pragma warning(push)
    #pragma warning(disable : 4365 4987)
#endif
#include <cstdio>
#include <fstream>
#ifdef _MSC_VER
    #pragma warning(pop)
#endif

#define ASSERT_STATUS_OK()                                                                                                       \
    if (!trio::Status::isOk()) {                                                                                                 \
        FAIL() << trio::Status::get().message;                                                                                   \
    }

#define ASSERT_STATUS_IS(candidate) ASSERT_EQ(trio::Status::get(), candidate);

template<typename T>
struct StreamFactory;

template<>
struct StreamFactory<trio::FileStream> {
    using PointerType = pma::ScopedPtr<trio::FileStream, pma::FactoryDestroy<trio::FileStream>>;

    static PointerType create(const char* path, trio::AccessMode accessMode) {
        return pma::makeScoped<trio::FileStream>(path, accessMode, trio::OpenMode::Binary);
    }
};

template<>
struct StreamFactory<trio::MemoryMappedFileStream> {
    using PointerType = pma::ScopedPtr<trio::MemoryMappedFileStream, pma::FactoryDestroy<trio::MemoryMappedFileStream>>;

    static PointerType create(const char* path, trio::AccessMode accessMode) {
        return pma::makeScoped<trio::MemoryMappedFileStream>(path, accessMode);
    }
};

template<typename T>
class StreamTest : public ::testing::Test {
protected:
    using TStream = T;

protected:
    static const char* GetTestFileName() {
        return "TRiO_Test_FileName.3l";
    }

    static void CreateTestFile(const char* data = nullptr, std::streamsize size = 0u) {
        std::ofstream file{GetTestFileName()};
        if ((data != nullptr) && (size > 0u)) {
            file.write(data, size);
        }
    }

    static void CompareTestFile(const char* expected, std::streamsize size) {
        std::ifstream file{GetTestFileName()};
        std::vector<char> buffer(static_cast<std::size_t>(size));
        file.read(buffer.data(), size);
        ASSERT_EQ(file.gcount(), size);
        ASSERT_ELEMENTS_EQ(buffer, expected, static_cast<std::size_t>(size));
    }

    void SetUp() {
        // Make sure file does not exist
        std::remove(GetTestFileName());
        // Clear status
        sc::StatusProvider::reset();
    }

    void TearDown() {
        // Cleanup
        std::remove(GetTestFileName());
        // Clear status
        sc::StatusProvider::reset();
    }
};

using StreamTypes = ::testing::Types<trio::FileStream, trio::MemoryMappedFileStream>;
TYPED_TEST_SUITE(StreamTest, StreamTypes, );
