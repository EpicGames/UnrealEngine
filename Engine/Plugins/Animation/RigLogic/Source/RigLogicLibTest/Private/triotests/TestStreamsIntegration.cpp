// Copyright Epic Games, Inc. All Rights Reserved.

#include "triotests/TestStreams.h"

#include <pma/TypeDefs.h>

TYPED_TEST(StreamTest, PersistenceIntegrationTest) {
    static const char data[] = {0x00, 0x01};
    static constexpr std::size_t dataSize = 2ul;
    // Write into a file
    auto outStream =
        StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Write);
    ASSERT_STATUS_OK();
    outStream->open();
    ASSERT_STATUS_OK();
    ASSERT_EQ(outStream->write(data, dataSize), dataSize);
    ASSERT_STATUS_OK();
    outStream->close();
    ASSERT_STATUS_OK();

    // Test written data
    char buffer1[dataSize] = {};
    auto inOutStream =
        StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::ReadWrite);
    ASSERT_STATUS_OK();
    ASSERT_EQ(inOutStream->size(), dataSize);
    inOutStream->open();
    ASSERT_STATUS_OK();
    ASSERT_EQ(inOutStream->read(buffer1, dataSize), dataSize);
    ASSERT_STATUS_OK();
    ASSERT_ELEMENTS_EQ(buffer1, data, dataSize);
    ASSERT_EQ(inOutStream->tell(), 2ul);

    // Append to the file
    ASSERT_EQ(inOutStream->write(data, dataSize), dataSize);
    ASSERT_STATUS_OK();
    ASSERT_EQ(inOutStream->size(), dataSize * 2ul);
    ASSERT_STATUS_OK();
    inOutStream->close();
    ASSERT_STATUS_OK();

    // Test file in read-only mode
    char buffer2[dataSize * 2ul] = {};
    const char expected[dataSize * 2ul] = {0x00, 0x01, 0x00, 0x01};
    auto inStream = StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::Read);
    ASSERT_STATUS_OK();
    ASSERT_EQ(inStream->size(), dataSize * 2ul);
    inStream->open();
    ASSERT_STATUS_OK();
    ASSERT_EQ(inStream->read(buffer2, dataSize * 2ul), dataSize * 2ul);
    ASSERT_STATUS_OK();
    ASSERT_ELEMENTS_EQ(buffer2, expected, dataSize * 2ul);
}

TYPED_TEST(StreamTest, ReopenStreamContinueWork) {
    TestFixture::CreateTestFile("test", 4ul);

    auto stream =
        StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::ReadWrite);
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    stream->seek(2ul);
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), 2ul);

    ASSERT_EQ(stream->write("hello", 5ul), 5ul);
    ASSERT_STATUS_OK();

    stream->close();
    ASSERT_STATUS_OK();

    stream->open();
    ASSERT_STATUS_OK();

    char buffer[7ul] = {};
    const char expected[] = {'t', 'e', 'h', 'e', 'l', 'l', 'o'};
    ASSERT_EQ(stream->read(buffer, 7ul), 7ul);
    ASSERT_STATUS_OK();
    ASSERT_ELEMENTS_EQ(buffer, expected, 7ul);
}

// *INDENT-OFF*
#ifdef TRIO_BUILD_LFS_TESTS
TYPED_TEST(StreamTest, LFSIntegrationTest) {
    pma::Vector<char> smallBuffer(711ul, 'x');
    pma::Vector<char> largeBuffer((1ul * 1024ul * 1024ul * 1024ul) + 12ul, 'y');

    auto stream =
        StreamFactory<typename TestFixture::TStream>::create(TestFixture::GetTestFileName(), trio::AccessMode::ReadWrite);
    ASSERT_STATUS_OK();
    stream->open();
    ASSERT_STATUS_OK();

    std::uint64_t expectedPosition = 0ul;

    // Interleave a series of small / large writes
    ASSERT_EQ(stream->write(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));

    ASSERT_EQ(stream->write(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));

    ASSERT_EQ(stream->write(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));

    ASSERT_EQ(stream->write(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));

    ASSERT_EQ(stream->write(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));

    ASSERT_EQ(stream->write(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));

    ASSERT_EQ(stream->write(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));

    ASSERT_EQ(stream->write(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));

    ASSERT_EQ(stream->write(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));

    ASSERT_EQ(stream->write(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));

    ASSERT_EQ(stream->write(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));

    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic push
        #pragma GCC diagnostic ignored "-Wuseless-cast"
    #endif
    const std::uint64_t expectedSize =
        (5ull * static_cast<std::uint64_t>(largeBuffer.size())) + (6ull * static_cast<std::uint64_t>(smallBuffer.size()));
    #if !defined(__clang__) && defined(__GNUC__)
        #pragma GCC diagnostic pop
    #endif
    ASSERT_EQ(stream->size(), expectedSize);

    // Verify written data
    expectedPosition = 0ul;
    stream->seek(0ul);
    ASSERT_STATUS_OK();

    std::fill(smallBuffer.begin(), smallBuffer.end(), char{0});
    ASSERT_EQ(stream->read(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));
    ASSERT_TRUE(std::all_of(smallBuffer.cbegin(), smallBuffer.cend(), [](char c) { return c == 'x'; }));

    std::fill(largeBuffer.begin(), largeBuffer.end(), char{0});
    ASSERT_EQ(stream->read(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));
    ASSERT_TRUE(std::all_of(largeBuffer.cbegin(), largeBuffer.cend(), [](char c) { return c == 'y'; }));

    std::fill(smallBuffer.begin(), smallBuffer.end(), char{0});
    ASSERT_EQ(stream->read(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));
    ASSERT_TRUE(std::all_of(smallBuffer.cbegin(), smallBuffer.cend(), [](char c) { return c == 'x'; }));

    std::fill(largeBuffer.begin(), largeBuffer.end(), char{0});
    ASSERT_EQ(stream->read(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));
    ASSERT_TRUE(std::all_of(largeBuffer.cbegin(), largeBuffer.cend(), [](char c) { return c == 'y'; }));

    std::fill(smallBuffer.begin(), smallBuffer.end(), char{0});
    ASSERT_EQ(stream->read(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));
    ASSERT_TRUE(std::all_of(smallBuffer.cbegin(), smallBuffer.cend(), [](char c) { return c == 'x'; }));

    std::fill(largeBuffer.begin(), largeBuffer.end(), char{0});
    ASSERT_EQ(stream->read(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));
    ASSERT_TRUE(std::all_of(largeBuffer.cbegin(), largeBuffer.cend(), [](char c) { return c == 'y'; }));

    std::fill(smallBuffer.begin(), smallBuffer.end(), char{0});
    ASSERT_EQ(stream->read(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));
    ASSERT_TRUE(std::all_of(smallBuffer.cbegin(), smallBuffer.cend(), [](char c) { return c == 'x'; }));

    std::fill(largeBuffer.begin(), largeBuffer.end(), char{0});
    ASSERT_EQ(stream->read(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));
    ASSERT_TRUE(std::all_of(largeBuffer.cbegin(), largeBuffer.cend(), [](char c) { return c == 'y'; }));

    std::fill(smallBuffer.begin(), smallBuffer.end(), char{0});
    ASSERT_EQ(stream->read(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));
    ASSERT_TRUE(std::all_of(smallBuffer.cbegin(), smallBuffer.cend(), [](char c) { return c == 'x'; }));

    std::fill(largeBuffer.begin(), largeBuffer.end(), char{0});
    ASSERT_EQ(stream->read(largeBuffer.data(), largeBuffer.size()), largeBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += largeBuffer.size()));
    ASSERT_TRUE(std::all_of(largeBuffer.cbegin(), largeBuffer.cend(), [](char c) { return c == 'y'; }));

    std::fill(smallBuffer.begin(), smallBuffer.end(), char{0});
    ASSERT_EQ(stream->read(smallBuffer.data(), smallBuffer.size()), smallBuffer.size());
    ASSERT_STATUS_OK();
    ASSERT_EQ(stream->tell(), (expectedPosition += smallBuffer.size()));
    ASSERT_TRUE(std::all_of(smallBuffer.cbegin(), smallBuffer.cend(), [](char c) { return c == 'x'; }));

    stream->close();
    ASSERT_STATUS_OK();
}
#endif  // TRIO_BUILD_LFS_TESTS
// *INDENT-ON*
