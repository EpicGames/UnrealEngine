// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class RigLogicLibTest : ModuleRules
	{
		public RigLogicLibTest(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseUnity = false; // A windows include is preprocessing some method names causing compile failures.
			bDisableStaticAnalysis = true;

			PrivateDefinitions.Add("RL_BUILD_WITH_XYZ_ROTATION_ORDER=1");

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				if (Target.Architecture == UnrealArch.X64)
				{
					PrivateDefinitions.Add("RL_BUILD_WITH_SSE=1");
				}
				else if (Target.Architecture == UnrealArch.Arm64 || Target.Architecture == UnrealArch.Arm64ec)
				{
					PrivateDefinitions.Add("RL_BUILD_WITH_NEON");
				}
				PublicDefinitions.Add("GTEST_OS_WINDOWS=1");
			}

			string RigLogicLibPath = Path.GetFullPath(Path.Combine(ModuleDirectory, "../RigLogicLib"));

			if (Target.LinkType == TargetLinkType.Monolithic || Target.bMergeModules)
			{
				PublicDependencyModuleNames.Add("RigLogicLib");
				// Only tests (this module) should have access to Lib module's Private directory, so that's why Private was not made Internal.
				CppCompileWarningSettings.ModuleIncludePrivateWarningLevel = WarningLevel.Off;
				PrivateIncludePaths.Add(Path.Combine(RigLogicLibPath, "Private"));
			}
			else
			{
				PrivateDefinitions.Add("RIGLOGIC_MODULE_DISCARD");
				ConditionalAddModuleDirectory(new DirectoryReference(RigLogicLibPath));
			}

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"GoogleTest"
				}
			);
		}
	}
}
