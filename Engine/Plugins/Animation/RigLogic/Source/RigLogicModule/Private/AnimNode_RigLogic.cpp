// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_RigLogic.h"

#include "Components/SkeletalMeshComponent.h"
#include "DNAAsset.h"
#include "DNA.h"
#include "DNAAssetUserData.h"
#include "DNAIndexMapping.h"
#include "DNAReader.h"
#include "Engine/SkeletalMesh.h"
#include "RigLogic.h"
#include "RigInstance.h"
#include "RigLogicModule.h"
#include "SharedRigRuntimeContext.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Animation/AnimInstanceProxy.h"
#include "HAL/LowLevelMemTracker.h"
#include "Animation/MorphTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RigLogic)

LLM_DEFINE_TAG(Animation_RigLogic);
DEFINE_LOG_CATEGORY(LogRigLogicAnimNode);

#if STATS

DEFINE_STAT(STAT_RigLogic_VertexCount);

#endif  // STATS

static constexpr uint16 ATTR_COUNT_PER_JOINT = 10;

FAnimNode_RigLogic::FAnimNode_RigLogic() : RigInstance(nullptr)
{
}

FAnimNode_RigLogic::~FAnimNode_RigLogic()
{
	if (RigInstance != nullptr)
	{
		delete RigInstance;
		RigInstance = nullptr;
	}
}

void FAnimNode_RigLogic::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_Initialize_AnyThread);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::Initialize_AnyThread");
#endif  // CPUPROFILERTRACE_ENABLED
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	AnimSequence.Initialize(Context);
}

void FAnimNode_RigLogic::CacheVariableJointAttributes(const FBoneContainer& RequiredBones)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_CacheVariableJointAttributes);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::CacheVariableJointAttributes");
#endif  // CPUPROFILERTRACE_ENABLED
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	// Populate mapping of DNA joint indices to CompactPoseBoneIndex entries (used in updating joints with results from RigLogic)
	const uint16 CurrentLOD = RigInstance->GetLOD();
	TArrayView<const uint16> VariableJointIndices = LocalRigRuntimeContext->VariableJointIndicesPerLOD[CurrentLOD].Values;
	FCachedJointMapping& CurrentLODJointMapping = LocalJointMappingsPerLOD[CurrentLOD];
	auto& JointsMapDNAIndicesToCompactPoseBoneIndices = CurrentLODJointMapping.JointsMapDNAIndicesToCompactPoseBoneIndices;
	JointsMapDNAIndicesToCompactPoseBoneIndices.Empty();
	JointsMapDNAIndicesToCompactPoseBoneIndices.Reserve(VariableJointIndices.Num());
	for (const uint16 JointIndex : VariableJointIndices)
	{
		if (JointIndex < LocalDNAIndexMapping->JointsMapDNAIndicesToMeshPoseBoneIndices.Num())
		{
			const FMeshPoseBoneIndex MeshPoseBoneIndex = LocalDNAIndexMapping->JointsMapDNAIndicesToMeshPoseBoneIndices[JointIndex];
			const FCompactPoseBoneIndex CompactPoseBoneIndex = RequiredBones.MakeCompactPoseIndex(MeshPoseBoneIndex);
			if (CompactPoseBoneIndex != INDEX_NONE)
			{
				JointsMapDNAIndicesToCompactPoseBoneIndices.Add({ JointIndex, CompactPoseBoneIndex });
			}
		}
	}
}

void FAnimNode_RigLogic::CacheDriverJoints(const FBoneContainer& RequiredBones)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_CacheDriverJoints);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::CacheDriverJoints");
#endif  // CPUPROFILERTRACE_ENABLED
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	const uint16 CurrentLOD = RigInstance->GetLOD();
	FCachedJointMapping& CurrentLODJointMapping = LocalJointMappingsPerLOD[CurrentLOD];
	auto& SparseDriverJointsToControlAttributesMap = CurrentLODJointMapping.SparseDriverJointsToControlAttributesMap;
	auto& DenseDriverJointsToControlAttributesMap = CurrentLODJointMapping.DenseDriverJointsToControlAttributesMap;
	// Populate driver joint to raw control attribute mapping (used to feed RigLogic with inputs from the joint hierarchy)
	SparseDriverJointsToControlAttributesMap.Empty();
	DenseDriverJointsToControlAttributesMap.Empty();
	DenseDriverJointsToControlAttributesMap.Reserve(LocalDNAIndexMapping->DriverJointsToControlAttributesMap.Num());
	// Sparse mapping will likely remain empty so no reservation happens
	for (const auto& Mapping : LocalDNAIndexMapping->DriverJointsToControlAttributesMap)
	{
		const FCompactPoseBoneIndex CompactPoseBoneIndex = RequiredBones.MakeCompactPoseIndex(Mapping.MeshPoseBoneIndex);
		if (CompactPoseBoneIndex != INDEX_NONE)
		{
			if ((Mapping.RotationX != INDEX_NONE) && (Mapping.RotationY != INDEX_NONE) && (Mapping.RotationZ != INDEX_NONE) && (Mapping.RotationW != INDEX_NONE))
			{
				DenseDriverJointsToControlAttributesMap.Add({ CompactPoseBoneIndex, Mapping.DNAJointIndex, Mapping.RotationX, Mapping.RotationY, Mapping.RotationZ, Mapping.RotationW });
			}
			else
			{
				SparseDriverJointsToControlAttributesMap.Add({ CompactPoseBoneIndex, Mapping.DNAJointIndex, Mapping.RotationX, Mapping.RotationY, Mapping.RotationZ, Mapping.RotationW });
			}
		}
	}
}

void FAnimNode_RigLogic::CachePoseCurvesToRigLogicControlsMap(const FPoseContext& InputContext, const FCachedIndexedCurve& IndexedCurves, TArray<int32>& Indices)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_CachePoseCurvesToRigLogicControlsMap);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::CachePoseCurvesToRigLogicControlsMap");
#endif  // CPUPROFILERTRACE_ENABLED
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	Indices.Init(INDEX_NONE, InputContext.Curve.Num());
	// As long as there is only one raw control or ML mask associated with a curve, this works.
	// If this no longer holds true, Indices[CurveIndex] should become another list of indices,
	// and each driven control / ML mask index should be added to that (instead of assigning a
	// single one as it is done now).
	int32 CurveIndex = 0;
	InputContext.Curve.ForEachElement([&](const UE::Anim::FCurveElement& InCurveElement)
		{
			IndexedCurves.ForEachElement([&](const UE::Anim::FCurveElementIndexed& InControlAttributeCurveElement)
				{
					if (InCurveElement.Name == InControlAttributeCurveElement.Name)
					{
						Indices[CurveIndex] = InControlAttributeCurveElement.Index;
					}
				});
			++CurveIndex;
		});
}

void FAnimNode_RigLogic::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_CacheBones_AnyThread);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::CacheBones_AnyThread");
#endif  // CPUPROFILERTRACE_ENABLED
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	AnimSequence.CacheBones(Context);

	USkeletalMeshComponent* SkeletalMeshComponent = Context.AnimInstanceProxy->GetSkelMeshComponent();
	if (SkeletalMeshComponent == nullptr)
	{
		return;
	}

	USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
	if (SkeletalMesh == nullptr)
	{
		return;
	}

	USkeleton* Skeleton = Context.AnimInstanceProxy->GetSkeleton();
	if (Skeleton == nullptr)
	{
		return;
	}

	TSharedPtr<FSharedRigRuntimeContext> SharedRigRuntimeContext;
	TSharedPtr<FDNAIndexMapping> SharedDNAIndexMapping;

	// We either have DNAAssetUserData on SKM and then use it to get DNA asset or we use legacy asset user data as a fallback option in else
	if (UDNAAssetUserData* DNAAssetUserData = SkeletalMesh->GetAssetUserData<UDNAAssetUserData>())
	{
		if (UDNA* DNA = DNAAssetUserData->DNAAsset)
		{
			SharedRigRuntimeContext = DNA->GetRigRuntimeContext();
			SharedDNAIndexMapping = DNA->GetDNAIndexMapping(Skeleton, SkeletalMesh);
		}
	}
	else
	{
		if (UDNAAsset* DNAAsset = SkeletalMesh->GetAssetUserData<UDNAAsset>())
		{
			SharedRigRuntimeContext = DNAAsset->GetRigRuntimeContext();
			SharedDNAIndexMapping = DNAAsset->GetDNAIndexMapping(Skeleton, SkeletalMesh);
		}
	}

	// Both the runtime context AND the index mapping must be valid to make progress here.
	// GetDNAIndexMapping() can return null when the underlying UDNAAsset has no DNA reader yet
	// (e.g. a legacy UDNAAsset re-attached via AssetUserData after being superseded by
	// UDNAAssetUserData on reimport). Evaluate_AnyThread already has the symmetric guard;
	// without it here, CacheVariableJointAttributes / CacheDriverJoints below would deref a
	// null LocalDNAIndexMapping.
	if (!SharedRigRuntimeContext.IsValid() || !SharedDNAIndexMapping.IsValid())
	{
		return;
	}

	bool bResetCaches = false;

	if (LocalRigRuntimeContext != SharedRigRuntimeContext)
	{
		LocalRigRuntimeContext = SharedRigRuntimeContext;
		if (RigInstance != nullptr)
		{
			delete RigInstance;
		}
		RigInstance = new FRigInstance(LocalRigRuntimeContext->RigLogic.Get());
		bResetCaches = true;
	}

	RigInstance->SetLOD(Context.AnimInstanceProxy->GetLODLevel());

	if (LocalDNAIndexMapping != SharedDNAIndexMapping)
	{
		LocalDNAIndexMapping = SharedDNAIndexMapping;
		bResetCaches = true;
	}

	const uint16 LODCount = LocalRigRuntimeContext->RigLogic->GetLODCount();
	if (bResetCaches)
	{
		LocalJointMappingsPerLOD.Empty();
		LocalJointMappingsPerLOD.SetNum(LODCount);
		PoseCurvesToRigLogicControlsMap.Reset(LODCount);
		PoseCurvesToRigLogicControlsMap.AddDefaulted(LODCount);
	}

	// CacheBones is called on LOD switches as well, in which case compact pose bone indices must be remapped
	const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	if (RequiredBones.IsValid() && (LODCount > 0))
	{
		const uint16 CurrentLOD = RigInstance->GetLOD();
		FCachedJointMapping& CurrentLODJointMapping = LocalJointMappingsPerLOD[CurrentLOD];
		// Lazily initialize and cache mappings for each LOD as they are requested
		const int32 BoneCountForLOD = RequiredBones.GetCompactPoseNumBones();
		if (CurrentLODJointMapping.BoneCount != BoneCountForLOD)
		{
			const FRigLogicConfiguration& RigLogicConfig = LocalRigRuntimeContext->RigLogic->GetConfiguration();
			if (RigLogicConfig.LoadJoints)
			{
				CacheVariableJointAttributes(RequiredBones);
			}
			if (RigLogicConfig.LoadTwistSwingBehavior || RigLogicConfig.LoadRBFBehavior)
			{
				CacheDriverJoints(RequiredBones);
			}
			CurrentLODJointMapping.BoneCount = BoneCountForLOD;
		}
	}
}

void FAnimNode_RigLogic::Update_AnyThread(const FAnimationUpdateContext& Context)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_Update_AnyThread);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::Update_AnyThread");
#endif  // CPUPROFILERTRACE_ENABLED

	GetEvaluateGraphExposedInputs().Execute(Context);
	AnimSequence.Update(Context);
}

void FAnimNode_RigLogic::Evaluate_AnyThread(FPoseContext& OutputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_Evaluate_AnyThread);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::Evaluate_AnyThread");
#endif  // CPUPROFILERTRACE_ENABLED

	AnimSequence.Evaluate(OutputContext);

	if (!LocalRigRuntimeContext.IsValid() || !LocalDNAIndexMapping.IsValid())
	{
		return;
	}

	if (!IsLODEnabled(OutputContext.AnimInstanceProxy))
	{
		return;
	}

	const FRigLogic* RigLogic = LocalRigRuntimeContext->RigLogic.Get();
	if (RigLogic->GetLODCount() == 0)
	{
		return;
	}

	const FRigLogicConfiguration& RigLogicConfig = RigLogic->GetConfiguration();

	UpdateControlCurves(OutputContext);
	CalculateRigLogic();

	if (RigLogicConfig.LoadJoints)
	{
		UpdateJoints(OutputContext);
	}
	if (RigLogicConfig.LoadBlendShapes)
	{
		UpdateBlendShapeCurves(OutputContext);
	}
	if (RigLogicConfig.LoadAnimatedMaps)
	{
		UpdateAnimMapCurves(OutputContext);
	}

#if STATS
	RigLogic->CollectCalculationStats(RigInstance);
	USkeletalMeshComponent* SkeletalMeshComponent = OutputContext.AnimInstanceProxy->GetSkelMeshComponent();
	const int32 LODIndex = SkeletalMeshComponent->GetPredictedLODLevel();
	if (const FSkeletalMeshRenderData* SkeletalMeshRenderData = SkeletalMeshComponent->GetSkeletalMeshRenderData())
	{
		if (SkeletalMeshRenderData->LODRenderData.IsValidIndex(LODIndex))
		{
			const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData->LODRenderData[LODIndex];
			SET_DWORD_STAT(STAT_RigLogic_VertexCount, LODData.GetNumVertices());
		}
	}
#endif  // STATS
}

void FAnimNode_RigLogic::GatherDebugData(FNodeDebugData& DebugData)
{
	AnimSequence.GatherDebugData(DebugData);
}

void FAnimNode_RigLogic::UpdateRawControls(const FPoseContext& InputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateRawControls);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateRawControls");
#endif  // CPUPROFILERTRACE_ENABLED

	// Combine control attribute curve with input curve to get indexed curve to apply to rig
	// Curve elements that dont have a control mapping will have INDEX_NONE as their index
	TArrayView<float> RawControlBuffer = RigInstance->GetRawControlValues();
	UE::Anim::FNamedValueArrayUtils::Union(InputContext.Curve, LocalDNAIndexMapping->ControlAttributeCurves,
		[this, RawControlBuffer](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementIndexed& InControlAttributeCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (InControlAttributeCurveElement.Index != INDEX_NONE)
			{
				RawControlBuffer[InControlAttributeCurveElement.Index] = FMath::Clamp(InCurveElement.Value, 0.0, 1.0);
			}
		});
}

void FAnimNode_RigLogic::UpdateRawControlsCached(const FPoseContext& InputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateRawControlsCached);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateRawControlsCached");
#endif  // CPUPROFILERTRACE_ENABLED

	const uint16 CurrentLOD = RigInstance->GetLOD();
	auto& RawControlIndices = PoseCurvesToRigLogicControlsMap[CurrentLOD].RawControlIndices;
	if (RawControlIndices.Num() != InputContext.Curve.Num())
	{
		CachePoseCurvesToRigLogicControlsMap(InputContext, LocalDNAIndexMapping->ControlAttributeCurves, RawControlIndices);
	}

	int32 CurveIndex = 0;
	TArrayView<float> RawControlBuffer = RigInstance->GetRawControlValues();
	InputContext.Curve.ForEachElement([&](const UE::Anim::FCurveElement& InCurveElement)
		{
			const int32 ControlIndex = RawControlIndices[CurveIndex];
			if (ControlIndex != INDEX_NONE)
			{
				RawControlBuffer[ControlIndex] = FMath::Clamp(InCurveElement.Value, 0.0, 1.0);
			}
			++CurveIndex;
		});
}

void FAnimNode_RigLogic::UpdateSparseDriverJointDrivenControlCurves(const FPoseContext& InputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateSparseDriverJointDrivenControlCurves);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateSparseDriverJointDrivenControlCurves");
#endif  // CPUPROFILERTRACE_ENABLED

	const uint16 CurrentLOD = RigInstance->GetLOD();
	const auto& SparseDriverJointsToControlAttributesMap = LocalJointMappingsPerLOD[CurrentLOD].SparseDriverJointsToControlAttributesMap;
	TArrayView<const FQuat> InverseNeutralJointRotations = LocalRigRuntimeContext->InverseNeutralJointRotations;
	TArrayView<float> RawControlBuffer = RigInstance->GetRawControlValues();
	// The sparse mapping is NOT guaranteed to supply all quaternion attributes, so checks for each attribute mapping are present
	for (int32 MappingIndex = 0; MappingIndex < SparseDriverJointsToControlAttributesMap.Num(); ++MappingIndex)
	{
		const FCompactPoseBoneControlAttributeMapping& Mapping = SparseDriverJointsToControlAttributesMap[MappingIndex];
		const FTransform& CompactPose = InputContext.Pose[Mapping.CompactPoseBoneIndex];
		// Translation and Scale is currently not used here, so to avoid the overhead of checking them, they are simply ignored.
		// Should the need arise to use them as well, this code will need adjustment.
		const FQuat AbsPoseRotation = CompactPose.GetRotation();
		const FQuat InvNeutralRotation = InverseNeutralJointRotations[Mapping.DNAJointIndex];
		const FQuat DeltaPoseRotation = InvNeutralRotation * AbsPoseRotation;

		if (Mapping.RotationX != INDEX_NONE)
		{
			RawControlBuffer[Mapping.RotationX] = DeltaPoseRotation.X;
		}
		if (Mapping.RotationY != INDEX_NONE)
		{
			RawControlBuffer[Mapping.RotationY] = DeltaPoseRotation.Y;
		}
		if (Mapping.RotationZ != INDEX_NONE)
		{
			RawControlBuffer[Mapping.RotationZ] = DeltaPoseRotation.Z;
		}
		if (Mapping.RotationW != INDEX_NONE)
		{
			RawControlBuffer[Mapping.RotationW] = DeltaPoseRotation.W;
		}
	}
}

void FAnimNode_RigLogic::UpdateDenseDriverJointDrivenControlCurves(const FPoseContext& InputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateDenseDriverJointDrivenControlCurves);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateDenseDriverJointDrivenControlCurves");
#endif  // CPUPROFILERTRACE_ENABLED

	const uint16 CurrentLOD = RigInstance->GetLOD();
	const auto& DenseDriverJointsToControlAttributesMap = LocalJointMappingsPerLOD[CurrentLOD].DenseDriverJointsToControlAttributesMap;
	TArrayView<const FQuat> InverseNeutralJointRotations = LocalRigRuntimeContext->InverseNeutralJointRotations;
	TArrayView<float> RawControlBuffer = RigInstance->GetRawControlValues();
	// The dense mapping is guaranteed to supply all quaternion attributes, so NO checks for each attribute mapping are present
	for (int32 MappingIndex = 0; MappingIndex < DenseDriverJointsToControlAttributesMap.Num(); ++MappingIndex)
	{
		const FCompactPoseBoneControlAttributeMapping& Mapping = DenseDriverJointsToControlAttributesMap[MappingIndex];
		const FTransform& CompactPose = InputContext.Pose[Mapping.CompactPoseBoneIndex];
		// Translation and Scale is currently not used here, so to avoid the overhead of checking them, they are simply ignored.
		// Should the need arise to use them as well, this code will need adjustment.
		const FQuat AbsPoseRotation = CompactPose.GetRotation();
		const FQuat InvNeutralRotation = InverseNeutralJointRotations[Mapping.DNAJointIndex];
		const FQuat DeltaPoseRotation = InvNeutralRotation * AbsPoseRotation;
		RawControlBuffer[Mapping.RotationX] = DeltaPoseRotation.X;
		RawControlBuffer[Mapping.RotationY] = DeltaPoseRotation.Y;
		RawControlBuffer[Mapping.RotationZ] = DeltaPoseRotation.Z;
		RawControlBuffer[Mapping.RotationW] = DeltaPoseRotation.W;
	}
}

void FAnimNode_RigLogic::UpdateMLMaskCurves(const FPoseContext& InputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateMLMaskCurves);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateMLMaskCurves");
#endif  // CPUPROFILERTRACE_ENABLED

	TArrayView<float> MaskBuffer = RigInstance->GetMLMaskValues();
	if ((LocalDNAIndexMapping->MLMaskCurves.Num() != 0) && (LocalDNAIndexMapping->MLMaskCurves.Num() == MaskBuffer.Num()))
	{
		UE::Anim::FNamedValueArrayUtils::Union(InputContext.Curve, LocalDNAIndexMapping->MLMaskCurves,
			[this, MaskBuffer](const UE::Anim::FCurveElement& InCurveElement, const UE::Anim::FCurveElementIndexed& InControlAttributeCurveElement, UE::Anim::ENamedValueUnionFlags InFlags)
			{
				if (InControlAttributeCurveElement.Index != INDEX_NONE)
				{
					MaskBuffer[InControlAttributeCurveElement.Index] = InCurveElement.Value;
				}
			});
	}
}

void FAnimNode_RigLogic::UpdateMLMaskCurvesCached(const FPoseContext& InputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateMLMaskCurvesCached);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateMLMaskCurvesCached");
#endif  // CPUPROFILERTRACE_ENABLED

	const uint16 CurrentLOD = RigInstance->GetLOD();
	const auto& MLMaskCurves = LocalDNAIndexMapping->MLMaskCurves;
	TArrayView<float> MaskBuffer = RigInstance->GetMLMaskValues();
	auto& MLMaskCurveIndices = PoseCurvesToRigLogicControlsMap[CurrentLOD].MLMaskCurveIndices;
	if ((MLMaskCurves.Num() != 0) && (MLMaskCurves.Num() == MaskBuffer.Num()) && (MLMaskCurveIndices.Num() != InputContext.Curve.Num()))
	{
		CachePoseCurvesToRigLogicControlsMap(InputContext, MLMaskCurves, MLMaskCurveIndices);
	}

	if (MLMaskCurveIndices.Num() != 0)
	{
		int32 CurveIndex = 0;
		InputContext.Curve.ForEachElement([&](const UE::Anim::FCurveElement& InCurveElement)
			{
				const int32 MaskIndex = MLMaskCurveIndices[CurveIndex];
				if (MaskIndex != INDEX_NONE)
				{
					MaskBuffer[MaskIndex] = InCurveElement.Value;
				}
				++CurveIndex;
			});
	}
}

void FAnimNode_RigLogic::UpdateControlCurves(const FPoseContext& InputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateControlCurves);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateControlCurves");
#endif  // CPUPROFILERTRACE_ENABLED

	const FRigLogicConfiguration& RigLogicConfig = LocalRigRuntimeContext->RigLogic->GetConfiguration();

	if (CacheAnimCurveNames)
	{
		UpdateRawControlsCached(InputContext);
		if (RigLogicConfig.LoadMachineLearnedBehavior)
		{
			UpdateMLMaskCurvesCached(InputContext);
		}
	}
	else
	{
		UpdateRawControls(InputContext);
		if (RigLogicConfig.LoadMachineLearnedBehavior)
		{
			UpdateMLMaskCurves(InputContext);
		}
	}

	if (RigLogicConfig.LoadRBFBehavior || RigLogicConfig.LoadTwistSwingBehavior)
	{
		UpdateSparseDriverJointDrivenControlCurves(InputContext);
		UpdateDenseDriverJointDrivenControlCurves(InputContext);
	}
}

void FAnimNode_RigLogic::CalculateRigLogic()
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_CalculateRigLogic);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::CalculateRigLogic");
#endif  // CPUPROFILERTRACE_ENABLED

	FRigLogic* RigLogic = LocalRigRuntimeContext->RigLogic.Get();
	// RigLogic has Null evaluators for each class of computations, so no explicit checks are necessary here
	// based on the chosen configuration, as no extra work will be performed if not needed.
	RigLogic->CalculateMLControls(RigInstance);
	RigLogic->CalculateRBFControls(RigInstance);
	RigLogic->CalculatePSDControls(RigInstance);
	RigLogic->CalculateJoints(RigInstance);
	RigLogic->CalculateBlendShapes(RigInstance);
	RigLogic->CalculateAnimatedMaps(RigInstance);
}

void FAnimNode_RigLogic::UpdateJoints(FPoseContext& OutputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateJoints);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateJoints");
#endif  // CPUPROFILERTRACE_ENABLED

	const uint16 LOD = RigInstance->GetLOD();
	TArrayView<const float> DeltaJointValues = RigInstance->GetJointOutputs();
	TArrayView<const float> NeutralJointValues = LocalRigRuntimeContext->RigLogic->GetNeutralJointValues();
	const float* N = NeutralJointValues.GetData();
	const float* D = DeltaJointValues.GetData();
	const auto& JointsMapDNAIndicesToCompactPoseBoneIndices = LocalJointMappingsPerLOD[LOD].JointsMapDNAIndicesToCompactPoseBoneIndices;
	for (const FJointCompactPoseBoneMapping& Mapping : JointsMapDNAIndicesToCompactPoseBoneIndices)
	{
		const uint16 AttrIndex = Mapping.JointIndex * ATTR_COUNT_PER_JOINT;
		FTransform& CompactPose = OutputContext.Pose[Mapping.CompactPoseBoneIndex];
		CompactPose.SetTranslation(FVector((N[AttrIndex + 0] + D[AttrIndex + 0]), (N[AttrIndex + 1] + D[AttrIndex + 1]), (N[AttrIndex + 2] + D[AttrIndex + 2])));
		CompactPose.SetRotation(FQuat(N[AttrIndex + 3], N[AttrIndex + 4], N[AttrIndex + 5], N[AttrIndex + 6]) * FQuat(D[AttrIndex + 3], D[AttrIndex + 4], D[AttrIndex + 5], D[AttrIndex + 6]));
		CompactPose.SetScale3D(FVector((N[AttrIndex + 7] + D[AttrIndex + 7]), (N[AttrIndex + 8] + D[AttrIndex + 8]), (N[AttrIndex + 9] + D[AttrIndex + 9])));
	}
}

void FAnimNode_RigLogic::UpdateBlendShapeCurves(FPoseContext& OutputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateBlendShapeCurves);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateBlendShapeCurves");
#endif  // CPUPROFILERTRACE_ENABLED

	const uint16 LOD = RigInstance->GetLOD();
	TArrayView<const float> BlendShapeValues = RigInstance->GetBlendShapeOutputs();
	const FDNAIndexMapping::FCachedIndexedCurve& MorphTargetCurve = LocalDNAIndexMapping->MorphTargetCurvesPerLOD[LOD];
	UE::Anim::FNamedValueArrayUtils::Union(OutputContext.Curve, MorphTargetCurve, [&BlendShapeValues](UE::Anim::FCurveElement& InOutResult, const UE::Anim::FCurveElementIndexed& InSource, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (BlendShapeValues.IsValidIndex(InSource.Index))
			{
				InOutResult.Value = BlendShapeValues[InSource.Index];
				InOutResult.Flags |= UE::Anim::ECurveElementFlags::MorphTarget;
			}
		});
}

void FAnimNode_RigLogic::UpdateAnimMapCurves(FPoseContext& OutputContext)
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AnimNode_RigLogic_UpdateAnimMapCurves);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAnimNode_RigLogic::UpdateAnimMapCurves");
#endif  // CPUPROFILERTRACE_ENABLED

	const uint16 LOD = RigInstance->GetLOD();
	TArrayView<const float> AnimMapOutputs = RigInstance->GetAnimatedMapOutputs();
	const FDNAIndexMapping::FCachedIndexedCurve& MaskMultiplierCurve = LocalDNAIndexMapping->MaskMultiplierCurvesPerLOD[LOD];
	UE::Anim::FNamedValueArrayUtils::Union(OutputContext.Curve, MaskMultiplierCurve, [&AnimMapOutputs](UE::Anim::FCurveElement& InOutResult, const UE::Anim::FCurveElementIndexed& InSource, UE::Anim::ENamedValueUnionFlags InFlags)
		{
			if (AnimMapOutputs.IsValidIndex(InSource.Index))
			{
				InOutResult.Value = AnimMapOutputs[InSource.Index];
				InOutResult.Flags |= UE::Anim::ECurveElementFlags::Material;
			}
		});
}
