// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNA.h"
#include "DNAUtils.h"

#include "ArchiveMemoryStream.h"
#include "DNAAssetCustomVersion.h"
#include "DNAReaderAdapter.h"
#include "DNAIndexMapping.h"
#include "DNAUtils.h"
#include "FMemoryResource.h"
#include "LegacyDNAAssetUtils.h"
#include "RigLogicMemoryStream.h"
#include "SharedRigRuntimeContext.h"

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif
#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#endif
#include "Engine/AssetUserData.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "UObject/UObjectGlobals.h"
#include "ObjectCacheContext.h"
#include "Components/ComponentInterfaces.h"
#include "HAL/LowLevelMemTracker.h"
#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include "riglogic/RigLogic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DNA)

DEFINE_LOG_CATEGORY(LogDNA);

namespace DNA_Helpers
{
	static TSharedPtr<IDNAReader> CopyDNA(const IDNAReader* Source)
	{
		TArray<uint8> MemoryBuffer;
		FRigLogicMemoryStream MemoryStream(&MemoryBuffer);
		SaveDNAToStream(Source, EDNADataLayer::All, &MemoryStream);
		MemoryStream.seek(0ul);
		return LoadDNAFromBuffer(&MemoryBuffer);
	}

	// Writes a minimum-runtime DNA payload (Definition layer + per-region/NN metadata) directly
	// to the supplied stream. Used by UDNA::Serialize's optimized cook-save path to avoid an
	// extra parse-back round-trip.
	static void WriteMinimumRuntimeDNAToStream(const IDNAReader* Source, dna::BoundedIOStream* OutStream)
	{
		auto DNAReader = Source->Unwrap();
		auto DNAWriter = rl4::makeScoped<dna::BinaryStreamWriter>(OutStream, FMemoryResource::Instance());

		DNAWriter->setFrom(DNAReader, dna::DataLayer::Definition);

		for (uint16 MI = DNAReader->getMeshCount(); MI > 0u; --MI)
		{
			const auto MeshIndex = static_cast<uint16>(MI - 1u);
			for (uint16 RI = DNAReader->getMeshRegionCount(MeshIndex); RI > 0u; --RI)
			{
				const auto RegionIndex = static_cast<uint16>(RI - 1u);
				DNAWriter->setMeshRegionName(MeshIndex, RegionIndex, DNAReader->getMeshRegionName(MeshIndex, RegionIndex));
			}
		}

		DNAWriter->write();
	}

	static TSharedPtr<IDNAReader> ReloadDNAReaderAs(TSharedPtr<IDNAReader> InDNAReader, const FDNAConfig& DNAConfig)
	{
		LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));
		pma::ScopedPtr<dna::MemoryStream> MemoryStream = pma::makeScoped<dna::MemoryStream>();
		pma::ScopedPtr<dna::BinaryStreamWriter> DNAWriter = pma::makeScoped<dna::BinaryStreamWriter>(MemoryStream.get());
		DNAWriter->setFrom(InDNAReader->Unwrap());
		DNAWriter->write();
		MemoryStream->seek(0);
		return ReadDNAFromStream(MemoryStream.get(), DNAConfig);
	}

	static int32 AutoSelectImplicitlyLoadedLayers(int32 Layers)
	{
		// GeometryWithoutBlendShapes takes precedence over Geometry
		constexpr int32 GeometryBit = 1 << static_cast<uint8>(EDNADataLayerUIProxy::Geometry);
		constexpr int32 GeometryNoBSBit = 1 << static_cast<uint8>(EDNADataLayerUIProxy::GeometryNoBlendShapes);
		if (Layers & GeometryNoBSBit)
		{
			Layers &= ~GeometryBit;
		}

		static constexpr EDNADataLayer Mapping[] = {
			EDNADataLayer::Descriptor,                 // UIProxy::Descriptor = 0
			EDNADataLayer::Definition,                 // UIProxy::Definition = 1
			EDNADataLayer::Behavior,                   // UIProxy::Behavior = 2
			EDNADataLayer::Geometry,                   // UIProxy::Geometry = 3
			EDNADataLayer::GeometryWithoutBlendShapes, // UIProxy::GeometryNoBlendShapes = 4
			EDNADataLayer::MachineLearnedBehavior,     // UIProxy::MLBehavior = 5
			EDNADataLayer::RBFBehavior,                // UIProxy::RBFBehavior = 6
		};
		int32 Result = 0;
		for (int32 i = 0; i < UE_ARRAY_COUNT(Mapping); ++i)
		{
			if (Layers & (1 << i))
			{
				Result |= static_cast<int32>(Mapping[i]);
			}
		}
		return Result;
	}
}

#if WITH_EDITOR
static ERigLogicFloatingPointType ResolveFloatingPointTypeForPlatform(ERigLogicFloatingPointType InType, const ITargetPlatform* TargetPlatform)
{
	if (InType != ERigLogicFloatingPointType::Auto)
	{
		return InType;
	}

	// Desktop platforms default to Float, everything else to HalfFloat
	const FString PlatformGroup = TargetPlatform->GetPlatformInfo().PlatformGroupName.ToString();
	if (PlatformGroup == TEXT("Desktop"))
	{
		return ERigLogicFloatingPointType::Float;
	}
	return ERigLogicFloatingPointType::HalfFloat;
}
#endif // WITH_EDITOR

UDNA::UDNA() : MemoryResource(FMemoryResource::SharedInstance()), RigRuntimeContext(nullptr)
{
}

UDNA::~UDNA() = default;

TSharedPtr<IDNAReader> UDNA::GetDNAReader()
{
	UE::TReadScopeLock DNAScopeLock(DNAUpdateLock);
	return DNAReader;
}

void UDNA::SetDNAReader(TSharedPtr<IDNAReader> SourceDNAReader, EDNACopyPolicy CopyPolicy, ERigLogicInitPolicy InitPolicy)
{
	// Hold DNAUpdateLock for the whole function so that DNAConfig and DNAReader are mutated
	// atomically with respect to concurrent readers.
	UE::TWriteScopeLock DNAScopeLock(DNAUpdateLock);

	DNAConfig.CoordinateSystem = SourceDNAReader->GetCoordinateSystem();
	DNAConfig.RotationSequence = SourceDNAReader->GetRotationSequence();
	DNAConfig.RotationSign = SourceDNAReader->GetRotationSign();
	DNAConfig.FaceWindingOrder = SourceDNAReader->GetFaceWindingOrder();
	DNAConfig.CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Transform;

	if (CopyPolicy == EDNACopyPolicy::Copy)
	{
		DNAReader = DNA_Helpers::CopyDNA(SourceDNAReader.Get());
	}
	else
	{
		DNAReader = SourceDNAReader;
	}

	// Defer invalidates the entire (RigRuntimeContext, DNAIndexMappingContainer) pair together.
	// Skipping invalidation under Defer is safe because GetDNAIndexMapping sources its reader
	// from RigRuntimeContext->DNAReader (not UDNA::DNAReader), so a stale-but-still-valid context
	// stays paired with mappings built from the same underlying reader. The follow-up call
	// (e.g. RestoreLegacyUEMHCCompatibility) will Invalidate+Initialize together.
	if (InitPolicy == ERigLogicInitPolicy::Initialize)
	{
		InvalidateRigRuntimeContext();
		InitializeRigRuntimeContext(DNAReader);
	}
}

void UDNA::InitializeForRuntimeFrom(UDNA* Other)
{
	UE::TWriteScopeLock DNAScopeLock(DNAUpdateLock);
	UE::TWriteScopeLock ContextScopeLock(RigRuntimeContextUpdateLock);
	UE::TWriteScopeLock MappingScopeLock(DNAIndexMappingUpdateLock);

	// Store a reference to the other asset's runtime context, using the accessor function to 
	// ensure that the necessary lock is taken.
	//
	// The runtime context is immutable, i.e. not modified after initialization, so it's safe to
	// share across UDNAs.
	RigRuntimeContext = Other->GetRigRuntimeContext();

	DNAConfig = Other->DNAConfig;

	// DNAReader is used at runtime by GetDNAIndexMapping, so it needs to be populated here.
	//
	// The reference is taken from RigRuntimeContext to ensure it's consistent with the runtime
	// context, as the UDNA's DNAReader can be changed to point to a new one before the
	// runtime context is updated.
	//
	// As with the runtime context itself, the DNAReader is immutable and safe to share.
	DNAReader = RigRuntimeContext->DNAReader;

	// Ensure bKeepDNAAfterInitialization reflects the current state of the DNAReader, i.e.
	// if bKeepDNAAfterInitialization is true, DNAReader will contain DNA and vice versa.
	bKeepDNAAfterInitialization = Other->bKeepDNAAfterInitialization;

	bUseOptimizedCooking = Other->bUseOptimizedCooking;

	// This map is populated on demand, so doesn't need to be copied.
	DNAIndexMappingContainer.Empty(1);

	// Clear any fields not needed at runtime to avoid any old data causing confusion
#if WITH_EDITORONLY_DATA
	AssetImportData = nullptr;
#endif
}


void UDNA::InvalidateRigRuntimeContext()
{
	UE::TWriteScopeLock ContextScopeLock(RigRuntimeContextUpdateLock);
	UE::TWriteScopeLock MappingScopeLock(DNAIndexMappingUpdateLock);
	RigRuntimeContext = nullptr;
	DNAIndexMappingContainer.Empty(1);
}

void UDNA::InitializeRigRuntimeContext(TSharedPtr<IDNAReader> ReaderForRigLogic)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	if (ReaderForRigLogic.IsValid())
	{
		TSharedPtr<FRigLogic> RigLogic = MakeShared<FRigLogic>(ReaderForRigLogic.Get(), RigLogicConfiguration);
		TSharedPtr<FSharedRigRuntimeContext> NewContext = MakeShared<FSharedRigRuntimeContext>(ReaderForRigLogic, RigLogic);
		{
			UE::TWriteScopeLock ContextScopeLock(RigRuntimeContextUpdateLock);
			RigRuntimeContext = NewContext;
		}
#if !WITH_EDITOR
		// In cooked builds (Optimized Cooking off) the stream still carries the full DNA. FRigLogic
		// has copied what it needs above, so unload the reader's heavy payload to reclaim memory.
		if (!bKeepDNAAfterInitialization)
		{
			ReaderForRigLogic->Unload(EDNADataLayer::Behavior);
			ReaderForRigLogic->Unload(EDNADataLayer::Geometry);
			ReaderForRigLogic->Unload(EDNADataLayer::MachineLearnedBehavior);
			ReaderForRigLogic->Unload(EDNADataLayer::RBFBehavior);
		}
#endif  // !WITH_EDITOR
	}
}

TSharedPtr<FSharedRigRuntimeContext> UDNA::GetRigRuntimeContext()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	UE::TReadScopeLock ContextScopeLock(RigRuntimeContextUpdateLock);
	return RigRuntimeContext;
}

TSharedPtr<FDNAIndexMapping> UDNA::GetDNAIndexMapping(const USkeleton* Skeleton, const USkeletalMesh* SkeletalMesh)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	UE::TReadScopeLock DNAScopeLock(DNAUpdateLock);

	// Snapshot the runtime context's reader before acquiring DNAIndexMappingUpdateLock to keep
	// the lock order (DNA -> Context -> Mapping) consistent with InitializeForRuntimeFrom and
	// InvalidateRigRuntimeContext, avoiding a Mapping<->Context inversion.
	TSharedPtr<IDNAReader> RuntimeReaderSnapshot;
	{
		UE::TReadScopeLock ContextScopeLock(RigRuntimeContextUpdateLock);
		if (RigRuntimeContext.IsValid())
		{
			RuntimeReaderSnapshot = RigRuntimeContext->DNAReader;
		}
	}

	UE::TWriteScopeLock MappingScopeLock(DNAIndexMappingUpdateLock);

	// Find currently needed mapping and also clean stale objects along the way (requires only one iteration over the map)
	TSharedPtr<FDNAIndexMapping> DNAIndexMapping;
	for (auto Iterator = DNAIndexMappingContainer.CreateIterator(); Iterator; ++Iterator)
	{
		if ((Iterator->Key.SkeletalMesh.IsValid()) && (Iterator->Key.Skeleton.IsValid()))
		{
			if ((Iterator->Key.SkeletalMesh == SkeletalMesh) && (Iterator->Key.Skeleton == Skeleton))
			{
				DNAIndexMapping = Iterator->Value;
			}
		}
		else
		{
			Iterator.RemoveCurrent();
		}
	}

	// Check if currently needed mapping exists, and if not, create it now
	const FGuid SkeletonGuid = Skeleton->GetGuid();
	if (!DNAIndexMapping.IsValid() || (SkeletonGuid != DNAIndexMapping->SkeletonGuid))
	{
		// Use the runtime context's reader (snapshotted earlier) so any AnimNode that pairs
		// (RigRuntimeContext, DNAIndexMapping) sees consistent buffer sizes. UDNA::DNAReader
		// can transiently differ from RigRuntimeContext->DNAReader between SetDNAReader(Defer)
		// and the follow-up Convert/Initialize. Falls back to UDNA::DNAReader when the context
		// isn't built yet (e.g. before the first Initialize).
		const IDNAReader* ReaderForMapping = RuntimeReaderSnapshot.IsValid() ? RuntimeReaderSnapshot.Get() : DNAReader.Get();
		if (ReaderForMapping == nullptr)
		{
			return nullptr;
		}

		DNAIndexMapping = MakeShared<FDNAIndexMapping>();
		DNAIndexMapping->Init(ReaderForMapping, Skeleton, SkeletalMesh);
		DNAIndexMappingContainer.Add({ SkeletalMesh, Skeleton }, DNAIndexMapping);
	}

	return DNAIndexMapping;
}

void UDNA::RestoreLegacyUEMHCCompatibility()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	// Hold DNAUpdateLock for the whole function so all DNAReader/DNAConfig reads and writes
	// are consistent with respect to concurrent readers.
	UE::TWriteScopeLock DNAScopeLock(DNAUpdateLock);

	// Backwards-compatible with the legacy broken pipeline (UEMHC):
	//   - DNAReader must end up in Source (Maya) space, wrapped in FLegacyDNAReader so geometry
	//     queries apply the historical Maya->UE swizzle that UEMHC consumers depend on.
	//   - RigLogic is initialized from a separate reader containing Legacy-space data.
	// Note on wrapper class: FDNAReader and FLegacyDNAReader return different values for
	// geometry-layer queries (vertex positions/normals, blend shape deltas, neutral joint
	// transforms) but identical values for everything else. The DNAReader fast path therefore
	// requires the wrapper class to already be FLegacyDNAReader; the RigLogic init fast path
	// does not, because FRigLogic uses Reader->Unwrap() and consumers of RigRuntimeContext->DNAReader
	// (FDNAIndexMapping, FRigUnit_RigLogic_Data) only call functions that both wrappers delegate identically.
	const FDNAConfig SourceDNAConfig = FDNAConfig::Source(ECoordinateSystemTransformPolicy::Transform);
	const FDNAConfig LegacyDNAConfig = FDNAConfig::Legacy(ECoordinateSystemTransformPolicy::Transform);

	// The Legacy reader is consumed only by RigLogic init (and its retained reference inside
	// FSharedRigRuntimeContext, which downstream code uses for non-geometry queries). Skip
	// parsing the Geometry layer to save the largest portion of the per-call cost.
	const FDNAConfig LegacyDNAConfigForRigLogic = [&]
	{
		FDNAConfig Cfg = LegacyDNAConfig;
		Cfg.Layers = static_cast<int32>(EDNADataLayer::RBFBehavior | EDNADataLayer::MachineLearnedBehavior);
		return Cfg;
	}();

	const FCoordinateSystem CurrentCoordinateSystem = DNAReader->GetCoordinateSystem();
	const ERotationSequence CurrentRotationSequence = DNAReader->GetRotationSequence();
	const FRotationSign CurrentRotationSign = DNAReader->GetRotationSign();
	const EFaceWindingOrder CurrentFaceWindingOrder = DNAReader->GetFaceWindingOrder();

	// RigLogic ignores Geometry, so winding is irrelevant; wrapper class is also irrelevant here
	// (see comment above) -- we only need the data to already be in Legacy space.
	const bool bAlreadyLegacy = (CurrentCoordinateSystem == LegacyDNAConfig.CoordinateSystem &&
								 CurrentRotationSequence == LegacyDNAConfig.RotationSequence &&
								 CurrentRotationSign == LegacyDNAConfig.RotationSign);

	// DNAReader is consumed for geometry too, so winding AND wrapper class both matter here.
	const bool bAlreadySource = DNAReader->IsLegacyWrapped() &&
								(CurrentCoordinateSystem == SourceDNAConfig.CoordinateSystem &&
								 CurrentRotationSequence == SourceDNAConfig.RotationSequence &&
								 CurrentRotationSign == SourceDNAConfig.RotationSign &&
								 CurrentFaceWindingOrder == SourceDNAConfig.FaceWindingOrder);

	// Compute the union of layers required by the readers that will parse from the stream.
	// When SourceReader can alias the existing DNAReader (bAlreadySource), the stream only
	// needs to feed the Legacy read, which never reads Geometry -- so we can skip serializing
	// the Geometry layer entirely (it's the bulk of MetaHuman body DNA size). When the Source
	// read needs to parse from the stream too, the stream must contain Geometry.
	EDNADataLayer RequiredLayers = static_cast<EDNADataLayer>(0);
	if (!bAlreadyLegacy)
	{
		RequiredLayers |= EDNADataLayer::RBFBehavior | EDNADataLayer::MachineLearnedBehavior;
	}
	if (!bAlreadySource)
	{
		RequiredLayers |= EDNADataLayer::All;
	}

	// Lazily serialize at most once, only if some conversion is actually needed.
	pma::ScopedPtr<dna::MemoryStream> MemoryStream;
	pma::ScopedPtr<dna::BinaryStreamWriter> DNAWriter;
	auto EnsureSerialized = [&]
		{
			if (!MemoryStream)
			{
				MemoryStream = pma::makeScoped<dna::MemoryStream>();
				DNAWriter = pma::makeScoped<dna::BinaryStreamWriter>(MemoryStream.get());
				DNAWriter->setFrom(DNAReader->Unwrap(), CalculateDNADataLayerBitmask(RequiredLayers));
				DNAWriter->write();
			}
			MemoryStream->seek(0);
		};

	// LegacyReader is the temporary reader used only to construct FRigLogic (and is retained
	// inside FSharedRigRuntimeContext for downstream non-geometry queries). We don't need to
	// mutate UDNA::DNAReader to it -- pass it directly to InitializeRigRuntimeContext below.
	TSharedPtr<IDNAReader> LegacyReader;
	if (bAlreadyLegacy)
	{
		LegacyReader = DNAReader;
	}
	else
	{
		EnsureSerialized();
		LegacyReader = ReadDNAFromStream(MemoryStream.get(), LegacyDNAConfigForRigLogic);
	}

	// SourceReader is the reader that ends up as UDNA::DNAReader. Geometry layer is required
	// here -- UEMHC consumers query vertex positions, normals, and blend shape deltas through
	// UDNA::DNAReader.
	TSharedPtr<IDNAReader> SourceReader;
	if (bAlreadySource)
	{
		SourceReader = DNAReader;
	}
	else
	{
		EnsureSerialized();
		SourceReader = ReadDNAFromStream(MemoryStream.get(), SourceDNAConfig);
	}

	InvalidateRigRuntimeContext();
	InitializeRigRuntimeContext(LegacyReader);

	// Only overwrite the coord-system-related fields. Wholesale-assigning DNAConfig = LegacyDNAConfig
	// here would silently reset user-tunable fields (Layers, MaxLODPerPlatform, MinLODPerPlatform,
	// ExactLODs) to their UPROPERTY defaults, which causes property-panel edits routed through
	// PostEditChangeProperty's Legacy-shape branch to revert on the next refresh.
	DNAConfig.CoordinateSystemTransformPolicy = LegacyDNAConfig.CoordinateSystemTransformPolicy;
	DNAConfig.CoordinateSystem = LegacyDNAConfig.CoordinateSystem;
	DNAConfig.RotationSign = LegacyDNAConfig.RotationSign;
	DNAConfig.RotationSequence = LegacyDNAConfig.RotationSequence;
	DNAConfig.FaceWindingOrder = LegacyDNAConfig.FaceWindingOrder;

	DNAReader = SourceReader;
}

bool UDNA::Init(const FString& DNAFilename)
{
	return Init(DNAFilename, {});
}

bool UDNA::Init(const FString& DNAFilename, const FDNAConfig& InDNAConfig)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	if (!rl4::Status::isOk())
	{
		UE_LOGF(LogDNA, Warning, "%ls", ANSI_TO_TCHAR(rl4::Status::get().message));
	}

#if WITH_EDITORONLY_DATA
	AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	TArray<FAssetImportInfo::FSourceFile> SourceFiles = { FAssetImportInfo::FSourceFile(DNAFilename) };
	AssetImportData->SetSourceFiles(MoveTemp(SourceFiles));
#endif

	if (!FPaths::FileExists(DNAFilename))
	{
		UE_LOGF(LogDNA, Error, "DNA file %ls doesn't exist!", *DNAFilename);
		return false;
	}

	// Temporary buffer for the DNA file
	TArray<uint8> TempFileBuffer;

	if (!FFileHelper::LoadFileToArray(TempFileBuffer, *DNAFilename)) //load entire DNA file into the array
	{
		UE_LOGF(LogDNA, Error, "Couldn't read DNA file %ls!", *DNAFilename);
		return false;
	}

	UE::TWriteScopeLock DNAScopeLock(DNAUpdateLock);

	DNAConfig = InDNAConfig;
	DNAReader = LoadDNAFromBuffer(&TempFileBuffer, DNAConfig);
	if (!DNAReader.IsValid())
	{
		return false;
	}

	InvalidateRigRuntimeContext();
	InitializeRigRuntimeContext(DNAReader);

	return true;
}

void UDNA::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDNAAssetCustomVersion::GUID);

	// CDO has no DNA payload to (de)serialize.
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	UE::TWriteScopeLock DNAScopeLock(DNAUpdateLock);

	if (Ar.CustomVer(FDNAAssetCustomVersion::GUID) >= FDNAAssetCustomVersion::BeforeCustomVersionWasAdded)
	{
		if (Ar.IsLoading())
		{
			if (Ar.CustomVer(FDNAAssetCustomVersion::GUID) == FDNAAssetCustomVersion::BeforeCustomVersionWasAdded)
			{
				// This code is referring to version where DNA Asset had different readers for Behavior and Geometry data of the DNA file
				FArchiveMemoryStream BehaviorStream(&Ar);
				const EDNADataLayer BehaviorLayers = (
					EDNADataLayer::Behavior |
					EDNADataLayer::MachineLearnedBehavior |
					EDNADataLayer::RBFBehavior
					);
				DNAConfig = FDNAConfig::Legacy();
				DNAConfig.Layers = static_cast<int32>(BehaviorLayers);
				TSharedPtr<IDNAReader> BehaviorReader = LoadDNAFromStream(&BehaviorStream, DNAConfig);
				// Geometry data is always present (even if only as an empty placeholder), just so the uasset
				// format remains consistent between editor and non-editor builds
				FArchiveMemoryStream GeometryStream(&Ar);
				DNAConfig.Layers = static_cast<int32>(EDNADataLayer::Geometry);
				TSharedPtr<IDNAReader> GeometryReader = LoadDNAFromStream(&GeometryStream, DNAConfig);

				pma::ScopedPtr<dna::MemoryStream> MemoryStream = pma::makeScoped<dna::MemoryStream>();
				pma::ScopedPtr<dna::BinaryStreamWriter> DnaWriter = pma::makeScoped<dna::BinaryStreamWriter>(MemoryStream.get());
#if WITH_EDITORONLY_DATA
				DnaWriter->setFrom(GeometryReader->Unwrap(), dna::DataLayer::Geometry);
#endif
				DnaWriter->setFrom(BehaviorReader->Unwrap(), CalculateDNADataLayerBitmask(BehaviorLayers));
				DnaWriter->write();
				MemoryStream->seek(0);
				// Now with combined DNA streams we can read the provided memory stream into unified DNAReader
				DNAConfig.CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Transform;
				DNAConfig.Layers = static_cast<int32>(EDNADataLayer::All);
				DNAReader = LoadDNAFromStream(MemoryStream.get(), DNAConfig);
				InvalidateRigRuntimeContext();
				InitializeRigRuntimeContext(DNAReader);
			}
			else if (bUseOptimizedCooking && Ar.IsLoadingFromCookedPackage() && (Ar.CustomVer(FDNAAssetCustomVersion::GUID) >= FDNAAssetCustomVersion::IntroduceOptimizedSerializationDuringCooking))
			{
				// Read already initialized RigLogic state from archive with minimum necessary DNA data
				bool IsValid = false;
				Ar << IsValid;
				if (IsValid)
				{
					FArchiveMemoryStream DNAStream(&Ar);
					// IMPORTANT: do NOT pass UDNA::DNAConfig here. The bytes in the stream are
					// MinRuntimeDNA, written from CookedDNA -- which was already LOD-filtered by
					// the cook-target's MaxLOD/MinLOD. UDNA::DNAConfig still carries those same
					// values (PreSave flattened them into Default), so applying them again would
					// re-clamp against an already-clamped stream and produce a zero-LOD reader,
					// breaking FDNAIndexMapping which relies on DNAReader->GetLODCount() to size
					// MorphTargetCurvesPerLOD / MaskMultiplierCurvesPerLOD.
					// A default FDNAConfig has MaxLOD=0 / MinLOD=-1 (load all LODs in the stream)
					// and Preserve coordinate-transform policy, which is what we want here since
					// the stream bytes are already in the cook-target's coordinate space.
					const FDNAConfig PassThroughConfig;
					DNAReader = LoadDNAFromStream(&DNAStream, PassThroughConfig);
					TSharedPtr<FRigLogic> RigLogic = MakeShared<FRigLogic>(&Ar, RigLogicConfiguration);
					TSharedPtr<FSharedRigRuntimeContext> NewContext = MakeShared<FSharedRigRuntimeContext>(DNAReader, RigLogic);
					{
						UE::TWriteScopeLock ContextScopeLock(RigRuntimeContextUpdateLock);
						RigRuntimeContext = NewContext;
					}
				}
			}
			else if (Ar.CustomVer(FDNAAssetCustomVersion::GUID) <= FDNAAssetCustomVersion::GeneralizedCoordinateSystemConversion)
			{
				FArchiveMemoryStream DNAStream(&Ar);
				EDNADataLayer DNALayers = (
					EDNADataLayer::Behavior |
					EDNADataLayer::MachineLearnedBehavior |
					EDNADataLayer::RBFBehavior
					);

				if (!Ar.IsCooking())
				{
					DNALayers |= EDNADataLayer::Geometry;
				}

				if (Ar.CustomVer(FDNAAssetCustomVersion::GUID) < FDNAAssetCustomVersion::GeneralizedCoordinateSystemConversion)
				{
					DNAConfig = FDNAConfig::Legacy(ECoordinateSystemTransformPolicy::Transform);
				}
				DNAConfig.Layers = static_cast<int32>(DNALayers);
				DNAReader = LoadDNAFromStream(&DNAStream, DNAConfig);
				// For future asset reimports, this must migrate to Transform
				DNAConfig.CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Transform;

				// Handle broken asset data
				const bool IsSameCoordinateSystem = (DNAReader->GetCoordinateSystem() == DNAConfig.CoordinateSystem &&
													DNAReader->GetRotationSequence() == DNAConfig.RotationSequence &&
													DNAReader->GetRotationSign() == DNAConfig.RotationSign &&
													DNAReader->GetFaceWindingOrder() == DNAConfig.FaceWindingOrder);
				if ((Ar.CustomVer(FDNAAssetCustomVersion::GUID) >= FDNAAssetCustomVersion::GeneralizedCoordinateSystemConversion) && !IsSameCoordinateSystem)
				{
					if (DNAReader->GetCoordinateSystem() == DNAConfig.CoordinateSystem)
					{
						// Coordinate system matches but rotation signs not, these are new assets created in the intermediate stage where the implementation incorrectly
						// assigned positive rotation signs for the destination UE system
						DNAConfig.CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Transform;
						DNAReader = DNA_Helpers::ReloadDNAReaderAs(DNAReader, DNAConfig);
					}
					else
					{
						// New asset but old coordinate system (UEMHC makes these types), legacy data
						DNAConfig = FDNAConfig::Legacy(ECoordinateSystemTransformPolicy::Transform);
						DNAConfig.Layers = static_cast<int32>(DNALayers);
						DNAReader = DNA_Helpers::ReloadDNAReaderAs(DNAReader, DNAConfig);
					}
				}

				InvalidateRigRuntimeContext();
				InitializeRigRuntimeContext(DNAReader);
			}
			else
			{
				FArchiveMemoryStream DNAStream(&Ar);
				DNAReader = LoadDNAFromStream(&DNAStream, DNAConfig);
				InvalidateRigRuntimeContext();
				InitializeRigRuntimeContext(DNAReader);
			}
		}
		else if (Ar.IsSaving())
		{
			if (bUseOptimizedCooking && Ar.IsCooking() && (Ar.CustomVer(FDNAAssetCustomVersion::GUID) >= FDNAAssetCustomVersion::IntroduceOptimizedSerializationDuringCooking))
			{
				// Write already initialized RigLogic state into archive with minimum necessary DNA data
				bool IsValid = DNAReader.IsValid() && RigRuntimeContext->RigLogic.IsValid();
				Ar << IsValid;
				if (IsValid)
				{
					// Determine the source reader and RigLogic to write. By default we use the
					// host-built reader and the live RigLogic. If the cook target's resolved
					// LOD / RigLogic config differs from the host's, we rebuild both with the
					// cook-target values so the cooked archive contains data matching the target
					// -- including the Definition-layer entries written via WriteMinimumRuntimeDNAToStream,
					// whose output is filtered by MaxLOD/MinLOD (joint, mesh, blend-shape names, etc.).
					IDNAReader* SourceReader = DNAReader.Get();
					TSharedPtr<FRigLogic> RigLogic = RigRuntimeContext->RigLogic;
					TSharedPtr<IDNAReader> CookedDNA;  // kept alive across the writes below when bNeedsRebuild

#if WITH_EDITOR
					// PreSave has already flattened DNAConfig.* and RigLogicConfiguration.*
					// per-platform fields into Default for the cook target, so GetDefault()
					// returns the resolved cook-target value here.
					const int32 CookMaxLOD = DNAConfig.MaxLODPerPlatform.GetDefault();
					const int32 CookMinLOD = DNAConfig.MinLODPerPlatform.GetDefault();
					const ERigLogicCalculationType CookCalcType = static_cast<ERigLogicCalculationType>(RigLogicConfiguration.CalculationTypePerPlatform.GetDefault());
					const ERigLogicFloatingPointType CookFPType = static_cast<ERigLogicFloatingPointType>(RigLogicConfiguration.FloatingPointTypePerPlatform.GetDefault());

					// Host snapshots: the FDNAConfig the reader was built with on this host
					// (cached at reader construction by ResolveDNAConfigForHost) and the
					// FRigLogicConfiguration the live FRigLogic instance was built with
					// (FRigLogic stores its own copy, untouched by PreSave). Both are
					// already resolved to scalar values, so GetDefault() is safe and
					// race-free (no GetValue() -> OnGetPreviewPlatform read).
					const FDNAConfig& HostDNAConfig = DNAReader->GetConfig();
					const FRigLogicConfiguration& HostRigLogicConfig = RigLogic->GetConfiguration();

					const bool bNeedsRebuild =
						CookMaxLOD != HostDNAConfig.MaxLODPerPlatform.GetDefault()
						|| CookMinLOD != HostDNAConfig.MinLODPerPlatform.GetDefault()
						|| CookCalcType != static_cast<ERigLogicCalculationType>(HostRigLogicConfig.CalculationTypePerPlatform.GetDefault())
						|| CookFPType != static_cast<ERigLogicFloatingPointType>(HostRigLogicConfig.FloatingPointTypePerPlatform.GetDefault());

					if (bNeedsRebuild)
					{
						FDNAConfig CookTargetDNAConfig = DNAConfig;
						CookTargetDNAConfig.MaxLODPerPlatform = FPerPlatformInt(CookMaxLOD);
						CookTargetDNAConfig.MinLODPerPlatform = FPerPlatformInt(CookMinLOD);

						FRigLogicConfiguration CookTargetRigLogicConfig = RigLogicConfiguration;
						CookTargetRigLogicConfig.CalculationTypePerPlatform = FPerPlatformERigLogicCalculationType(CookCalcType);
						CookTargetRigLogicConfig.FloatingPointTypePerPlatform = FPerPlatformERigLogicFloatingPointType(CookFPType);

						CookedDNA = DNA_Helpers::ReloadDNAReaderAs(DNAReader, CookTargetDNAConfig);
						SourceReader = CookedDNA.Get();
						RigLogic = MakeShared<FRigLogic>(SourceReader, CookTargetRigLogicConfig);
					}
#endif  // WITH_EDITOR

					FArchiveMemoryStream DNAStream(&Ar);
					DNA_Helpers::WriteMinimumRuntimeDNAToStream(SourceReader, &DNAStream);
					RigLogic->Dump(&Ar);
				}
			}
			else
			{
				const IDNAReader* ReaderPtr = DNAReader.IsValid() ? DNAReader.Get() : nullptr;
				FArchiveMemoryStream DNAStream(&Ar);
				SaveDNAToStream(ReaderPtr, static_cast<EDNADataLayer>(DNAConfig.Layers), &DNAStream);
			}
		}
	}
}

void UDNA::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);

#if WITH_EDITOR
	if (SaveContext.IsCooking())
	{
		const ITargetPlatform* Platform = SaveContext.GetTargetPlatform();
		if (Platform)
		{
			const FName PlatformName = *Platform->IniPlatformName();

			// Flatten per-platform values into Default for the cooked build.
			// The PerPlatform map is WITH_EDITORONLY_DATA and will be stripped,
			// so Default must hold the resolved value for this platform.

			// DNAConfig
			DNAConfig.MaxLODPerPlatform.Default = DNAConfig.MaxLODPerPlatform.GetValueForPlatform(PlatformName);
			DNAConfig.MinLODPerPlatform.Default = DNAConfig.MinLODPerPlatform.GetValueForPlatform(PlatformName);

			// RigLogicConfiguration
			RigLogicConfiguration.CalculationTypePerPlatform.Default = RigLogicConfiguration.CalculationTypePerPlatform.GetValueForPlatform(PlatformName);
			RigLogicConfiguration.FloatingPointTypePerPlatform.Default = static_cast<int32>(ResolveFloatingPointTypeForPlatform(RigLogicConfiguration.FloatingPointTypePerPlatform.GetEnumValueForPlatform(PlatformName), Platform));
			RigLogicConfiguration.EnableMultiThreadMLComputePerPlatform.Default = RigLogicConfiguration.EnableMultiThreadMLComputePerPlatform.GetValueForPlatform(PlatformName);
		}
	}
#endif
}

void UDNA::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (!DNAReader.IsValid())
	{
		return;
	}

	const FDNAConfig LegacyConfig = FDNAConfig::Legacy();
	if (DNAConfig.CoordinateSystem == LegacyConfig.CoordinateSystem &&
		DNAConfig.RotationSign == LegacyConfig.RotationSign &&
		DNAConfig.RotationSequence == LegacyConfig.RotationSequence &&
		DNAConfig.FaceWindingOrder == LegacyConfig.FaceWindingOrder)
	{
		RestoreLegacyUEMHCCompatibility();
	}
#endif
}

#if WITH_EDITOR
void UDNA::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty == nullptr)
	{
		return;
	}

	const FName& MemberPropertyName = PropertyChangedEvent.MemberProperty->GetFName();
	const bool bRigLogicConfigChanged = (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UDNA, RigLogicConfiguration));
	const bool bDNAConfigChanged      = (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UDNA, DNAConfig));

	if (!bRigLogicConfigChanged && !bDNAConfigChanged)
	{
		return;
	}

	// Snapshot the meshes that have resolved a DNAIndexMapping against this
	// DNA. Both branches below rebuild RigRuntimeContext (and DNAConfig also
	// invalidates the index mapping cache), but FAnimNode_RigLogic caches both
	// in CacheBones_AnyThread and won't pick up the new values until something
	// forces CacheBones to re-run. We use this list afterwards to reinit those
	// components' anim instances. Held briefly under the mapping lock; weak
	// pointers are resolved later, after the rebuild has completed, so we never
	// touch potentially-stale UObjects while holding the lock.
	TSet<TWeakObjectPtr<USkeletalMesh>> MeshesToRefresh;
	{
		UE::TWriteScopeLock MappingScopeLock(DNAIndexMappingUpdateLock);
		MeshesToRefresh.Reserve(DNAIndexMappingContainer.Num());
		for (const auto& KV : DNAIndexMappingContainer)
		{
			if (KV.Key.SkeletalMesh.IsValid())
			{
				MeshesToRefresh.Add(const_cast<USkeletalMesh*>(KV.Key.SkeletalMesh.Get()));
			}
		}
	}

	if (bRigLogicConfigChanged)
	{
		UE::TWriteScopeLock DNAScopeLock(DNAUpdateLock);
		InvalidateRigRuntimeContext();
		InitializeRigRuntimeContext(DNAReader);
	}
	else if (bDNAConfigChanged)
	{
		DNAConfig.Layers = DNA_Helpers::AutoSelectImplicitlyLoadedLayers(DNAConfig.Layers);

		const FDNAConfig LegacyConfig = FDNAConfig::Legacy();
		if (DNAConfig.CoordinateSystem == LegacyConfig.CoordinateSystem &&
			DNAConfig.RotationSign == LegacyConfig.RotationSign &&
			DNAConfig.RotationSequence == LegacyConfig.RotationSequence &&
			DNAConfig.FaceWindingOrder == LegacyConfig.FaceWindingOrder)
		{
			RestoreLegacyUEMHCCompatibility();
		}
		else
		{
			UE::TWriteScopeLock DNAScopeLock(DNAUpdateLock);
			pma::ScopedPtr<dna::MemoryStream> MemoryStream = pma::makeScoped<dna::MemoryStream>();
			pma::ScopedPtr<dna::BinaryStreamWriter> DNAWriter = pma::makeScoped<dna::BinaryStreamWriter>(MemoryStream.get());
			DNAWriter->setFrom(DNAReader->Unwrap());
			DNAWriter->write();
			MemoryStream->seek(0);
			DNAReader = LoadDNAFromStream(MemoryStream.get(), DNAConfig);
			InvalidateRigRuntimeContext();
			InitializeRigRuntimeContext(DNAReader);
		}

		// Editing DNAConfig changes which LODs (and which Definition-layer entries)
		// the reader exposes. Cached FDNAIndexMapping entries were built against the
		// previous reader and would silently return stale data on lookup, since the
		// container is keyed by (SkeletalMesh, Skeleton) and only invalidates on
		// Skeleton GUID change. Drop the cache so consumers rebuild against the new
		// reader on next GetDNAIndexMapping call.
		{
			UE::TWriteScopeLock MappingScopeLock(DNAIndexMappingUpdateLock);
			DNAIndexMappingContainer.Empty();
		}
	}

	// Force any AnimInstance whose FAnimNode_RigLogic cached our previous
	// (RigRuntimeContext, DNAIndexMapping) pair to re-run CacheBones_AnyThread.
	// FObjectCacheContext maintains a reverse index from USkinnedAsset to its
	// consuming ISkinnedMeshComponents, so we never have to walk every skinned
	// mesh component in the world. Stale weak ptrs (mesh GC'd between snapshot
	// and now) just resolve to nullptr and are skipped.
	if (MeshesToRefresh.Num() == 0)
	{
		return;
	}

	FObjectCacheContextScope ObjectCacheScope;
	FObjectCacheContext& ObjectCache = ObjectCacheScope.GetContext();

	for (const TWeakObjectPtr<USkeletalMesh>& WeakMesh : MeshesToRefresh)
	{
		USkeletalMesh* Mesh = WeakMesh.Get();
		if (Mesh == nullptr)
		{
			continue;
		}

		for (ISkinnedMeshComponent* SkinnedComp : ObjectCache.GetSkinnedMeshComponents(Mesh))
		{
			IPrimitiveComponent* PrimComp = SkinnedComp != nullptr ? SkinnedComp->GetPrimitiveComponentInterface() : nullptr;
			USkeletalMeshComponent* SMC = PrimComp != nullptr ? PrimComp->GetUObject<USkeletalMeshComponent>() : nullptr;
			if (SMC != nullptr && !SMC->IsTemplate())
			{
				SMC->InitAnim(/*bForceReinit=*/true);
			}
		}
	}
}
#endif
