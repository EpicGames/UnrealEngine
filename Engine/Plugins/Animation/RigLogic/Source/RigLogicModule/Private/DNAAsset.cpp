// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAAsset.h"
#include "DNAUtils.h"

#include "ArchiveMemoryStream.h"
#include "DNAAssetCustomVersion.h"
#include "DNAReaderAdapter.h"
#include "DNAUtils.h"
#include "DNAIndexMapping.h"
#include "FMemoryResource.h"
#include "LegacyDNAReaderAdapter.h"
#include "RigLogicMemoryStream.h"
#include "SharedRigRuntimeContext.h"

#if WITH_EDITORONLY_DATA
    #include "EditorFramework/AssetImportData.h"
#endif
#include "Engine/AssetUserData.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "UObject/UObjectGlobals.h"
#include "HAL/LowLevelMemTracker.h"
#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#include "riglogic/RigLogic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DNAAsset)

DEFINE_LOG_CATEGORY(LogDNAAsset);

namespace DNAAsset_Helpers
{
	static TSharedPtr<IDNAReader> CopyDNA(const IDNAReader* Source)
	{
		TArray<uint8> MemoryBuffer;
		FRigLogicMemoryStream MemoryStream(&MemoryBuffer);
		SaveDNAToStream(Source, EDNADataLayer::All, &MemoryStream);
		MemoryStream.seek(0ul);
		return LoadDNAFromBuffer(&MemoryBuffer);
	}
}


UDNAAsset::UDNAAsset() : RigRuntimeContext{nullptr}
{
}

UDNAAsset::~UDNAAsset() = default;

const TSharedRef<IDNAReader> UDNAAsset::GetDnaReaderFromAsset() 
{
	return DNAReader.ToSharedRef();
}

TSharedPtr<IDNAReader> UDNAAsset::GetBehaviorReader()
{
	UE::TReadScopeLock DNAScopeLock{DNAUpdateLock};
	return DNAReader;
}

#if WITH_EDITORONLY_DATA
TSharedPtr<IDNAReader> UDNAAsset::GetGeometryReader()
{
	UE::TReadScopeLock DNAScopeLock{DNAUpdateLock};
	return DNAReader;
}
#endif

TSharedPtr<IDNAReader> UDNAAsset::GetDNAReader()
{
	UE::TReadScopeLock DNAScopeLock{ DNAUpdateLock };
	return DNAReader;
}

void UDNAAsset::SetDNAReader(TSharedPtr<IDNAReader> SourceDNAReader, EDNACopyPolicy CopyPolicy, ERigLogicInitPolicy InitPolicy)
{
	UE::TWriteScopeLock DNAScopeLock{DNAUpdateLock};

	if (CopyPolicy == EDNACopyPolicy::Copy)
	{
		DNAReader = DNAAsset_Helpers::CopyDNA(SourceDNAReader.Get());
	}
	else
	{
		DNAReader = SourceDNAReader;
	}

	// Defer invalidates the entire (RigRuntimeContext, DNAIndexMappingContainer) pair together.
	// Skipping invalidation under Defer is safe because GetDNAIndexMapping sources its reader
	// from RigRuntimeContext->DNAReader (not UDNAAsset::DNAReader), so a stale-but-still-valid
	// context stays paired with mappings built from the same underlying reader. The follow-up
	// call (e.g. RestoreLegacyUEMHCCompatibility) will Invalidate+Initialize together.
	if (InitPolicy == ERigLogicInitPolicy::Initialize)
	{
		InvalidateRigRuntimeContext();
		InitializeRigRuntimeContext(DNAReader);
	}
}

void UDNAAsset::SetBehaviorReader(TSharedPtr<IDNAReader> SourceDNAReader)
{
	SetDNAReader(SourceDNAReader);
}

void UDNAAsset::SetGeometryReader(TSharedPtr<IDNAReader> SourceDNAReader)
{
	SetDNAReader(SourceDNAReader);
}

void UDNAAsset::InitializeForRuntimeFrom(UDNAAsset* Other)
{
	UE::TWriteScopeLock DNAScopeLock{DNAUpdateLock};
	UE::TWriteScopeLock ContextScopeLock{RigRuntimeContextUpdateLock};
	UE::TWriteScopeLock MappingScopeLock{DNAIndexMappingUpdateLock};

	// Store a reference to the other asset's runtime context, using the accessor function to 
	// ensure that the necessary lock is taken.
	//
	// The runtime context is immutable, i.e. not modified after initialization, so it's safe to
	// share across UDNAAssets.
	RigRuntimeContext = Other->GetRigRuntimeContext();

	// DNAReader is used at runtime by GetDNAIndexMapping, so it needs to be populated here.
	//
	// The reference is taken from RigRuntimeContext to ensure it's consistent with the runtime
	// context, as the UDNAAsset's DNAReader can be changed to point to a new one before the
	// runtime context is updated.
	//
	// As with the runtime context itself, the DNAReader is immutable and safe to share.
	DNAReader = RigRuntimeContext->DNAReader;

	// Ensure bKeepDNAAfterInitialization reflects the current state of the DNAReader, i.e.
	// if bKeepDNAAfterInitialization is true, DNAReader will contain DNA and vice versa.
	bKeepDNAAfterInitialization = Other->bKeepDNAAfterInitialization;

	// This map is populated on demand, so doesn't need to be copied.
	DNAIndexMappingContainer.Empty(1);

	// Clear any fields not needed at runtime to avoid any old data causing confusion
#if WITH_EDITORONLY_DATA
	AssetImportData = nullptr;
#endif
}

void UDNAAsset::InvalidateRigRuntimeContext()
{
	UE::TWriteScopeLock ContextScopeLock{RigRuntimeContextUpdateLock};
	UE::TWriteScopeLock MappingScopeLock(DNAIndexMappingUpdateLock);
	RigRuntimeContext = nullptr;
	DNAIndexMappingContainer.Empty(1);
}

void UDNAAsset::InitializeRigRuntimeContext(TSharedPtr<IDNAReader> ReaderForRigLogic)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	if (ReaderForRigLogic.IsValid() && (ReaderForRigLogic->GetJointCount() != 0))
	{
		TSharedPtr<FRigLogic> RigLogic = MakeShared<FRigLogic>(ReaderForRigLogic.Get(), RigLogicConfiguration);
		TSharedPtr<FSharedRigRuntimeContext> NewContext = MakeShared<FSharedRigRuntimeContext>(ReaderForRigLogic, RigLogic);
		{
			UE::TWriteScopeLock ContextScopeLock{RigRuntimeContextUpdateLock};
			RigRuntimeContext = NewContext;
		}
#if !WITH_EDITOR
		if (!bKeepDNAAfterInitialization)
		{
			ReaderForRigLogic->Unload(EDNADataLayer::Behavior);
			ReaderForRigLogic->Unload(EDNADataLayer::Geometry);
			ReaderForRigLogic->Unload(EDNADataLayer::MachineLearnedBehavior);
			ReaderForRigLogic->Unload(EDNADataLayer::RBFBehavior);
		}
#endif  // !WITH_EDITOR
	}
}

TSharedPtr<FSharedRigRuntimeContext> UDNAAsset::GetRigRuntimeContext()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	UE::TReadScopeLock ContextScopeLock{RigRuntimeContextUpdateLock};
	return RigRuntimeContext;
}

TSharedPtr<FDNAIndexMapping> UDNAAsset::GetDNAIndexMapping(const USkeleton* Skeleton, const USkeletalMesh* SkeletalMesh)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	UE::TReadScopeLock DNAScopeLock{DNAUpdateLock};

	// Snapshot the runtime context's reader before acquiring DNAIndexMappingUpdateLock to keep
	// the lock order (DNA -> Context -> Mapping) consistent with InitializeForRuntimeFrom and
	// InvalidateRigRuntimeContext, avoiding a Mapping<->Context inversion.
	TSharedPtr<IDNAReader> RuntimeReaderSnapshot;
	{
		UE::TReadScopeLock ContextScopeLock(RigRuntimeContextUpdateLock);
		if (RigRuntimeContext.IsValid())
		{
			RuntimeReaderSnapshot = RigRuntimeContext->DNAReader;
		}
	}

	UE::TWriteScopeLock MappingScopeLock(DNAIndexMappingUpdateLock);

	// Find currently needed mapping and also clean stale objects along the way (requires only one iteration over the map)
	TSharedPtr<FDNAIndexMapping> DNAIndexMapping;
	for (auto Iterator = DNAIndexMappingContainer.CreateIterator(); Iterator; ++Iterator)
	{
		if ((Iterator->Key.SkeletalMesh.IsValid()) && (Iterator->Key.Skeleton.IsValid()))
		{
			if ((Iterator->Key.SkeletalMesh == SkeletalMesh) && (Iterator->Key.Skeleton == Skeleton))
			{
				DNAIndexMapping = Iterator->Value;
			}
		}
		else
		{
			Iterator.RemoveCurrent();
		}
	}

	// Check if currently needed mapping exists, and if not, create it now
	const FGuid SkeletonGuid = Skeleton->GetGuid();
	if (!DNAIndexMapping.IsValid() || (SkeletonGuid != DNAIndexMapping->SkeletonGuid))
	{
		// Use the runtime context's reader (snapshotted earlier) so any AnimNode that pairs
		// (RigRuntimeContext, DNAIndexMapping) sees consistent buffer sizes. UDNAAsset::DNAReader
		// can transiently differ from RigRuntimeContext->DNAReader between SetDNAReader(Defer)
		// and the follow-up Convert/Initialize. Falls back to UDNAAsset::DNAReader when the
		// context isn't built yet (e.g. before the first Initialize).
		const IDNAReader* ReaderForMapping = RuntimeReaderSnapshot.IsValid() ? RuntimeReaderSnapshot.Get() : DNAReader.Get();
		if (ReaderForMapping == nullptr)
		{
			return nullptr;
		}

		DNAIndexMapping = MakeShared<FDNAIndexMapping>();
		DNAIndexMapping->Init(ReaderForMapping, Skeleton, SkeletalMesh);
		DNAIndexMappingContainer.Add({SkeletalMesh, Skeleton}, DNAIndexMapping);
	}

	return DNAIndexMapping;
}

void UDNAAsset::RestoreLegacyUEMHCCompatibility()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	// Hold DNAUpdateLock for the whole function so all DNAReader reads and writes are
	// consistent with respect to concurrent readers.
	UE::TWriteScopeLock DNAScopeLock{DNAUpdateLock};

	// Backwards-compatible with the legacy broken pipeline (UEMHC):
	//   - DNAReader must end up in Source (Maya) space, wrapped in FLegacyDNAReader so geometry
	//     queries apply the historical Maya->UE swizzle that UEMHC consumers depend on.
	//   - RigLogic is initialized from a separate reader containing Legacy-space data.
	// Note on wrapper class: FDNAReader and FLegacyDNAReader return different values for
	// geometry-layer queries (vertex positions/normals, blend shape deltas, neutral joint
	// transforms) but identical values for everything else. The DNAReader fast path therefore
	// requires the wrapper class to already be FLegacyDNAReader; the RigLogic init fast path
	// does not, because FRigLogic uses Reader->Unwrap() and consumers of RigRuntimeContext->DNAReader
	// (FDNAIndexMapping, FRigUnit_RigLogic_Data) only call functions that both wrappers delegate identically.
	const FDNAConfig SourceDNAConfig = FDNAConfig::Source(ECoordinateSystemTransformPolicy::Transform);
	const FDNAConfig LegacyDNAConfig = FDNAConfig::Legacy(ECoordinateSystemTransformPolicy::Transform);

	// The Legacy reader is consumed only by RigLogic init (and its retained reference inside
	// FSharedRigRuntimeContext, which downstream code uses for non-geometry queries). Skip
	// parsing the Geometry layer to save the largest portion of the per-call cost.
	const FDNAConfig LegacyDNAConfigForRigLogic = [&]
	{
		FDNAConfig Cfg = LegacyDNAConfig;
		Cfg.Layers = static_cast<int32>(EDNADataLayer::RBFBehavior | EDNADataLayer::MachineLearnedBehavior);
		return Cfg;
	}();

	const FCoordinateSystem CurrentCoordinateSystem = DNAReader->GetCoordinateSystem();
	const ERotationSequence CurrentRotationSequence = DNAReader->GetRotationSequence();
	const FRotationSign CurrentRotationSign = DNAReader->GetRotationSign();
	const EFaceWindingOrder CurrentFaceWindingOrder = DNAReader->GetFaceWindingOrder();

	// RigLogic ignores Geometry, so winding is irrelevant; wrapper class is also irrelevant here
	// (see comment above) -- we only need the data to already be in Legacy space.
	const bool bAlreadyLegacy = (CurrentCoordinateSystem == LegacyDNAConfig.CoordinateSystem &&
								 CurrentRotationSequence == LegacyDNAConfig.RotationSequence &&
								 CurrentRotationSign == LegacyDNAConfig.RotationSign);

	// DNAReader is consumed for geometry too, so winding AND wrapper class both matter here.
	const bool bAlreadySource = DNAReader->IsLegacyWrapped() &&
								(CurrentCoordinateSystem == SourceDNAConfig.CoordinateSystem &&
								 CurrentRotationSequence == SourceDNAConfig.RotationSequence &&
								 CurrentRotationSign == SourceDNAConfig.RotationSign &&
								 CurrentFaceWindingOrder == SourceDNAConfig.FaceWindingOrder);

	// Compute the union of layers required by the readers that will parse from the stream.
	// When SourceReader can alias the existing DNAReader (bAlreadySource), the stream only
	// needs to feed the Legacy read, which never reads Geometry -- so we can skip serializing
	// the Geometry layer entirely (it's the bulk of MetaHuman body DNA size). When the Source
	// read needs to parse from the stream too, the stream must contain Geometry.
	EDNADataLayer RequiredLayers = static_cast<EDNADataLayer>(0);
	if (!bAlreadyLegacy)
	{
		RequiredLayers |= EDNADataLayer::RBFBehavior | EDNADataLayer::MachineLearnedBehavior;
	}
	if (!bAlreadySource)
	{
		RequiredLayers |= EDNADataLayer::All;
	}

	// Lazily serialize at most once, only if some conversion is actually needed.
	pma::ScopedPtr<dna::MemoryStream> MemoryStream;
	pma::ScopedPtr<dna::BinaryStreamWriter> DNAWriter;
	auto EnsureSerialized = [&]
		{
			if (!MemoryStream)
			{
				MemoryStream = pma::makeScoped<dna::MemoryStream>();
				DNAWriter = pma::makeScoped<dna::BinaryStreamWriter>(MemoryStream.get());
				DNAWriter->setFrom(DNAReader->Unwrap(), CalculateDNADataLayerBitmask(RequiredLayers));
				DNAWriter->write();
			}
			MemoryStream->seek(0);
		};

	// LegacyReader is the temporary reader used only to construct FRigLogic (and is retained
	// inside FSharedRigRuntimeContext for downstream non-geometry queries). We don't need to
	// mutate UDNAAsset::DNAReader to it -- pass it directly to InitializeRigRuntimeContext below.
	TSharedPtr<IDNAReader> LegacyReader;
	if (bAlreadyLegacy)
	{
		LegacyReader = DNAReader;
	}
	else
	{
		EnsureSerialized();
		LegacyReader = ReadDNAFromStream(MemoryStream.get(), LegacyDNAConfigForRigLogic);
	}

	// SourceReader is the reader that ends up as UDNAAsset::DNAReader. Geometry layer is required
	// here -- UEMHC consumers query vertex positions, normals, and blend shape deltas through
	// UDNAAsset::DNAReader.
	TSharedPtr<IDNAReader> SourceReader;
	if (bAlreadySource)
	{
		SourceReader = DNAReader;
	}
	else
	{
		EnsureSerialized();
		SourceReader = ReadDNAFromStream(MemoryStream.get(), SourceDNAConfig);
	}

	InvalidateRigRuntimeContext();
	InitializeRigRuntimeContext(LegacyReader);

	DNAReader = SourceReader;
}

bool UDNAAsset::Init(const FString& DNAFilename)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	if (!rl4::Status::isOk())
	{
		UE_LOGF(LogDNAAsset, Warning, "%ls", ANSI_TO_TCHAR(rl4::Status::get().message));
	}

#if WITH_EDITORONLY_DATA
	AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	TArray<FAssetImportInfo::FSourceFile> SourceFiles = { FAssetImportInfo::FSourceFile(DNAFilename) };
	AssetImportData->SetSourceFiles(MoveTemp(SourceFiles));
#endif
	
	if (!FPaths::FileExists(DNAFilename))
	{
		UE_LOGF(LogDNAAsset, Error, "DNA file %ls doesn't exist!", *DNAFilename);
		return false;
	}
	
	// Temporary buffer for the DNA file
	TArray<uint8> TempFileBuffer;
	
	if (!FFileHelper::LoadFileToArray(TempFileBuffer, *DNAFilename)) //load entire DNA file into the array
	{
		UE_LOGF(LogDNAAsset, Error, "Couldn't read DNA file %ls!", *DNAFilename);
		return false;
	}
	
	UE::TWriteScopeLock DNAScopeLock{DNAUpdateLock};

	DNAReader = LoadDNAFromBuffer(&TempFileBuffer);
	if (!DNAReader.IsValid())
	{
		return false;
	}

	InvalidateRigRuntimeContext();
	InitializeRigRuntimeContext(DNAReader);

	return true;
}

void UDNAAsset::Serialize(FArchive& Ar)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDNAAssetCustomVersion::GUID);

	UE::TWriteScopeLock DNAScopeLock{DNAUpdateLock};

	if (Ar.CustomVer(FDNAAssetCustomVersion::GUID) >= FDNAAssetCustomVersion::BeforeCustomVersionWasAdded)
	{
		if (Ar.IsLoading())
		{
			if(Ar.CustomVer(FDNAAssetCustomVersion::GUID) == FDNAAssetCustomVersion::BeforeCustomVersionWasAdded)
			{
				// This code is referring to version where DNA Asset had different readers for Behavior and Geometry data of the DNA file
				FArchiveMemoryStream BehaviorStream{&Ar};
				const EDNADataLayer BehaviorLayers = (
					EDNADataLayer::Behavior |
					EDNADataLayer::MachineLearnedBehavior |
					EDNADataLayer::RBFBehavior
					);
				FDNAConfig DNAConfig = FDNAConfig::Legacy();
				DNAConfig.Layers = static_cast<int32>(BehaviorLayers);
				TSharedPtr<IDNAReader> BehaviorReader = LoadDNAFromStream(&BehaviorStream, DNAConfig);
				// Geometry data is always present (even if only as an empty placeholder), just so the uasset
				// format remains consistent between editor and non-editor builds
				FArchiveMemoryStream GeometryStream{&Ar};
				DNAConfig.Layers = static_cast<int32>(EDNADataLayer::Geometry);
				TSharedPtr<IDNAReader> GeometryReader = LoadDNAFromStream(&GeometryStream, DNAConfig);

				pma::ScopedPtr<dna::MemoryStream> MemoryStream = pma::makeScoped<dna::MemoryStream>();
				pma::ScopedPtr<dna::BinaryStreamWriter> DnaWriter = pma::makeScoped<dna::BinaryStreamWriter>(MemoryStream.get());
#if WITH_EDITORONLY_DATA
				DnaWriter->setFrom(GeometryReader->Unwrap(), dna::DataLayer::Geometry);
#endif
				DnaWriter->setFrom(BehaviorReader->Unwrap(), CalculateDNADataLayerBitmask(BehaviorLayers));
				DnaWriter->write();
				MemoryStream->seek(0);
				// Now with combined DNA streams we can read the provided memory stream into unified DNAReader
				DNAConfig.Layers = static_cast<int32>(EDNADataLayer::All);
				DNAConfig.CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Transform;
				// Use old coordinate system parameters to match old behavior of already serialized assets
				DNAReader = LoadDNAFromStream(MemoryStream.get(), DNAConfig);
			}
			else
			{
				FArchiveMemoryStream DNAStream{&Ar};
				EDNADataLayer DNALayers = (
					EDNADataLayer::Behavior |
					EDNADataLayer::MachineLearnedBehavior |
					EDNADataLayer::RBFBehavior
					);
				
				if(!Ar.IsCooking())
				{
					DNALayers |= EDNADataLayer::Geometry;
				}

				FDNAConfig DNAConfig = FDNAConfig::Legacy(ECoordinateSystemTransformPolicy::Transform);
				DNAConfig.Layers = static_cast<int32>(DNALayers);
				DNAReader = LoadDNAFromStream(&DNAStream, DNAConfig);
			}
			InvalidateRigRuntimeContext();
			InitializeRigRuntimeContext(DNAReader);
		}
		else if (Ar.IsSaving())
		{
			IDNAReader* ReaderPtr = (DNAReader.IsValid() ? static_cast<IDNAReader*>(DNAReader.Get()) : nullptr);
			FArchiveMemoryStream DNAStream{&Ar};
			EDNADataLayer DNALayers = (
				EDNADataLayer::Behavior |
				EDNADataLayer::MachineLearnedBehavior |
				EDNADataLayer::RBFBehavior
				);
#if WITH_EDITORONLY_DATA
			// Geometry data is discarded unless in Editor
			DNALayers |= EDNADataLayer::Geometry;
#endif // #if WITH_EDITORONLY_DATA
			SaveDNAToStream(ReaderPtr, DNALayers, &DNAStream);
		}
	}
}

#if WITH_EDITOR
void UDNAAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty == nullptr)
	{
		return;
	}

	const FName& MemberPropertyName = PropertyChangedEvent.MemberProperty->GetFName();
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UDNAAsset, RigLogicConfiguration))
	{
		UE::TWriteScopeLock DNAScopeLock{DNAUpdateLock};
		InvalidateRigRuntimeContext();
		InitializeRigRuntimeContext(DNAReader);
	}
}
#endif
