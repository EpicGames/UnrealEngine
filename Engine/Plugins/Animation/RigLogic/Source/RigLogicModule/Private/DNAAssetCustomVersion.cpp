// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAAssetCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FDNAAssetCustomVersion::GUID(0x9DE7BD98, 0x67D445B2, 0x8C0E9D73, 0xFDE1E367);

// Register the custom version with core
FCustomVersionRegistration GRegisterDNAAssetCustomVersion(FDNAAssetCustomVersion::GUID, FDNAAssetCustomVersion::LatestVersion, TEXT("DNAAssetVer"));
