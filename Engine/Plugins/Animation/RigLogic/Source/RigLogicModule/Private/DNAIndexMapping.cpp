// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAIndexMapping.h"

#include "HAL/LowLevelMemTracker.h"
#include "DNA.h"
#include "DNAReader.h"
#include "Animation/Skeleton.h"
#include "Animation/SmartName.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimCurveTypes.h"


/** Constructs curve name from nameToSplit using formatString of form x<obj>y<attr>z **/
static FString CreateCurveName(const FString& NameToSplit, const FString& FormatString)
{
	// constructs curve name from NameToSplit (always in form <obj>.<attr>)
	// using FormatString of form x<obj>y<attr>z
	// where x, y and z are arbitrary strings
	// example:
	// FormatString="mesh_<obj>_<attr>"
	// 'head.blink_L' becomes 'mesh_head_blink_L'
	FString ObjectName, AttributeName;
	if (!NameToSplit.Split(".", &ObjectName, &AttributeName))
	{
		UE_LOG(LogDNA, Warning,
			TEXT("CreateCurveName: DNA name '%s' is not in '<obj>.<attr>' form; control will be skipped. ")
			TEXT("This typically indicates a malformed raw-control or animated-map name in the DNA."),
			*NameToSplit);
		return TEXT("");
	}
	FString CurveName = FormatString;
	CurveName = CurveName.Replace(TEXT("<obj>"), *ObjectName);
	CurveName = CurveName.Replace(TEXT("<attr>"), *AttributeName);
	return CurveName;
}

namespace
{
	enum class EDriverJointAttribute : uint8
	{
		None = 0,
		RotationX,
		RotationY,
		RotationZ,
		RotationW,
	};

	struct FDriverJointAttributeMatch
	{
		EDriverJointAttribute Attr = EDriverJointAttribute::None;
		int32 BoneIndex = INDEX_NONE;
	};

	// Classifies a raw control name as a driver-joint quaternion attribute.
	// A driver-joint attribute has the form "<boneName>.<component>" where
	// <component> is one of x/y/z/w or qx/qy/qz/qw, and <boneName> resolves
	// to a bone in the reference skeleton. These slots are populated by
	// FAnimNode_RigLogic's Sparse/Dense driver-joint passes, NOT by the
	// curve-driven ControlAttributeCurves path.
	static FDriverJointAttributeMatch ClassifyDriverJointAttribute(const FString& RawControlName, const FReferenceSkeleton& RefSkeleton)
	{
		FDriverJointAttributeMatch Result;

		if (RawControlName.EndsWith(TEXT(".x")) || RawControlName.EndsWith(TEXT(".qx")))
		{
			Result.Attr = EDriverJointAttribute::RotationX;
		}
		else if (RawControlName.EndsWith(TEXT(".y")) || RawControlName.EndsWith(TEXT(".qy")))
		{
			Result.Attr = EDriverJointAttribute::RotationY;
		}
		else if (RawControlName.EndsWith(TEXT(".z")) || RawControlName.EndsWith(TEXT(".qz")))
		{
			Result.Attr = EDriverJointAttribute::RotationZ;
		}
		else if (RawControlName.EndsWith(TEXT(".w")) || RawControlName.EndsWith(TEXT(".qw")))
		{
			Result.Attr = EDriverJointAttribute::RotationW;
		}
		else
		{
			return Result;
		}

		int32 IndexOfLastDot = INDEX_NONE;
		RawControlName.FindLastChar('.', IndexOfLastDot);
		if (IndexOfLastDot == INDEX_NONE)
		{
			Result.Attr = EDriverJointAttribute::None;
			return Result;
		}
		const FString PrefixName = RawControlName.Mid(0, IndexOfLastDot);
		Result.BoneIndex = RefSkeleton.FindBoneIndex(FName(*PrefixName));
		if (Result.BoneIndex == INDEX_NONE)
		{
			Result.Attr = EDriverJointAttribute::None;
		}
		return Result;
	}
}

void FDNAIndexMapping::MapControlCurves(const IDNAReader* DNAReader, const USkeleton* Skeleton)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	const uint32 ControlCount = DNAReader->GetRawControlCount();
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	ControlAttributeCurves.Empty();
	ControlAttributeCurves.Reserve(ControlCount);

	for (uint32_t ControlIndex = 0; ControlIndex < ControlCount; ++ControlIndex)
	{
		const FString DNAControlName = DNAReader->GetRawControlName(ControlIndex);

		// Skip driver-joint quaternion attributes ("<bone>.qw" / "<bone>.w" etc).
		// These raw-control slots are populated by the Sparse/Dense driver-joint passes in
		// FAnimNode_RigLogic, NOT by external curves. Registering them here causes
		// UpdateRawControls' Union() to write a default-constructed 0.0 over the
		// initializer's 1.0 (for the W slot) whenever the pose supplies no matching curve,
		// producing zero-length quaternions and NaN in TwistSwing/RBF on the first
		// evaluate after SetSkeletalMeshAsset.
		if (ClassifyDriverJointAttribute(DNAControlName, RefSkeleton).Attr != EDriverJointAttribute::None)
		{
			continue;
		}

		const FString AnimatedControlName = CreateCurveName(DNAControlName, TEXT("<obj>_<attr>"));
		if (AnimatedControlName == TEXT(""))
		{
			// Skip this entry, but keep mapping the rest. The original code returned here, which
			// silently dropped every subsequent raw control AND skipped the final sort below
			// (whose comment warns about concurrent-mutation hazards if it doesn't run).
			// CreateCurveName already logs a warning identifying the offending name.
			continue;
		}
		ControlAttributeCurves.Add(*AnimatedControlName, ControlIndex);
	}

	// This has the side-effect of sorting the NamedValueArray, which is necessary to avoid it happening at
	// runtime, which would cause concurrent mutation of the data structure, not guarded by any locking mechanism
	UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(ControlAttributeCurves, FCachedIndexedCurve{}, [](const UE::Anim::FCurveElementIndexed&, const UE::Anim::FCurveElementIndexed&) { return true; });
}

void FDNAIndexMapping::MapMLMaskCurves(const IDNAReader* DNAReader, const USkeleton* Skeleton)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	const uint16 MeshCount = DNAReader->GetMeshCount();
	const uint16 MappingCount = [MeshCount, DNAReader]()
	{
		uint32 Count = 0;
		for (uint16 MeshIndex = 0; MeshIndex < MeshCount; ++MeshIndex)
		{
			Count += DNAReader->GetMeshRegionCount(MeshIndex);
		}
		return static_cast<uint16>(Count);
	}();
	MLMaskCurves.Empty();
	MLMaskCurves.Reserve(MappingCount);

	uint32 MLMaskIndex = 0;
	for (uint16 MeshIndex = 0; MeshIndex < MeshCount; ++MeshIndex)
	{
		const uint16 MeshRegionCount = DNAReader->GetMeshRegionCount(MeshIndex);
		for (uint16 RegionIndex = 0; RegionIndex < MeshRegionCount; ++RegionIndex, ++MLMaskIndex)
		{
			const FString& MeshRegionName = DNAReader->GetMeshRegionName(MeshIndex, RegionIndex);
			const FString MaskCurveName = TEXT("CTRL_ML_") + MeshRegionName;
			MLMaskCurves.Add(*MaskCurveName, MLMaskIndex);
		}
	}

	// This has the side-effect of sorting the NamedValueArray, which is necessary to avoid it happening at
	// runtime, which would cause concurrent mutation of the data structure, not guarded by any locking mechanism
	UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(MLMaskCurves, FCachedIndexedCurve{}, [](const UE::Anim::FCurveElementIndexed&, const UE::Anim::FCurveElementIndexed&) { return true; });
}

void FDNAIndexMapping::MapDriverJoints(const IDNAReader* DNAReader, const USkeletalMesh* SkeletalMesh)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	if ((DNAReader->GetRBFSolverCount() == 0) && (DNAReader->GetSwingCount() == 0) && (DNAReader->GetTwistCount() == 0))
	{
		return;
	}

	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const uint32 ControlCount = DNAReader->GetRawControlCount();

	DriverJointsToControlAttributesMap.Empty();
	// This is a correct approximation as long as only 4 (rotation) attributes are used as driver joint attributes
	// and no regular raw controls are present in the DNA
	DriverJointsToControlAttributesMap.Reserve(ControlCount / 4);

	for (uint32_t ControlIndex = 0; ControlIndex < ControlCount; ++ControlIndex)
	{
		const FString DriverJointAttrName = DNAReader->GetRawControlName(ControlIndex);
		const FDriverJointAttributeMatch Match = ClassifyDriverJointAttribute(DriverJointAttrName, RefSkeleton);
		if (Match.Attr == EDriverJointAttribute::None)
		{
			// Mixed DNAs contain both driver-joint quaternion attributes and ordinary
			// raw controls (e.g. expression sliders like "CTRL_expressions_browLateralR")
			// in the same list. Non-quaternion or non-bone-prefixed names are skipped here.
			continue;
		}

		const int32 BoneIndex = Match.BoneIndex;
		int32 MappingIndex = DriverJointsToControlAttributesMap.FindLastByPredicate([BoneIndex](const FMeshPoseBoneControlAttributeMapping& Element)
		{
			return Element.MeshPoseBoneIndex.GetInt() == BoneIndex;
		});
		if (MappingIndex == INDEX_NONE)
		{
			FMeshPoseBoneControlAttributeMapping NewMapping{FMeshPoseBoneIndex{BoneIndex}, INDEX_NONE, INDEX_NONE , INDEX_NONE , INDEX_NONE, INDEX_NONE};
			MappingIndex = DriverJointsToControlAttributesMap.Add(NewMapping);
		}

		FMeshPoseBoneControlAttributeMapping& Mapping = DriverJointsToControlAttributesMap[MappingIndex];
		Mapping.DNAJointIndex = JointsMapDNAIndicesToMeshPoseBoneIndices.Find(Mapping.MeshPoseBoneIndex);

		switch (Match.Attr)
		{
			case EDriverJointAttribute::RotationX: Mapping.RotationX = ControlIndex; break;
			case EDriverJointAttribute::RotationY: Mapping.RotationY = ControlIndex; break;
			case EDriverJointAttribute::RotationZ: Mapping.RotationZ = ControlIndex; break;
			case EDriverJointAttribute::RotationW: Mapping.RotationW = ControlIndex; break;
			default: break; // unreachable - guarded by the None early-out above
		}
	}

	// Don't keep reservation if no mappings were created
	if (DriverJointsToControlAttributesMap.Num() == 0)
	{
		DriverJointsToControlAttributesMap.Empty();
	}
}

void FDNAIndexMapping::MapJoints(const IDNAReader* DNAReader, const USkeletalMesh* SkeletalMesh)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const uint16 JointCount = DNAReader->GetJointCount();
	JointsMapDNAIndicesToMeshPoseBoneIndices.Reset(JointCount);

	for (uint16 JointIndex = 0; JointIndex < JointCount; ++JointIndex)
	{
		const FString JointName = DNAReader->GetJointName(JointIndex);
		const FName BoneName = FName(*JointName);
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);

		// BoneIndex may be INDEX_NONE, but it's handled properly by the Evaluate method
		JointsMapDNAIndicesToMeshPoseBoneIndices.Add(FMeshPoseBoneIndex{ BoneIndex });
	}
}

void FDNAIndexMapping::MapMorphTargets(const IDNAReader* DNAReader, const USkeleton* Skeleton, const USkeletalMesh* SkeletalMesh)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	const uint16 LODCount = DNAReader->GetLODCount();
	const TMap<FName, int32>& MorphTargetIndexMap = SkeletalMesh->GetMorphTargetIndexMap();
	const TArray<UMorphTarget*>& MorphTargets = SkeletalMesh->GetMorphTargets();

	MorphTargetCurvesPerLOD.Reset(LODCount);
	MorphTargetCurvesPerLOD.AddDefaulted(LODCount);

	for (uint16 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
	{
		TArrayView<const uint16> MappingIndicesForLOD = DNAReader->GetMeshBlendShapeChannelMappingIndicesForLOD(LODIndex);

		MorphTargetCurvesPerLOD[LODIndex].Reserve(MappingIndicesForLOD.Num());

		for (uint16 MappingIndex : MappingIndicesForLOD)
		{
			const FMeshBlendShapeChannelMapping Mapping = DNAReader->GetMeshBlendShapeChannelMapping(MappingIndex);
			const FString MeshName = DNAReader->GetMeshName(Mapping.MeshIndex);
			const FString BlendShapeName = DNAReader->GetBlendShapeChannelName(Mapping.BlendShapeChannelIndex);
			const FString MorphTargetStr = MeshName + TEXT("__") + BlendShapeName;
			const FName MorphTargetName(*MorphTargetStr);
			const int32* MorphTargetIndex = MorphTargetIndexMap.Find(MorphTargetName);
			if ((MorphTargetIndex != nullptr) && (*MorphTargetIndex != INDEX_NONE))
			{
				const UMorphTarget* MorphTarget = MorphTargets[*MorphTargetIndex];
				MorphTargetCurvesPerLOD[LODIndex].Add(MorphTarget->GetFName(), Mapping.BlendShapeChannelIndex);
			}
		}

		// This has the side-effect of sorting the NamedValueArray, which is necessary to avoid it happening at
		// runtime, which would cause concurrent mutation of the data structure, not guarded by any locking mechanism
		UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(MorphTargetCurvesPerLOD[LODIndex], FCachedIndexedCurve{}, [](const UE::Anim::FCurveElementIndexed&, const UE::Anim::FCurveElementIndexed&) { return true; });
	}
}

void FDNAIndexMapping::MapMaskMultipliers(const IDNAReader* DNAReader, const USkeleton* Skeleton)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	const uint16 LODCount = DNAReader->GetLODCount();

	MaskMultiplierCurvesPerLOD.Reset();
	MaskMultiplierCurvesPerLOD.AddDefaulted(LODCount);

	for (uint16 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
	{
		TArrayView<const uint16> IndicesPerLOD = DNAReader->GetAnimatedMapIndicesForLOD(LODIndex);

		MaskMultiplierCurvesPerLOD[LODIndex].Reserve(IndicesPerLOD.Num());

		for (uint16 AnimMapIndex : IndicesPerLOD)
		{
			const FString AnimatedMapName = DNAReader->GetAnimatedMapName(AnimMapIndex);
			const FString MaskMultiplierNameStr = CreateCurveName(AnimatedMapName, TEXT("<obj>_<attr>"));
			if (MaskMultiplierNameStr == "")
			{
				// Skip this entry, but keep mapping the rest of this LOD and all subsequent LODs.
				// The original code returned here, which aborted the outer LOD loop as well, leaving
				// MaskMultiplierCurvesPerLOD[L] empty for every L > current and skipping the
				// per-LOD sort (concurrent-mutation hazard documented below).
				continue;
			}

			MaskMultiplierCurvesPerLOD[LODIndex].Add(*MaskMultiplierNameStr, AnimMapIndex);
		}

		// This has the side-effect of sorting the NamedValueArray, which is necessary to avoid it happening at
		// runtime, which would cause concurrent mutation of the data structure, not guarded by any locking mechanism
		UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(MaskMultiplierCurvesPerLOD[LODIndex], FCachedIndexedCurve{}, [](const UE::Anim::FCurveElementIndexed&, const UE::Anim::FCurveElementIndexed&) { return true; });
	}
}

void FDNAIndexMapping::Init(const IDNAReader* DNAReader, const USkeleton* Skeleton, const USkeletalMesh* SkeletalMesh)
{
	SkeletonGuid = Skeleton->GetGuid();
	MapControlCurves(DNAReader, Skeleton);
	MapMLMaskCurves(DNAReader, Skeleton);
	MapJoints(DNAReader, SkeletalMesh);
	MapDriverJoints(DNAReader, SkeletalMesh);
	MapMorphTargets(DNAReader, Skeleton, SkeletalMesh);
	MapMaskMultipliers(DNAReader, Skeleton);
}