// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAIndexMapping.h"

#include "DNAIndexMappingDeprecated.generated.h"

// Used and needed for previously serialized skeletal mesh assets
// that contain the DNA index mapping as part of their user data.
UCLASS(NotBlueprintable, hidecategories = (Object), deprecated)
class UDEPRECATED_DNAIndexMapping : public UAssetUserData
{
	GENERATED_BODY()
};

