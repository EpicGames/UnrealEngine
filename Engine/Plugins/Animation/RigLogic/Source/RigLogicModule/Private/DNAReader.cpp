// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAReader.h"

DEFINE_LOG_CATEGORY(LogDNAReader);
