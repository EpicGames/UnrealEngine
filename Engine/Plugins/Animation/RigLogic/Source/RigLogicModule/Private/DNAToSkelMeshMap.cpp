// Copyright Epic Games, Inc. All Rights Reserved.

#include "DNAToSkelMeshMap.h"

#include "DNAAsset.h"
#include "DNA.h"
#include "DNAAssetUserData.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "UObject/UObjectGlobals.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Animation/SmartName.h"
#include "Engine/SkeletalMesh.h"

#include "riglogic/RigLogic.h"

DEFINE_LOG_CATEGORY(LogDNAToSkelMeshMap);

#if WITH_EDITORONLY_DATA
/** Creates mappings between source DNA and target SkelMesh. */
void FDNAToSkelMeshMap::InitBaseMesh(IDNAReader* InSourceDNAReader, USkeletalMesh* InTargetSkeletalMesh)
{
	TargetSkelMesh = InTargetSkeletalMesh; //maps are created for specific SkelMesh, so we can memorize it for clarity

	InitVertexMap(InSourceDNAReader);

	//NOTE: this is just init for updating base mesh
	// before updating joints and morph targets, MapJoints and MapMorphTarget must be called respectively
}
#endif


/** Creates and initiates DNAReader and creates mapping between DNA and provided SkelMesh. */
bool FDNAToSkelMeshMap::InitFromDNAAsset(USkeletalMesh* InSkelMesh)
{
#if WITH_EDITORONLY_DATA
	// Fetch DNAAsset from SkelMesh and use its StreamReader
	// We either have DNAAssetUserData on SKM and then use it to get DNA asset or we use legacy asset user data as a fallback option in else
	if(UDNAAssetUserData* DNAAssetUserData = InSkelMesh->GetAssetUserData<UDNAAssetUserData>())
	{
		if (UDNA* DNA = DNAAssetUserData->DNAAsset)
		{
			InitBaseMesh(DNA->GetDNAReader().Get(), InSkelMesh);
			return true;
		}
		else
		{
			FString PathName = GetPathNameSafe(InSkelMesh);
			UE_LOGF(LogDNAToSkelMeshMap, Warning, "No DNA asset is referenced on DNAAssetUserData for %ls", *PathName);
			return false;
		}
	}
	else
	{
		if(UDNAAsset* DNAAsset = InSkelMesh->GetAssetUserData<UDNAAsset>())
		{
			InitBaseMesh(DNAAsset->GetDNAReader().Get(), InSkelMesh);
			return true;
		}
		else
		{
			FString PathName = GetPathNameSafe(InSkelMesh);
			UE_LOGF(LogDNAToSkelMeshMap, Warning, "Could not find DNAAsset user data for %ls", *PathName);
			return false;
		}
	}

#else
	return false;
#endif //WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
/** Maps vertices and meshes from SkelMesh to DNA. */
void FDNAToSkelMeshMap::InitVertexMap(IDNAReader* InDNAReader)
{
	double StartTime = FPlatformTime::Seconds();

	ImportVtxToDNAMeshIndex.Empty();
	ImportVtxToDNAVtxIndex.Empty();

	FSkeletalMeshModel* ImportedModel = TargetSkelMesh->GetImportedModel();
	int32 NumEngineLods = ImportedModel->LODModels.Num();
	ImportVtxToDNAMeshIndex.AddZeroed(NumEngineLods);
	ImportVtxToDNAVtxIndex.AddZeroed(NumEngineLods);
	for (int32 i = 0; i < NumEngineLods; i++)
	{
		FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[i];
		ImportVtxToDNAMeshIndex[i].AddZeroed(LODModel.MeshToImportVertexMap.Num());
		ImportVtxToDNAVtxIndex[i].AddZeroed(LODModel.MeshToImportVertexMap.Num());
	}

	TArray<TArray<int32>> MeshStartIndices; // For each LOD, store the start index offset of each mesh.
	TArray<int32> TotalVertices; // Counts total indices for each mesh in DNA, per LOD.
	uint16 NumLODs = InDNAReader->GetLODCount();
	MeshStartIndices.AddZeroed(NumLODs);
	TotalVertices.AddZeroed(NumLODs);
	ImportDNAVtxToUEVtxIndex.AddZeroed(NumLODs);
	// First find start indices for each mesh and for each LOD in DNA.
	uint16 TotalMeshCount = InDNAReader->GetMeshCount();
	for (uint16 LODIndex = 0; LODIndex < NumLODs; LODIndex++)
	{
		const TArrayView<const uint16> LODMeshIndices = InDNAReader->GetMeshIndicesForLOD(LODIndex);

		ImportDNAVtxToUEVtxIndex[LODIndex].AddZeroed(TotalMeshCount);

		for (size_t MeshIndex = 0; MeshIndex < LODMeshIndices.Num(); MeshIndex++)
		{
			MeshStartIndices[LODIndex].Add(TotalVertices[LODIndex]);

			const int32 VertexCount = InDNAReader->GetVertexPositionCount(LODMeshIndices[MeshIndex]);
			TotalVertices[LODIndex] += VertexCount;
			ImportDNAVtxToUEVtxIndex[LODIndex][LODMeshIndices[MeshIndex]].Init(INDEX_NONE, VertexCount);
		}
		MeshStartIndices[LODIndex].Add(TotalVertices[LODIndex]);
	}
	// Then for each LOD and each Vertex in LODModel Vertex Map find corresponding DNA Vertex.
	int32 ImportedVertexCount = 0;
	for (int32 LODIndex = 0; LODIndex < NumEngineLods && LODIndex < NumLODs; LODIndex++)
	{
		const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
		const TArrayView<const uint16> LODMeshIndices = InDNAReader->GetMeshIndicesForLOD(LODIndex);
		const int32 StartIndicesCount = MeshStartIndices[LODIndex].Num();

		int32 LODMeshVtxCount = LODModel.MeshToImportVertexMap.Num();
		for (int32 LODMeshVtxIndex = 0; LODMeshVtxIndex < LODMeshVtxCount; LODMeshVtxIndex++)
		{
			int32 FbxVertexIndex = LODModel.MeshToImportVertexMap[LODMeshVtxIndex];
			bool bFound = false;
			// For each LODModel Vertex try to find the corresponding DNA Vertex Index by checking prepared list of Start Indices.
			for (int k = 1; k < StartIndicesCount; k++)
			{
				if (FbxVertexIndex < MeshStartIndices[LODIndex][k])
				{
					bFound = true;
					int32 MeshIndex = LODMeshIndices[k - 1];
					ImportVtxToDNAMeshIndex[LODIndex][LODMeshVtxIndex] = MeshIndex;
					const int32 DNAVertexIndex = FbxVertexIndex - MeshStartIndices[LODIndex][k - 1];
					ImportVtxToDNAVtxIndex[LODIndex][LODMeshVtxIndex] = DNAVertexIndex;
					ImportDNAVtxToUEVtxIndex[LODIndex][MeshIndex][DNAVertexIndex] = LODMeshVtxIndex;
					break;
				}
			}
			// If no matching DNA Vertex Index is found mark LODModel Vertex Index with -1.
			if (!bFound)
			{
#ifdef DEBUG
				UE_LOGF(LogDNAToSkelMeshMap, Warning, "Not sorted fbx vertex found %d", FbxVertexIndex);
#endif
				ImportVtxToDNAMeshIndex[LODIndex][LODMeshVtxIndex] = INDEX_NONE;
				ImportVtxToDNAVtxIndex[LODIndex][LODMeshVtxIndex] = INDEX_NONE;
			}
		}
	}

	// Find and map overlapping indices and vertex to section indices.
	OverlappingVertices.Empty(NumEngineLods);
	OverlappingVertices.AddZeroed(NumEngineLods);
	UEVertexToSectionIndices.Empty(NumEngineLods);
	UEVertexToSectionIndices.AddZeroed(NumEngineLods);
	for (int32 i = 0; i < NumEngineLods; i++)
	{
		FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[i];

		UEVertexToSectionIndices[i].Init(INDEX_NONE, LODModel.NumVertices);

		const int32 NumSections = LODModel.Sections.Num();
		OverlappingVertices[i].AddZeroed(NumSections);
		for (int32 SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
		{
			const FSkelMeshSection& Section = LODModel.Sections[SectionIdx];
			const int32 NumVertices = Section.GetNumVertices();			

			TArray<bool> VerticesCovered;
			VerticesCovered.AddZeroed(NumVertices);

			OverlappingVertices[i][SectionIdx].AddZeroed(NumVertices);
			for (int32 VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
			{
				const int32 VertexBufferIndex = Section.GetVertexBufferIndex() + VertexIndex;
				UEVertexToSectionIndices[i][VertexBufferIndex] = SectionIdx;

				const int32 DNAVertexIndex = ImportVtxToDNAVtxIndex[i][VertexBufferIndex];
				if (DNAVertexIndex >= 0)
				{
					VerticesCovered[VertexIndex] = true;

					if (Section.OverlappingVertices.Contains(VertexIndex))
					{
						const int32 OverlappingCount = Section.OverlappingVertices[VertexIndex].Num();
						for (int32 OverlappingIndex = 0; OverlappingIndex < OverlappingCount; ++OverlappingIndex)
						{
							const int32 OverlappingVertexIndex = Section.OverlappingVertices[VertexIndex][OverlappingIndex];
							if (!VerticesCovered[OverlappingVertexIndex]) // No need to update the same vertex twice.
							{
								OverlappingVertices[i][SectionIdx][VertexIndex].Add(OverlappingVertexIndex);
							}
						}
					}
				}
			}
		}
	}

	double TimeToMap = FPlatformTime::Seconds();
	TimeToMap -= StartTime;

	UE_LOGF(LogDNAToSkelMeshMap, Log, "	InitVertexMap:	%.6f", TimeToMap * 1000.0f);
}

/** Makes a map of all Joints from DNA to Bones in Reference Skeleton.
 * Additionally makes a map of all Skinned joints. */
void FDNAToSkelMeshMap::MapJoints(IDNAReader* InDNAReader)
{

	double StartTime = FPlatformTime::Seconds();

	FReferenceSkeleton* RefSkeleton = &TargetSkelMesh->GetRefSkeleton();
	uint32 JointCount = InDNAReader->GetJointCount();

	// Map Joints to Bones.
	RLJointToUEBoneIndices.Empty();
	RLJointToUEBoneIndices.Reserve(JointCount);
	for (uint32 JntIndex = 0; JntIndex < JointCount; ++JntIndex)
	{
		const FString boneNameFStr = InDNAReader->GetJointName(JntIndex);
		const FName BoneName = FName{ *boneNameFStr };
		const int32 BoneIndex = RefSkeleton->FindBoneIndex(BoneName);

		// BoneIndex can be INDEX_NONE;
		// We can safely put it into the map with other indices, it will be handled in the Evaluate method.
		RLJointToUEBoneIndices.Add(BoneIndex);
	}


	double TimeToMap = FPlatformTime::Seconds();
	TimeToMap -= StartTime;

	UE_LOGF(LogDNAToSkelMeshMap, Log, "	Map joints:	%.6f", TimeToMap * 1000.0f);
}

void FDNAToSkelMeshMap::MapMorphTargets(IDNAReader* InDNAReader)
{
	double StartTime = FPlatformTime::Seconds();

	uint16 MeshCount = InDNAReader->GetMeshCount();
	// Assemble the reverse map, for each MorphTarget index save corresponding FDNABlendShapeTarget Index
	// Keeping MeshIndex and TargetIndex for each MorphTarget.
	MeshBlendShapeTargets.Empty();
	MeshBlendShapeTargets.Init(FDNABlendShapeTarget(), TargetSkelMesh->GetMorphTargets().Num());

	for (int16 MeshIndex = 0; MeshIndex < MeshCount; MeshIndex++)
	{
		uint16 TargetCount = InDNAReader->GetBlendShapeTargetCount(MeshIndex);
		for (int16 TargetIndex = 0; TargetIndex < TargetCount; TargetIndex++)
		{
			// Construct MorphTarget name by combining Blend Shape Channel name and Mesh name from DNA.
			const uint16 ChannelIndex = InDNAReader->GetBlendShapeChannelIndex(MeshIndex, TargetIndex);
			const FString BlendShapeStr = InDNAReader->GetBlendShapeChannelName(ChannelIndex);
			const FString MeshStr = InDNAReader->GetMeshName(MeshIndex);
			const FString MorphTargetStr = MeshStr + TEXT("__") + BlendShapeStr;
			const FName MorphTargetName = FName{ *MorphTargetStr };

			// Find MorphTarget from SkelMesh by previously constructed name.
			int32 MorphTargetIndex;
			UMorphTarget* MorphTarget = TargetSkelMesh->FindMorphTargetAndIndex(MorphTargetName, MorphTargetIndex);
			if (MorphTarget != nullptr)
			{
				// Store MeshIndex and TargetIndex for found MorphTarget.
				MeshBlendShapeTargets[MorphTargetIndex].MeshIndex = MeshIndex;
				MeshBlendShapeTargets[MorphTargetIndex].TargetIndex = TargetIndex;
			}
			else
			{
#ifdef DEBUG
				UE_LOGF(LogDNAToSkelMeshMap, Error, "Could not find morph target %ls (probably below threshold) Channel %d, Target %d", *MorphTargetStr, ChannelIndex, TargetIndex);
#endif
			}
		}
	}

	double TimeToMap = FPlatformTime::Seconds();
	TimeToMap -= StartTime;

	//UE_LOGF(LogDNAToSkelMeshMap, Log, "	MapMorphTargets:	%.6f", TimeToMap * 1000.0f);
}
#endif // WITH_EDITORONLY_DATA