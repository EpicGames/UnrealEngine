// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
class Hasher
{
public:
	explicit Hasher(uint32 InitialValue = 0ul) : HashedValue{InitialValue}
	{
	}
	uint32 GetHash() const
	{
		return HashedValue;
	}
	void Update(const FString& Value)
	{
		const auto& Data = Value.GetCharArray();
		Update(reinterpret_cast<const char*>(Data.GetData()), Data.Num() * Data.GetTypeSize());
	}
	template <typename T>
	void Update(T Value)
	{
		Update(reinterpret_cast<const char*>(&Value), sizeof(T));
	}
	template <typename T>
	void Update(TArrayView<T> Values)
	{
		Update(reinterpret_cast<const char*>(Values.Data()), Values.Num() * Values.GetTypeSize());
	}
private:
	void Update(const char* Bytes, size_t Length)
	{
		uint32 NewHash = HashedValue;
		for (size_t i = 0ul; i < Length; ++i)
		{
			NewHash = Bytes[i] + (NewHash << 6) + (NewHash << 16) - NewHash;
		}
		HashedValue = NewHash;
	}
private:
	uint32 HashedValue;
};