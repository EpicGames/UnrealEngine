// Copyright Epic Games, Inc. All Rights Reserved.

#include "LegacyDNAAssetUtils.h"

#include "FMemoryResource.h"
#include "tdm/TDM.h"
#include "riglogic/RigLogic.h"


struct ApplyRotationTransform
{
	tdm::fvec3 ExtractScaleVector(const tdm::fmat4& TransformationMatrix)
	{
		const tdm::fmat4& M = TransformationMatrix;
		const float SX = tdm::fvec3(M(0, 0), M(0, 1), M(0, 2)).length();
		const float SY = tdm::fvec3(M(1, 0), M(1, 1), M(1, 2)).length();
		const float SZ = tdm::fvec3(M(2, 0), M(2, 1), M(2, 2)).length();
		return { SX, SY, SZ };
	}

	tdm::fmat3 ExtractRotationMatrix(const tdm::fmat4& TransformationMatrix)
	{
		const tdm::fmat4 Scale = tdm::scale(ExtractScaleVector(TransformationMatrix));
		const tdm::fmat4 InverseScale = tdm::inverse(Scale);
		const tdm::fmat3 R = (InverseScale * TransformationMatrix).submat<3, 3>(0, 0);
		return R;
	}

	tdm::fvec3 ExtractTranslationVector(const tdm::fmat4& TransformationMatrix)
	{
		return { TransformationMatrix(3, 0), TransformationMatrix(3, 1), TransformationMatrix(3, 2) };
	}

	tdm::frad3 ExtractRotationVector(const tdm::fmat4& TransformationMatrix)
	{
		const tdm::fmat3 R = ExtractRotationMatrix(TransformationMatrix);
		tdm::rot_sign Positive{ tdm::rot_dir::positive, tdm::rot_dir::positive, tdm::rot_dir::positive };
		return tdm::impl::mat2euler<float>(R, tdm::rot_seq::xyz, Positive);
	}

	void operator()(const dna::Reader* DNAReader, dna::Writer* DNAWriter, const tdm::fmat4& InverseRotation)
	{
		{
			// Neutral joints
			const dna::RotationUnit RotationUnit = DNAReader->getRotationUnit();
			const dna::RotationSequence RotationSequence = DNAReader->getRotationSequence();
			const dna::RotationSign RotationSigns = DNAReader->getRotationSign();
			const uint16 JointCount = DNAReader->getJointCount();
			TArray<dna::Vector3> JointTranslations;
			TArray<dna::Vector3> JointRotations;
			JointTranslations.Reserve(JointCount);
			JointRotations.Reserve(JointCount);
			for (uint16 JointIndex = 0; JointIndex < JointCount; ++JointIndex)
			{
				const uint16 ParentIndex = DNAReader->getJointParentIndex(JointIndex);
				// Only root joint has been rotated, so we undo the rotation.
				if (JointIndex == ParentIndex)
				{
					const dna::Vector3 NeutralRotation = DNAReader->getNeutralJointRotation(JointIndex);
					const dna::Vector3 NeutralTranslation = DNAReader->getNeutralJointTranslation(JointIndex);
					const auto Angle2Rad = [RotationUnit](float A) { return (RotationUnit == dna::RotationUnit::degrees) ? tdm::frad(tdm::fdeg(A)) : tdm::frad(A); };
					const auto Rad2Angle = [RotationUnit](tdm::frad R) { return (RotationUnit == dna::RotationUnit::degrees) ? tdm::fdeg(R).value : R.value; };
					const auto RotationMatrix = tdm::rotate(Angle2Rad(NeutralRotation.x), Angle2Rad(NeutralRotation.y), Angle2Rad(NeutralRotation.z), RotationSequence, RotationSigns);
					const tdm::fmat4 TranslationMatrix = tdm::translate(tdm::fvec3(NeutralTranslation.x, NeutralTranslation.y, NeutralTranslation.z));
					const tdm::fmat4 TransformMatrix = RotationMatrix * TranslationMatrix * InverseRotation;
					const tdm::fvec3 Translation = ExtractTranslationVector(TransformMatrix);
					const tdm::frad3 Rotation = ExtractRotationVector(TransformMatrix);
					JointTranslations.Add(dna::Vector3(Translation[0], Translation[1], Translation[2]));
					JointRotations.Add(dna::Vector3(Rad2Angle(Rotation[0]), Rad2Angle(Rotation[1]), Rad2Angle(Rotation[2])));
				}
				else
				{
					JointTranslations.Add(DNAReader->getNeutralJointTranslation(JointIndex));
					JointRotations.Add(DNAReader->getNeutralJointRotation(JointIndex));
				}
			}
			DNAWriter->setNeutralJointTranslations(JointTranslations.GetData(), static_cast<uint16>(JointTranslations.Num()));
			DNAWriter->setNeutralJointRotations(JointRotations.GetData(), static_cast<uint16>(JointRotations.Num()));
		}

		// Vertex positions
		for (uint16 MeshIndex = 0; MeshIndex < DNAReader->getMeshCount(); ++MeshIndex)
		{
			const uint32 VertexCount = DNAReader->getVertexPositionCount(MeshIndex);
			dna::ConstArrayView<float> XS = DNAReader->getVertexPositionXs(MeshIndex);
			dna::ConstArrayView<float> YS = DNAReader->getVertexPositionYs(MeshIndex);
			dna::ConstArrayView<float> ZS = DNAReader->getVertexPositionZs(MeshIndex);
			assert((XS.size() == YS.size()) && (YS.size() == ZS.size()));
			TArray<dna::Vector3> Vertices;
			Vertices.Reserve(VertexCount);
			for (uint32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
			{
				const tdm::fvec4 Vertex{ XS[VertexIndex], YS[VertexIndex], ZS[VertexIndex], 1.0f };
				const tdm::fvec4 RotatedVertex = Vertex * InverseRotation;
				Vertices.Add(dna::Vector3(RotatedVertex[0], RotatedVertex[1], RotatedVertex[2]));
			}
			DNAWriter->setVertexPositions(MeshIndex, Vertices.GetData(), static_cast<uint32>(Vertices.Num()));
		}

		// Blend shape targets
		for (uint16 MeshIndex = 0; MeshIndex < DNAReader->getMeshCount(); ++MeshIndex)
		{
			for (uint16 BlendShapeTargetIndex = 0; BlendShapeTargetIndex < DNAReader->getBlendShapeTargetCount(MeshIndex); ++BlendShapeTargetIndex)
			{
				const uint32 DeltaCount = DNAReader->getBlendShapeTargetDeltaCount(MeshIndex, BlendShapeTargetIndex);
				dna::ConstArrayView<float> XS = DNAReader->getBlendShapeTargetDeltaXs(MeshIndex, BlendShapeTargetIndex);
				dna::ConstArrayView<float> YS = DNAReader->getBlendShapeTargetDeltaYs(MeshIndex, BlendShapeTargetIndex);
				dna::ConstArrayView<float> ZS = DNAReader->getBlendShapeTargetDeltaZs(MeshIndex, BlendShapeTargetIndex);
				assert((XS.size() == YS.size()) && (YS.size() == ZS.size()));
				TArray<dna::Vector3> Deltas;
				Deltas.Reserve(DeltaCount);
				for (uint32 DeltaIndex = 0; DeltaIndex < DeltaCount; ++DeltaIndex)
				{
					const tdm::fvec4 Delta{ XS[DeltaIndex], YS[DeltaIndex], ZS[DeltaIndex], 1.0f };
					const tdm::fvec4 RotatedDelta = Delta * InverseRotation;
					Deltas.Add(dna::Vector3(RotatedDelta[0], RotatedDelta[1], RotatedDelta[2]));
				}
				DNAWriter->setBlendShapeTargetDeltas(MeshIndex, BlendShapeTargetIndex, Deltas.GetData(), static_cast<uint32>(Deltas.Num()));
			}
		}
	}
};

bool IsLegacyDNAAsset(const dna::Reader* DNAReader)
{
	const FString FNDB("FN");
	const FString FNDB_V2("FN_MH_v2");
	const FString DBName(ANSI_TO_TCHAR(DNAReader->getDBName().data()));
	return (DBName == FNDB || DBName == FNDB_V2);
}

static bool IsLegacyDNAAssetMigratable(const dna::Reader* DNAReader)
{
	const dna::CoordinateSystem FNCSv1(dna::Direction::right, dna::Direction::back, dna::Direction::up);
	const dna::CoordinateSystem FNCSv2(dna::Direction::left, dna::Direction::back, dna::Direction::up);
	const dna::CoordinateSystem FNCSv3(dna::Direction::right, dna::Direction::up, dna::Direction::front);
	const dna::CoordinateSystem CS = DNAReader->getCoordinateSystem();
	return (CS == FNCSv1 || CS == FNCSv2 || CS == FNCSv3);
}

// Support DNAs that were incorrectly converted into UE space, using DNACalib2::RotateCommand. Instead of change of basis,
// only root joint, vertices, and blend shape deltas were rotated.
dna::ScopedPtr<dna::BinaryStreamReader> MigrateLegacyDNAAsset(dna::BoundedIOStream* Stream, const dna::Configuration& DNAConfig)
{
	// Reload untouched DNA (since coordinate system was transformed in-place).
	Stream->seek(0);
	auto OriginalDNA = dna::makeScoped<dna::BinaryStreamReader>(Stream, dna::Configuration{}, FMemoryResource::Instance());
	OriginalDNA->read();
	// Only certain (legacy, invalid) coordinate system configurations need migration
	if (!IsLegacyDNAAssetMigratable(OriginalDNA.get()))
	{
		return nullptr;
	}

	// Undo the incorrect rotate transforms (which was the fake coordinate system conversion so far).
	auto MemoryStream = dna::makeScoped<dna::MemoryStream>();
	auto DNAStreamWriter = dna::makeScoped<dna::BinaryStreamWriter>(MemoryStream.get(), FMemoryResource::Instance());
	DNAStreamWriter->setFrom(OriginalDNA.get(), dna::DataLayer::All, dna::UnknownLayerPolicy::Preserve, FMemoryResource::Instance());
	const tdm::rot_seq RotationSequence = OriginalDNA->getRotationSequence();
	const tdm::rot_sign RotationSigns = OriginalDNA->getRotationSign();
	const tdm::fmat4 InverseRotation = tdm::rotate(tdm::frad(tdm::fdeg(-90.0f)), tdm::frad(tdm::fdeg(0.0f)), tdm::frad(tdm::fdeg(0.0f)), RotationSequence, RotationSigns);
	ApplyRotationTransform()(OriginalDNA.get(), DNAStreamWriter.get(), InverseRotation);
	dna::CoordinateSystem OriginalCS(dna::Direction::left, dna::Direction::up, dna::Direction::front);
	DNAStreamWriter->setCoordinateSystem(OriginalCS);
	DNAStreamWriter->write();
	MemoryStream->seek(0);
	// Load DNA again with coordinate system conversion enabled since source data is now corrected.
	auto DNAReader = dna::makeScoped<dna::BinaryStreamReader>(MemoryStream.get(), DNAConfig, FMemoryResource::Instance());
	DNAReader->read();
	return DNAReader;
}
