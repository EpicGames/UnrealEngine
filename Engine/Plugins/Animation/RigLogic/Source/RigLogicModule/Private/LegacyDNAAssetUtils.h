// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "riglogic/RigLogic.h"

bool IsLegacyDNAAsset(const dna::Reader* DNAReader);
dna::ScopedPtr<dna::BinaryStreamReader> MigrateLegacyDNAAsset(dna::BoundedIOStream* Stream, const dna::Configuration& DNAConfig);
