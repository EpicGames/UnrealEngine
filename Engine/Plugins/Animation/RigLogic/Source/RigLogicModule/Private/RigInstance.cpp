// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigInstance.h"

#include "FMemoryResource.h"
#include "RigLogic.h"

#include "riglogic/RigLogic.h"

void FRigInstance::FRigInstanceDeleter::operator()(rl4::RigInstance* Pointer)
{
	rl4::RigInstance::destroy(Pointer);
}

FRigInstance::FRigInstance(FRigLogic* RigLogic) :
	MemoryResource{FMemoryResource::SharedInstance()},
	RigInstance{rl4::RigInstance::create(RigLogic->Unwrap(), MemoryResource.Get())}
{
}

FRigInstance::~FRigInstance() = default;

uint16 FRigInstance::GetGUIControlCount() const
{
	return RigInstance->getGUIControlCount();
}

float FRigInstance::GetGUIControl(uint16 Index) const
{
	return RigInstance->getGUIControl(Index);
}

void FRigInstance::SetGUIControl(uint16 Index, float Value)
{
	RigInstance->setGUIControl(Index, Value);
}

TArrayView<float> FRigInstance::GetGUIControlValues()
{
	rl4::ArrayView<float> Values = RigInstance->getGUIControlValues();
	return TArrayView<float>{Values.data(), static_cast<int32>(Values.size())};
}

TArrayView<const float> FRigInstance::GetGUIControlValues() const
{
	rl4::ConstArrayView<float> Values = RigInstance->getGUIControlValues();
	return TArrayView<const float>{Values.data(), static_cast<int32>(Values.size())};
}

uint16 FRigInstance::GetRawControlCount() const
{
	return RigInstance->getRawControlCount();
}

float FRigInstance::GetRawControl(uint16 Index) const
{
	return RigInstance->getRawControl(Index);
}

void FRigInstance::SetRawControl(uint16 Index, float Value)
{
	RigInstance->setRawControl(Index, Value);
}

TArrayView<float> FRigInstance::GetRawControlValues()
{
	rl4::ArrayView<float> Values = RigInstance->getRawControlValues();
	return TArrayView<float>{Values.data(), static_cast<int32>(Values.size())};
}

TArrayView<const float> FRigInstance::GetRawControlValues() const
{
	rl4::ConstArrayView<float> Values = RigInstance->getRawControlValues();
	return TArrayView<const float>{Values.data(), static_cast<int32>(Values.size())};
}

uint16 FRigInstance::GetPSDControlCount() const
{
	return RigInstance->getPSDControlCount();
}

float FRigInstance::GetPSDControl(uint16 Index) const
{
	return RigInstance->getPSDControl(Index);
}

TArrayView<const float> FRigInstance::GetPSDControlValues() const
{
	rl4::ConstArrayView<float> Values = RigInstance->getPSDControlValues();
	return TArrayView<const float>{Values.data(), static_cast<int32>(Values.size())};
}

uint16 FRigInstance::GetMLControlCount() const
{
	return RigInstance->getMLControlCount();
}

float FRigInstance::GetMLControl(uint16 Index) const
{
	return RigInstance->getMLControl(Index);
}

TArrayView<const float> FRigInstance::GetMLControlValues() const
{
	rl4::ConstArrayView<float> Values = RigInstance->getMLControlValues();
	return TArrayView<const float>{Values.data(), static_cast<int32>(Values.size())};
}

uint16 FRigInstance::GetMLTypeCount() const
{
	return RigInstance->getMLTypeCount();
}

uint16 FRigInstance::GetMLOperationSetCount(uint16 MLTypeIndex) const
{
	return RigInstance->getMLOperationSetCount(MLTypeIndex);
}

uint16 FRigInstance::GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return RigInstance->getMLOperationCount(MLTypeIndex, MLOperationSetIndex);
}

TArrayView<float> FRigInstance::GetMLMaskValues()
{
	rl4::ArrayView<float> Values = RigInstance->getMLMaskValues();
	return TArrayView<float>{Values.data(), static_cast<int32>(Values.size())};
}

TArrayView<const float> FRigInstance::GetMLMaskValues() const
{
	rl4::ConstArrayView<float> Values = RigInstance->getMLMaskValues();
	return TArrayView<const float>{Values.data(), static_cast<int32>(Values.size())};
}

uint16 FRigInstance::GetRBFControlCount() const
{
	return RigInstance->getRBFControlCount();
}

float FRigInstance::GetRBFControl(uint16 Index) const
{
	return RigInstance->getRBFControl(Index);
}

TArrayView<const float> FRigInstance::GetRBFControlValues() const
{
	rl4::ConstArrayView<float> Values = RigInstance->getRBFControlValues();
	return TArrayView<const float>{Values.data(), static_cast<int32>(Values.size())};
}

TArrayView<const float> FRigInstance::GetJointOutputs() const
{
	rl4::ConstArrayView<float> Outputs = RigInstance->getJointOutputs();
	return TArrayView<const float>{Outputs.data(), static_cast<int32>(Outputs.size())};
}

TArrayView<const float> FRigInstance::GetBlendShapeOutputs() const
{
	rl4::ConstArrayView<float> Outputs = RigInstance->getBlendShapeOutputs();
	return TArrayView<const float>{Outputs.data(), static_cast<int32>(Outputs.size())};
}

TArrayView<const float> FRigInstance::GetAnimatedMapOutputs() const
{
	rl4::ConstArrayView<float> Outputs = RigInstance->getAnimatedMapOutputs();
	return TArrayView<const float>{Outputs.data(), static_cast<int32>(Outputs.size())};
}

uint16 FRigInstance::GetLOD() const
{
	return RigInstance->getLOD();
}

void FRigInstance::SetLOD(uint16 LOD)
{
	RigInstance->setLOD(LOD);
}

rl4::RigInstance* FRigInstance::Unwrap() const
{
	return RigInstance.Get();
}
