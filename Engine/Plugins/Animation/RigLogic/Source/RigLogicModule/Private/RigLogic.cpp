// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogic.h"
 
#include "ArchiveMemoryStream.h"
#include "DNAReader.h"
#include "FMemoryResource.h"
#include "RigInstance.h"
#include "RigLogic.h"
#include "RigLogicModule.h"
#include "Stats/Stats.h"
#include "Stats/StatsHierarchical.h"
#include "Runtime/Core/Public/Async/ParallelFor.h"

#include "riglogic/RigLogic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigLogic)

#if STATS

DEFINE_STAT(STAT_RigLogic_CalculationType);
DEFINE_STAT(STAT_RigLogic_FloatingPointType);
DEFINE_STAT(STAT_RigLogic_MultiThreadMLCompute);
DEFINE_STAT(STAT_RigLogic_LOD);
DEFINE_STAT(STAT_RigLogic_RBFSolverCount);
DEFINE_STAT(STAT_RigLogic_MLOperationCount);
DEFINE_STAT(STAT_RigLogic_PSDCount);
DEFINE_STAT(STAT_RigLogic_JointCount);
DEFINE_STAT(STAT_RigLogic_JointDeltaValueCount);
DEFINE_STAT(STAT_RigLogic_BlendShapeChannelCount);
DEFINE_STAT(STAT_RigLogic_AnimatedMapCount);
DEFINE_STAT(STAT_RigLogic_MapGUIToRawControlsTime);
DEFINE_STAT(STAT_RigLogic_MapRawToGUIControlsTime);
DEFINE_STAT(STAT_RigLogic_CalculateRBFControlsTime);
DEFINE_STAT(STAT_RigLogic_CalculateMLControlsTime);
DEFINE_STAT(STAT_RigLogic_CalculatePSDControlsTime);
DEFINE_STAT(STAT_RigLogic_CalculateJointsTime);
DEFINE_STAT(STAT_RigLogic_CalculateBlendShapesTime);
DEFINE_STAT(STAT_RigLogic_CalculateAnimatedMapsTime);

#endif  // STATS

static bool IsRotationSequenceCompiledIn(ERotationSequence RotationSequence)
{
	switch (RotationSequence)
	{
#if defined(RL_BUILD_WITH_XYZ_ROTATION_ORDER)
	case ERotationSequence::XYZ: return true;
#endif
#if defined(RL_BUILD_WITH_XZY_ROTATION_ORDER)
	case ERotationSequence::XZY: return true;
#endif
#if defined(RL_BUILD_WITH_YXZ_ROTATION_ORDER)
	case ERotationSequence::YXZ: return true;
#endif
#if defined(RL_BUILD_WITH_YZX_ROTATION_ORDER)
	case ERotationSequence::YZX: return true;
#endif
#if defined(RL_BUILD_WITH_ZXY_ROTATION_ORDER)
	case ERotationSequence::ZXY: return true;
#endif
#if defined(RL_BUILD_WITH_ZYX_ROTATION_ORDER)
	case ERotationSequence::ZYX: return true;
#endif
	}
	return false;
}

static const TCHAR* LexToString(ERotationSequence RotationSequence)
{
	switch (RotationSequence)
	{
	case ERotationSequence::XYZ: return TEXT("XYZ");
	case ERotationSequence::XZY: return TEXT("XZY");
	case ERotationSequence::YXZ: return TEXT("YXZ");
	case ERotationSequence::YZX: return TEXT("YZX");
	case ERotationSequence::ZXY: return TEXT("ZXY");
	case ERotationSequence::ZYX: return TEXT("ZYX");
	}
	return TEXT("<unknown>");
}

static FString GetCompiledInRotationSequencesString()
{
	TArray<const TCHAR*, TInlineAllocator<6>> Compiled;
#if defined(RL_BUILD_WITH_XYZ_ROTATION_ORDER)
	Compiled.Add(TEXT("XYZ"));
#endif
#if defined(RL_BUILD_WITH_XZY_ROTATION_ORDER)
	Compiled.Add(TEXT("XZY"));
#endif
#if defined(RL_BUILD_WITH_YXZ_ROTATION_ORDER)
	Compiled.Add(TEXT("YXZ"));
#endif
#if defined(RL_BUILD_WITH_YZX_ROTATION_ORDER)
	Compiled.Add(TEXT("YZX"));
#endif
#if defined(RL_BUILD_WITH_ZXY_ROTATION_ORDER)
	Compiled.Add(TEXT("ZXY"));
#endif
#if defined(RL_BUILD_WITH_ZYX_ROTATION_ORDER)
	Compiled.Add(TEXT("ZYX"));
#endif
	return Compiled.Num() > 0 ? FString::Join(Compiled, TEXT(", ")) : FString(TEXT("<none>"));
}

void FRigLogicConfiguration::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// Migrate old plain enum to per-platform if the new field is still at default
			// and the old field has a non-default value
			if (CalculationTypePerPlatform.GetDefault() == static_cast<int32>(ERigLogicCalculationType::AnyVector) && CalculationType_DEPRECATED != ERigLogicCalculationType::AnyVector)
			{
				CalculationTypePerPlatform = FPerPlatformERigLogicCalculationType(CalculationType_DEPRECATED);
			}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FRigLogic::FRigLogicDeleter::operator()(rl4::RigLogic* Pointer)
{
	if (Pointer != nullptr)
	{
		rl4::RigLogic::destroy(Pointer);
	}
}

static ERigLogicFloatingPointType ResolveFloatingPointType(ERigLogicFloatingPointType InType)
{
	if (InType != ERigLogicFloatingPointType::Auto)
	{
		return InType;
	}

#ifdef RIGLOGIC_DEFAULT_HALF_FLOATS
	return ERigLogicFloatingPointType::HalfFloat;
#else
	return ERigLogicFloatingPointType::Float;
#endif
}

static rl4::Configuration ConvertToRigLogicConfig(const FRigLogicConfiguration& Config)
{
	rl4::Configuration Copy = {};
	Copy.calculationType = static_cast<rl4::CalculationType>(Config.CalculationTypePerPlatform.GetEnumValue());
	Copy.floatingPointType = static_cast<rl4::FloatingPointType>(ResolveFloatingPointType(Config.FloatingPointTypePerPlatform.GetEnumValue()));
	Copy.loadJoints = Config.LoadJoints;
	Copy.loadBlendShapes = Config.LoadBlendShapes;
	Copy.loadAnimatedMaps = Config.LoadAnimatedMaps;
	Copy.loadMachineLearnedBehavior = Config.LoadMachineLearnedBehavior;
	Copy.loadRBFBehavior = Config.LoadRBFBehavior;
	Copy.loadTwistSwingBehavior = Config.LoadTwistSwingBehavior;
	Copy.translationType = static_cast<rl4::TranslationType>(Config.TranslationType);
	Copy.rotationType = static_cast<rl4::RotationType>(Config.RotationType);
	Copy.scaleType = static_cast<rl4::ScaleType>(Config.ScaleType);
	Copy.translationPruningThreshold = Config.TranslationPruningThreshold;
	Copy.rotationPruningThreshold = Config.RotationPruningThreshold;
	Copy.scalePruningThreshold = Config.ScalePruningThreshold;
	return Copy;
}

static void UpdateAppliedRigLogicConfig(FRigLogicConfiguration& Config, const rl4::RigLogic* RigLogic)
{
	const rl4::Configuration& AppliedConfig = RigLogic->getConfiguration();
	Config.CalculationTypePerPlatform = FPerPlatformERigLogicCalculationType(static_cast<ERigLogicCalculationType>(AppliedConfig.calculationType));
	Config.FloatingPointTypePerPlatform = FPerPlatformERigLogicFloatingPointType(static_cast<ERigLogicFloatingPointType>(AppliedConfig.floatingPointType));
	Config.LoadJoints = AppliedConfig.loadJoints;
	Config.LoadBlendShapes = AppliedConfig.loadBlendShapes;
	Config.LoadAnimatedMaps = AppliedConfig.loadAnimatedMaps;
	Config.LoadMachineLearnedBehavior = AppliedConfig.loadMachineLearnedBehavior;
	Config.LoadRBFBehavior = AppliedConfig.loadRBFBehavior;
	Config.LoadTwistSwingBehavior = AppliedConfig.loadTwistSwingBehavior;
	Config.TranslationType = static_cast<ERigLogicTranslationType>(AppliedConfig.translationType);
	Config.RotationType = static_cast<ERigLogicRotationType>(AppliedConfig.rotationType);
	Config.ScaleType = static_cast<ERigLogicScaleType>(AppliedConfig.scaleType);
	Config.TranslationPruningThreshold = AppliedConfig.translationPruningThreshold;
	Config.RotationPruningThreshold = AppliedConfig.rotationPruningThreshold;
	Config.ScalePruningThreshold = AppliedConfig.scalePruningThreshold;
}

FRigLogic::FRigLogic(const IDNAReader* Reader, const FRigLogicConfiguration& Config) :
	MemoryResource{FMemoryResource::SharedInstance()},
	RigLogic{rl4::RigLogic::create(Reader->Unwrap(), ConvertToRigLogicConfig(Config), MemoryResource.Get())},
	Configuration{Config},
	CachedEnableMultiThreadMLCompute{Configuration.EnableMultiThreadMLComputePerPlatform.GetValue()}
{
	UpdateAppliedRigLogicConfig(Configuration, RigLogic.Get());
	// RigLogicLib only compiles rotation-sequence templates that are explicitly enabled
	// via RL_BUILD_WITH_<ORDER>_ROTATION_ORDER. If a rotation-sequence for which support
	// was not compiled-in is chosen, the joint evaluator of RigLogic will output no
	// deltas at all (null evaluator, works but no output).
	const ERotationSequence RotationSequence = Reader->GetRotationSequence();
	if (!IsRotationSequenceCompiledIn(RotationSequence))
	{
		UE_LOGF(LogRigLogic, Warning,
			"DNA rotation sequence '%ls' is not compiled into this RigLogicLib build. "
			"Compiled-in rotation sequences: [%ls]. No joint deltas will be output by RigLogic.",
			LexToString(RotationSequence),
			*GetCompiledInRotationSequencesString());
	}
}

FRigLogic::FRigLogic(FArchive* Archive, const FRigLogicConfiguration& Config) :
	MemoryResource{FMemoryResource::SharedInstance()},
	RigLogic{nullptr},
	Configuration{Config},
	CachedEnableMultiThreadMLCompute{Configuration.EnableMultiThreadMLComputePerPlatform.GetValue()}
{
	FArchiveMemoryStream Stream(Archive);
	RigLogic.Reset(rl4::RigLogic::restore(&Stream, FMemoryResource::Instance()));
	UpdateAppliedRigLogicConfig(Configuration, RigLogic.Get());
}

FRigLogic::~FRigLogic() = default;

void FRigLogic::Dump(FArchive* Archive) const
{
	FArchiveMemoryStream Stream(Archive);
	RigLogic->dump(&Stream);
}

const FRigLogicConfiguration& FRigLogic::GetConfiguration() const
{
	return Configuration;
}

uint16 FRigLogic::GetLODCount() const
{
	return RigLogic->getLODCount();
}

TArrayView<const uint16> FRigLogic::GetRBFSolverIndicesForLOD(uint16 LOD) const
{
	rl4::ConstArrayView<uint16> Indices = RigLogic->getRBFSolverIndicesForLOD(LOD);
	return TArrayView<const uint16>{Indices.data(), static_cast<int32>(Indices.size())};
}

TArrayView<const uint16> FRigLogic::GetMLOperationIndicesForLOD(uint16 LOD, uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	rl4::ConstArrayView<uint16> Indices = RigLogic->getMLOperationIndicesForLOD(LOD, MLTypeIndex, MLOperationSetIndex);
	return TArrayView<const uint16>{Indices.data(), static_cast<int32>(Indices.size())};
}

TArrayView<const uint16> FRigLogic::GetBlendShapeChannelIndicesForLOD(uint16 LOD) const
{
	rl4::ConstArrayView<uint16> Indices = RigLogic->getBlendShapeChannelIndicesForLOD(LOD);
	return TArrayView<const uint16>{Indices.data(), static_cast<int32>(Indices.size())};
}

TArrayView<const uint16> FRigLogic::GetAnimatedMapIndicesForLOD(uint16 LOD) const
{
	rl4::ConstArrayView<uint16> Indices = RigLogic->getAnimatedMapIndicesForLOD(LOD);
	return TArrayView<const uint16>{Indices.data(), static_cast<int32>(Indices.size())};
}

TArrayView<const uint16> FRigLogic::GetJointIndicesForLOD(uint16 LOD) const
{
	rl4::ConstArrayView<uint16> Indices = RigLogic->getJointIndicesForLOD(LOD);
	return TArrayView<const uint16>{Indices.data(), static_cast<int32>(Indices.size())};
}

TArrayView<const float> FRigLogic::GetNeutralJointValues() const
{
	rl4::ConstArrayView<float> Values = RigLogic->getNeutralJointValues();
	return TArrayView<const float>{Values.data(), static_cast<int32>(Values.size())};
}

TArrayView<const uint16> FRigLogic::GetJointVariableAttributeIndices(uint16 LOD) const
{
	rl4::ConstArrayView<uint16> Indices = RigLogic->getJointVariableAttributeIndices(LOD);
	return TArrayView<const uint16>{Indices.data(), static_cast<int32>(Indices.size())};
}

uint16 FRigLogic::GetJointGroupCount() const
{
	return RigLogic->getJointGroupCount();
}

uint16 FRigLogic::GetRBFSolverCount() const
{
	return RigLogic->getRBFSolverCount();
}

uint16 FRigLogic::GetTwistCount() const
{
	return RigLogic->getTwistCount();
}

uint16 FRigLogic::GetSwingCount() const
{
	return RigLogic->getSwingCount();
}

uint16 FRigLogic::GetMeshCount() const
{
	return RigLogic->getMeshCount();
}

uint16 FRigLogic::GetMeshRegionCount(uint16 MeshIndex) const
{
	return RigLogic->getMeshRegionCount(MeshIndex);
}

uint16 FRigLogic::GetMLTypeCount() const
{
	return RigLogic->getMLTypeCount();
}

uint16 FRigLogic::GetMLOperationSetCount(uint16 MLTypeIndex) const
{
	return RigLogic->getMLOperationSetCount(MLTypeIndex);
}

uint16 FRigLogic::GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return RigLogic->getMLOperationCount(MLTypeIndex, MLOperationSetIndex);
}

void FRigLogic::MapGUIToRawControls(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_MapGUIToRawControlsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::MapGUIToRawControls");
#endif  // CPUPROFILERTRACE_ENABLED
	RigLogic->mapGUIToRawControls(Instance->Unwrap());
}

void FRigLogic::MapRawToGUIControls(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_MapRawToGUIControlsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::MapRawToGUIControls");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->mapRawToGUIControls(Instance->Unwrap());
}

void FRigLogic::CalculateMLControls(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateMLControlsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateMLControls");
#endif  // CPUPROFILERTRACE_ENABLED

	if (CachedEnableMultiThreadMLCompute)
	{
		const uint16 MLTypeCount = RigLogic->getMLTypeCount();
		const uint16 LOD = Instance->GetLOD();
		for (uint16 MLTypeIndex = {}; MLTypeIndex < MLTypeCount; ++MLTypeIndex)
		{
			const uint16 MLOperationSetCount = RigLogic->getMLOperationSetCount(MLTypeIndex);
			for (uint16 MLOperationSetIndex = {}; MLOperationSetIndex < MLOperationSetCount; ++MLOperationSetIndex)
			{
				rl4::ConstArrayView<uint16> MLOperationIndices = RigLogic->getMLOperationIndicesForLOD(LOD, MLTypeIndex, MLOperationSetIndex);
				ParallelFor(MLOperationIndices.size(), [this, Instance, MLTypeIndex, MLOperationSetIndex, MLOperationIndices](int32 i)
					{
						RigLogic->calculateMLControls(Instance->Unwrap(), MLTypeIndex, MLOperationSetIndex, MLOperationIndices[i]);
					});
			}
		}
	}
	else
	{
		RigLogic->calculateMLControls(Instance->Unwrap());
	}
}

void FRigLogic::CalculateMLControls(FRigInstance* Instance, uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateMLControlsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateMLControls");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculateMLControls(Instance->Unwrap(), MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

void FRigLogic::CalculateRBFControls(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateRBFControlsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateRBFControls");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculateRBFControls(Instance->Unwrap());
}

void FRigLogic::CalculateRBFControls(FRigInstance* Instance, uint16 SolverIndex) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateRBFControlsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateRBFControls");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculateRBFControls(Instance->Unwrap(), SolverIndex);
}

void FRigLogic::CalculatePSDControls(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculatePSDControlsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculatePSDControls");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculatePSDControls(Instance->Unwrap());
}

void FRigLogic::CalculateJoints(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateJointsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateJoints");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculateJoints(Instance->Unwrap());
}

void FRigLogic::CalculateJoints(FRigInstance* Instance, uint16 JointGroupIndex) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateJointsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateJoints");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculateJoints(Instance->Unwrap(), JointGroupIndex);
}

void FRigLogic::CalculateBlendShapes(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateBlendShapesTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateBlendShapes");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculateBlendShapes(Instance->Unwrap());
}

void FRigLogic::CalculateAnimatedMaps(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	SCOPE_CYCLE_COUNTER(STAT_RigLogic_CalculateAnimatedMapsTime);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::CalculateAnimatedMaps");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculateAnimatedMaps(Instance->Unwrap());
}

void FRigLogic::Calculate(FRigInstance* Instance) const
{
#if STATS
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigLogic_Calculate);
#endif  // STATS
#if CPUPROFILERTRACE_ENABLED
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigLogic::Calculate");
#endif  // CPUPROFILERTRACE_ENABLED

	RigLogic->calculate(Instance->Unwrap());
}

void FRigLogic::CollectCalculationStats(FRigInstance* Instance) const
{
#if STATS
	rl4::Stats Stats = {};
	RigLogic->collectCalculationStats(Instance->Unwrap(), &Stats);
	SET_DWORD_STAT(STAT_RigLogic_CalculationType, Stats.calculationType);
	SET_DWORD_STAT(STAT_RigLogic_FloatingPointType, Stats.floatingPointType);
	SET_DWORD_STAT(STAT_RigLogic_MultiThreadMLCompute, CachedEnableMultiThreadMLCompute);
	SET_DWORD_STAT(STAT_RigLogic_LOD, Instance->GetLOD());
	SET_DWORD_STAT(STAT_RigLogic_RBFSolverCount, Stats.rbfSolverCount);
	SET_DWORD_STAT(STAT_RigLogic_MLOperationCount, Stats.mlOperationCount);
	SET_DWORD_STAT(STAT_RigLogic_PSDCount, Stats.psdCount);
	SET_DWORD_STAT(STAT_RigLogic_JointCount, Stats.jointCount);
	SET_DWORD_STAT(STAT_RigLogic_JointDeltaValueCount, Stats.jointDeltaValueCount);
	SET_DWORD_STAT(STAT_RigLogic_BlendShapeChannelCount, Stats.blendShapeChannelCount);
	SET_DWORD_STAT(STAT_RigLogic_AnimatedMapCount, Stats.animatedMapCount);
#endif  // STATS
}

rl4::RigLogic* FRigLogic::Unwrap() const
{
	return RigLogic.Get();
}
