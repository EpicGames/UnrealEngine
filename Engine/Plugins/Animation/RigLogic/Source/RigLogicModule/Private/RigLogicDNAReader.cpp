// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicDNAReader.h"

RigLogicDNAReader::RigLogicDNAReader(const dna::Reader* DNAReader) : Reader{DNAReader}
{
}

// Header
std::uint16_t RigLogicDNAReader::getFileFormatGeneration() const
{
	return Reader->getFileFormatGeneration();
}
std::uint16_t RigLogicDNAReader::getFileFormatVersion() const
{
	return Reader->getFileFormatVersion();
}

// Descriptor
dna::StringView RigLogicDNAReader::getName() const
{
	return Reader->getName();
}
dna::Archetype RigLogicDNAReader::getArchetype() const
{
	return Reader->getArchetype();
}
dna::Gender RigLogicDNAReader::getGender() const
{
	return Reader->getGender();
}
std::uint16_t RigLogicDNAReader::getAge() const
{
	return Reader->getAge();
}
std::uint32_t RigLogicDNAReader::getMetaDataCount() const
{
	return Reader->getMetaDataCount();
}
dna::StringView RigLogicDNAReader::getMetaDataKey(std::uint32_t index) const
{
	return Reader->getMetaDataKey(index);
}
dna::StringView RigLogicDNAReader::getMetaDataValue(const char* key) const
{
	return Reader->getMetaDataValue(key);
}
dna::TranslationUnit RigLogicDNAReader::getTranslationUnit() const
{
	return Reader->getTranslationUnit();
}
dna::RotationUnit RigLogicDNAReader::getRotationUnit() const
{
	return Reader->getRotationUnit();
}
dna::CoordinateSystem RigLogicDNAReader::getCoordinateSystem() const
{
	return Reader->getCoordinateSystem();
}
std::uint16_t RigLogicDNAReader::getLODCount() const
{
	return Reader->getLODCount();
}
std::uint16_t RigLogicDNAReader::getDBMaxLOD() const
{
	return Reader->getDBMaxLOD();
}
dna::StringView RigLogicDNAReader::getDBComplexity() const
{
	return Reader->getDBComplexity();
}
dna::StringView RigLogicDNAReader::getDBName() const
{
	return Reader->getDBName();
}

// Definition
std::uint16_t RigLogicDNAReader::getGUIControlCount() const
{
	return Reader->getGUIControlCount();
}
dna::StringView RigLogicDNAReader::getGUIControlName(std::uint16_t index) const
{
	return Reader->getGUIControlName(index);
}
std::uint16_t RigLogicDNAReader::getRawControlCount() const
{
	return Reader->getRawControlCount();
}
dna::StringView RigLogicDNAReader::getRawControlName(std::uint16_t index) const
{
	return Reader->getRawControlName(index);
}
std::uint16_t RigLogicDNAReader::getJointCount() const
{
	return Reader->getJointCount();
}
dna::StringView RigLogicDNAReader::getJointName(std::uint16_t index) const
{
	return Reader->getJointName(index);
}
std::uint16_t RigLogicDNAReader::getJointIndexListCount() const
{
	return Reader->getJointIndexListCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getJointIndicesForLOD(std::uint16_t lod) const
{
	return Reader->getJointIndicesForLOD(lod);
}
std::uint16_t RigLogicDNAReader::getJointParentIndex(std::uint16_t index) const
{
	return Reader->getJointParentIndex(index);
}
std::uint16_t RigLogicDNAReader::getBlendShapeChannelCount() const
{
	return Reader->getBlendShapeChannelCount();
}
dna::StringView RigLogicDNAReader::getBlendShapeChannelName(std::uint16_t index) const
{
	return Reader->getBlendShapeChannelName(index);
}
std::uint16_t RigLogicDNAReader::getBlendShapeChannelIndexListCount() const
{
	return Reader->getBlendShapeChannelIndexListCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getBlendShapeChannelIndicesForLOD(std::uint16_t lod) const
{
	return Reader->getBlendShapeChannelIndicesForLOD(lod);
}
std::uint16_t RigLogicDNAReader::getAnimatedMapCount() const
{
	return Reader->getAnimatedMapCount();
}
dna::StringView RigLogicDNAReader::getAnimatedMapName(std::uint16_t index) const
{
	return Reader->getAnimatedMapName(index);
}
std::uint16_t RigLogicDNAReader::getAnimatedMapIndexListCount() const
{
	return Reader->getAnimatedMapIndexListCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getAnimatedMapIndicesForLOD(std::uint16_t lod) const
{
	return Reader->getAnimatedMapIndicesForLOD(lod);
}
std::uint16_t RigLogicDNAReader::getMeshCount() const
{
	return Reader->getMeshCount();
}
dna::StringView RigLogicDNAReader::getMeshName(std::uint16_t index) const
{
	return Reader->getMeshName(index);
}
std::uint16_t RigLogicDNAReader::getMeshIndexListCount() const
{
	return Reader->getMeshIndexListCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getMeshIndicesForLOD(std::uint16_t lod) const
{
	return Reader->getMeshIndicesForLOD(lod);
}
std::uint16_t RigLogicDNAReader::getMeshBlendShapeChannelMappingCount() const
{
	return Reader->getMeshBlendShapeChannelMappingCount();
}
dna::MeshBlendShapeChannelMapping RigLogicDNAReader::getMeshBlendShapeChannelMapping(std::uint16_t index) const
{
	return Reader->getMeshBlendShapeChannelMapping(index);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getMeshBlendShapeChannelMappingIndicesForLOD(std::uint16_t lod) const
{
	return Reader->getMeshBlendShapeChannelMappingIndicesForLOD(lod);
}
dna::Vector3 RigLogicDNAReader::getNeutralJointTranslation(std::uint16_t index) const
{
	return Reader->getNeutralJointTranslation(index);
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeutralJointTranslationXs() const
{
	return Reader->getNeutralJointTranslationXs();
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeutralJointTranslationYs() const
{
	return Reader->getNeutralJointTranslationYs();
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeutralJointTranslationZs() const
{
	return Reader->getNeutralJointTranslationZs();
}
dna::Vector3 RigLogicDNAReader::getNeutralJointRotation(std::uint16_t index) const
{
	return Reader->getNeutralJointRotation(index);
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeutralJointRotationXs() const
{
	return Reader->getNeutralJointRotationXs();
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeutralJointRotationYs() const
{
	return Reader->getNeutralJointRotationYs();
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeutralJointRotationZs() const
{
	return Reader->getNeutralJointRotationZs();
}

// Behavior
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getGUIToRawInputIndices() const
{
	return Reader->getGUIToRawInputIndices();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getGUIToRawOutputIndices() const
{
	return Reader->getGUIToRawOutputIndices();
}
dna::ConstArrayView<float> RigLogicDNAReader::getGUIToRawFromValues() const
{
	return Reader->getGUIToRawFromValues();
}
dna::ConstArrayView<float> RigLogicDNAReader::getGUIToRawToValues() const
{
	return Reader->getGUIToRawToValues();
}
dna::ConstArrayView<float> RigLogicDNAReader::getGUIToRawSlopeValues() const
{
	return Reader->getGUIToRawSlopeValues();
}
dna::ConstArrayView<float> RigLogicDNAReader::getGUIToRawCutValues() const
{
	return Reader->getGUIToRawCutValues();
}
std::uint16_t RigLogicDNAReader::getPSDCount() const
{
	return Reader->getPSDCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getPSDRowIndices() const
{
	return Reader->getPSDRowIndices();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getPSDColumnIndices() const
{
	return Reader->getPSDColumnIndices();
}
dna::ConstArrayView<float> RigLogicDNAReader::getPSDValues() const
{
	return Reader->getPSDValues();
}
std::uint16_t RigLogicDNAReader::getJointRowCount() const
{
	return Reader->getJointRowCount();
}
std::uint16_t RigLogicDNAReader::getJointColumnCount() const
{
	return Reader->getJointColumnCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getJointVariableAttributeIndices(std::uint16_t lod) const
{
	return Reader->getJointVariableAttributeIndices(lod);
}
std::uint16_t RigLogicDNAReader::getJointGroupCount() const
{
	return Reader->getJointGroupCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getJointGroupLODs(std::uint16_t jointGroupIndex) const
{
	return Reader->getJointGroupLODs(jointGroupIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getJointGroupInputIndices(std::uint16_t jointGroupIndex) const
{
	return Reader->getJointGroupInputIndices(jointGroupIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getJointGroupOutputIndices(std::uint16_t jointGroupIndex) const
{
	return Reader->getJointGroupOutputIndices(jointGroupIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getJointGroupValues(std::uint16_t jointGroupIndex) const
{
	return Reader->getJointGroupValues(jointGroupIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getJointGroupJointIndices(std::uint16_t jointGroupIndex) const
{
	return Reader->getJointGroupJointIndices(jointGroupIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getBlendShapeChannelLODs() const
{
	return Reader->getBlendShapeChannelLODs();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getBlendShapeChannelInputIndices() const
{
	return Reader->getBlendShapeChannelInputIndices();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getBlendShapeChannelOutputIndices() const
{
	return Reader->getBlendShapeChannelOutputIndices();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getAnimatedMapLODs() const
{
	return Reader->getAnimatedMapLODs();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getAnimatedMapInputIndices() const
{
	return Reader->getAnimatedMapInputIndices();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getAnimatedMapOutputIndices() const
{
	return Reader->getAnimatedMapOutputIndices();
}
dna::ConstArrayView<float> RigLogicDNAReader::getAnimatedMapFromValues() const
{
	return Reader->getAnimatedMapFromValues();
}
dna::ConstArrayView<float> RigLogicDNAReader::getAnimatedMapToValues() const
{
	return Reader->getAnimatedMapToValues();
}
dna::ConstArrayView<float> RigLogicDNAReader::getAnimatedMapSlopeValues() const
{
	return Reader->getAnimatedMapSlopeValues();
}
dna::ConstArrayView<float> RigLogicDNAReader::getAnimatedMapCutValues() const
{
	return Reader->getAnimatedMapCutValues();
}

// Geometry
std::uint32_t RigLogicDNAReader::getVertexPositionCount(std::uint16_t meshIndex) const
{
	return Reader->getVertexPositionCount(meshIndex);
}
dna::Position RigLogicDNAReader::getVertexPosition(std::uint16_t meshIndex, std::uint32_t vertexIndex) const
{
	return Reader->getVertexPosition(meshIndex, vertexIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexPositionXs(std::uint16_t meshIndex) const
{
	return Reader->getVertexPositionXs(meshIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexPositionYs(std::uint16_t meshIndex) const
{
	return Reader->getVertexPositionYs(meshIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexPositionZs(std::uint16_t meshIndex) const
{
	return Reader->getVertexPositionZs(meshIndex);
}
std::uint32_t RigLogicDNAReader::getVertexTextureCoordinateCount(std::uint16_t meshIndex) const
{
	return Reader->getVertexTextureCoordinateCount(meshIndex);
}
dna::TextureCoordinate RigLogicDNAReader::getVertexTextureCoordinate(std::uint16_t meshIndex, std::uint32_t textureCoordinateIndex) const
{
	return Reader->getVertexTextureCoordinate(meshIndex, textureCoordinateIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexTextureCoordinateUs(std::uint16_t meshIndex) const
{
	return Reader->getVertexTextureCoordinateUs(meshIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexTextureCoordinateVs(std::uint16_t meshIndex) const
{
	return Reader->getVertexTextureCoordinateVs(meshIndex);
}
std::uint32_t RigLogicDNAReader::getVertexNormalCount(std::uint16_t meshIndex) const
{
	return Reader->getVertexNormalCount(meshIndex);
}
dna::Normal RigLogicDNAReader::getVertexNormal(std::uint16_t meshIndex, std::uint32_t normalIndex) const
{
	return Reader->getVertexNormal(meshIndex, normalIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexNormalXs(std::uint16_t meshIndex) const
{
	return Reader->getVertexNormalXs(meshIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexNormalYs(std::uint16_t meshIndex) const
{
	return Reader->getVertexNormalYs(meshIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getVertexNormalZs(std::uint16_t meshIndex) const
{
	return Reader->getVertexNormalZs(meshIndex);
}
std::uint32_t RigLogicDNAReader::getVertexLayoutCount(std::uint16_t meshIndex) const
{
	return Reader->getVertexLayoutCount(meshIndex);
}
dna::VertexLayout RigLogicDNAReader::getVertexLayout(std::uint16_t meshIndex, std::uint32_t layoutIndex) const
{
	return Reader->getVertexLayout(meshIndex, layoutIndex);
}
dna::ConstArrayView<std::uint32_t> RigLogicDNAReader::getVertexLayoutPositionIndices(std::uint16_t meshIndex) const
{
	return Reader->getVertexLayoutPositionIndices(meshIndex);
}
dna::ConstArrayView<std::uint32_t> RigLogicDNAReader::getVertexLayoutTextureCoordinateIndices(std::uint16_t meshIndex) const
{
	return Reader->getVertexLayoutTextureCoordinateIndices(meshIndex);
}
dna::ConstArrayView<std::uint32_t> RigLogicDNAReader::getVertexLayoutNormalIndices(std::uint16_t meshIndex) const
{
	return Reader->getVertexLayoutNormalIndices(meshIndex);
}
std::uint32_t RigLogicDNAReader::getFaceCount(std::uint16_t meshIndex) const
{
	return Reader->getFaceCount(meshIndex);
}
dna::ConstArrayView<std::uint32_t> RigLogicDNAReader::getFaceVertexLayoutIndices(std::uint16_t meshIndex, std::uint32_t faceIndex) const
{
	return Reader->getFaceVertexLayoutIndices(meshIndex, faceIndex);
}
std::uint16_t RigLogicDNAReader::getMaximumInfluencePerVertex(std::uint16_t meshIndex) const
{
	return Reader->getMaximumInfluencePerVertex(meshIndex);
}
std::uint32_t RigLogicDNAReader::getSkinWeightsCount(std::uint16_t meshIndex) const
{
	return Reader->getSkinWeightsCount(meshIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getSkinWeightsValues(std::uint16_t meshIndex, std::uint32_t vertexIndex) const
{
	return Reader->getSkinWeightsValues(meshIndex, vertexIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getSkinWeightsJointIndices(std::uint16_t meshIndex, std::uint32_t vertexIndex) const
{
	return Reader->getSkinWeightsJointIndices(meshIndex, vertexIndex);
}
std::uint16_t RigLogicDNAReader::getBlendShapeTargetCount(std::uint16_t meshIndex) const
{
	return Reader->getBlendShapeTargetCount(meshIndex);
}
std::uint16_t RigLogicDNAReader::getBlendShapeChannelIndex(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const
{
	return Reader->getBlendShapeChannelIndex(meshIndex, blendShapeTargetIndex);
}
std::uint32_t RigLogicDNAReader::getBlendShapeTargetDeltaCount(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const
{
	return Reader->getBlendShapeTargetDeltaCount(meshIndex, blendShapeTargetIndex);
}
dna::Delta RigLogicDNAReader::getBlendShapeTargetDelta(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex, std::uint32_t deltaIndex) const
{
	return Reader->getBlendShapeTargetDelta(meshIndex, blendShapeTargetIndex, deltaIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getBlendShapeTargetDeltaXs(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const
{
	return Reader->getBlendShapeTargetDeltaXs(meshIndex, blendShapeTargetIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getBlendShapeTargetDeltaYs(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const
{
	return Reader->getBlendShapeTargetDeltaYs(meshIndex, blendShapeTargetIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getBlendShapeTargetDeltaZs(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const
{
	return Reader->getBlendShapeTargetDeltaZs(meshIndex, blendShapeTargetIndex);
}
dna::ConstArrayView<std::uint32_t> RigLogicDNAReader::getBlendShapeTargetVertexIndices(std::uint16_t meshIndex, std::uint16_t blendShapeTargetIndex) const
{
	return Reader->getBlendShapeTargetVertexIndices(meshIndex, blendShapeTargetIndex);
}

// Machine Learned Behavior
std::uint16_t RigLogicDNAReader::getMLControlCount() const
{
	return Reader->getMLControlCount();
}
dna::StringView RigLogicDNAReader::getMLControlName(std::uint16_t index) const
{
	return Reader->getMLControlName(index);
}
std::uint16_t RigLogicDNAReader::getNeuralNetworkCount() const
{
	return Reader->getNeuralNetworkCount();
}
std::uint16_t RigLogicDNAReader::getNeuralNetworkIndexListCount() const
{
	return Reader->getNeuralNetworkIndexListCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getNeuralNetworkIndicesForLOD(std::uint16_t lod) const
{
	return Reader->getNeuralNetworkIndicesForLOD(lod);
}
std::uint16_t RigLogicDNAReader::getMeshRegionCount(std::uint16_t meshIndex) const
{
	return Reader->getMeshRegionCount(meshIndex);
}
dna::StringView RigLogicDNAReader::getMeshRegionName(std::uint16_t meshIndex, std::uint16_t regionIndex) const
{
	return Reader->getMeshRegionName(meshIndex, regionIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getNeuralNetworkIndicesForMeshRegion(std::uint16_t meshIndex, std::uint16_t regionIndex) const
{
	return Reader->getNeuralNetworkIndicesForMeshRegion(meshIndex, regionIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getNeuralNetworkInputIndices(std::uint16_t netIndex) const
{
	return Reader->getNeuralNetworkInputIndices(netIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getNeuralNetworkOutputIndices(std::uint16_t netIndex) const
{
	return Reader->getNeuralNetworkOutputIndices(netIndex);
}
std::uint16_t RigLogicDNAReader::getNeuralNetworkLayerCount(std::uint16_t netIndex) const
{
	return Reader->getNeuralNetworkLayerCount(netIndex);
}
dna::ActivationFunction RigLogicDNAReader::getNeuralNetworkLayerActivationFunction(std::uint16_t netIndex, std::uint16_t layerIndex) const
{
	return Reader->getNeuralNetworkLayerActivationFunction(netIndex, layerIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeuralNetworkLayerActivationFunctionParameters(std::uint16_t netIndex, std::uint16_t layerIndex) const
{
	return Reader->getNeuralNetworkLayerActivationFunctionParameters(netIndex, layerIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeuralNetworkLayerBiases(std::uint16_t netIndex, std::uint16_t layerIndex) const
{
	return Reader->getNeuralNetworkLayerBiases(netIndex, layerIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getNeuralNetworkLayerWeights(std::uint16_t netIndex, std::uint16_t layerIndex) const
{
	return Reader->getNeuralNetworkLayerWeights(netIndex, layerIndex);
}

// RBFBehaviorReader methods
std::uint16_t RigLogicDNAReader::getRBFPoseCount() const
{
	return Reader->getRBFPoseCount();
}
dna::StringView RigLogicDNAReader::getRBFPoseName(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseName(poseIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFPoseJointOutputIndices(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseJointOutputIndices(poseIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFPoseBlendShapeChannelOutputIndices(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseBlendShapeChannelOutputIndices(poseIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFPoseAnimatedMapOutputIndices(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseAnimatedMapOutputIndices(poseIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getRBFPoseJointOutputValues(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseJointOutputValues(poseIndex);
}
float RigLogicDNAReader::getRBFPoseScale(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseScale(poseIndex);
}
std::uint16_t RigLogicDNAReader::getRBFPoseControlCount() const
{
	return Reader->getRBFPoseControlCount();
}
dna::StringView RigLogicDNAReader::getRBFPoseControlName(std::uint16_t poseControlIndex) const
{
	return Reader->getRBFPoseControlName(poseControlIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFPoseInputControlIndices(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseInputControlIndices(poseIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFPoseOutputControlIndices(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseOutputControlIndices(poseIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getRBFPoseOutputControlWeights(std::uint16_t poseIndex) const
{
	return Reader->getRBFPoseOutputControlWeights(poseIndex);
}
std::uint16_t RigLogicDNAReader::getRBFSolverCount() const
{
	return Reader->getRBFSolverCount();
}
std::uint16_t RigLogicDNAReader::getRBFSolverIndexListCount() const
{
	return Reader->getRBFSolverIndexListCount();
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFSolverIndicesForLOD(std::uint16_t lod) const
{
	return Reader->getRBFSolverIndicesForLOD(lod);
}
dna::StringView RigLogicDNAReader::getRBFSolverName(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverName(solverIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFSolverRawControlIndices(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverRawControlIndices(solverIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getRBFSolverPoseIndices(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverPoseIndices(solverIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getRBFSolverRawControlValues(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverRawControlValues(solverIndex);
}
dna::RBFSolverType RigLogicDNAReader::getRBFSolverType(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverType(solverIndex);
}
float RigLogicDNAReader::getRBFSolverRadius(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverRadius(solverIndex);
}
dna::AutomaticRadius RigLogicDNAReader::getRBFSolverAutomaticRadius(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverAutomaticRadius(solverIndex);
}
float RigLogicDNAReader::getRBFSolverWeightThreshold(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverWeightThreshold(solverIndex);
}
dna::RBFDistanceMethod RigLogicDNAReader::getRBFSolverDistanceMethod(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverDistanceMethod(solverIndex);
}
dna::RBFNormalizeMethod RigLogicDNAReader::getRBFSolverNormalizeMethod(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverNormalizeMethod(solverIndex);
}
dna::RBFFunctionType RigLogicDNAReader::getRBFSolverFunctionType(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverFunctionType(solverIndex);
}
dna::TwistAxis RigLogicDNAReader::getRBFSolverTwistAxis(std::uint16_t solverIndex) const
{
	return Reader->getRBFSolverTwistAxis(solverIndex);
}

// JointBehaviorMetadataReader methods
dna::TranslationRepresentation RigLogicDNAReader::getJointTranslationRepresentation(std::uint16_t jointIndex) const
{
	return Reader->getJointTranslationRepresentation(jointIndex);
}
dna::RotationRepresentation RigLogicDNAReader::getJointRotationRepresentation(std::uint16_t jointIndex) const
{
	return Reader->getJointRotationRepresentation(jointIndex);
}
dna::ScaleRepresentation RigLogicDNAReader::getJointScaleRepresentation(std::uint16_t jointIndex) const
{
	return Reader->getJointScaleRepresentation(jointIndex);
}

// TwistSwingBehaviorReader methods
std::uint16_t RigLogicDNAReader::getTwistCount() const
{
	return Reader->getTwistCount();
}
dna::TwistAxis RigLogicDNAReader::getTwistSetupTwistAxis(std::uint16_t twistIndex) const
{
	return Reader->getTwistSetupTwistAxis(twistIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getTwistInputControlIndices(std::uint16_t twistIndex) const
{
	return Reader->getTwistInputControlIndices(twistIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getTwistOutputJointIndices(std::uint16_t twistIndex) const
{
	return Reader->getTwistOutputJointIndices(twistIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getTwistBlendWeights(std::uint16_t twistIndex) const
{
	return Reader->getTwistBlendWeights(twistIndex);
}
std::uint16_t RigLogicDNAReader::getSwingCount() const
{
	return Reader->getSwingCount();
}
dna::TwistAxis RigLogicDNAReader::getSwingSetupTwistAxis(std::uint16_t swingIndex) const
{
	return Reader->getSwingSetupTwistAxis(swingIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getSwingInputControlIndices(std::uint16_t swingIndex) const
{
	return Reader->getSwingInputControlIndices(swingIndex);
}
dna::ConstArrayView<std::uint16_t> RigLogicDNAReader::getSwingOutputJointIndices(std::uint16_t swingIndex) const
{
	return Reader->getSwingOutputJointIndices(swingIndex);
}
dna::ConstArrayView<float> RigLogicDNAReader::getSwingBlendWeights(std::uint16_t swingIndex) const
{
	return Reader->getSwingBlendWeights(swingIndex);
}

// MachineLearnedBehaviorExtReader methods
uint16 RigLogicDNAReader::getMLTypeCount() const
{
	return Reader->getMLTypeCount();
}

uint16 RigLogicDNAReader::getMLOperationSetCount(uint16 mlTypeIndex) const
{
	return Reader->getMLOperationSetCount(mlTypeIndex);
}

uint16 RigLogicDNAReader::getMLOperationCount(uint16 mlTypeIndex, uint16 mlOperationSetIndex) const
{
	return Reader->getMLOperationCount(mlTypeIndex, mlOperationSetIndex);
}

dna::MachineLearnedBehaviorOperationType RigLogicDNAReader::getMLOperationType(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const
{
	return Reader->getMLOperationType(mlTypeIndex, mlOperationSetIndex, mlOperationIndex);
}

dna::ConstArrayView<uint32> RigLogicDNAReader::getMLOperationParameters(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const
{
	return Reader->getMLOperationParameters(mlTypeIndex, mlOperationSetIndex, mlOperationIndex);
}

dna::ConstArrayView<uint16> RigLogicDNAReader::getMLOperationDependencyOperationSetIndices(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const
{
	return Reader->getMLOperationDependencyOperationSetIndices(mlTypeIndex, mlOperationSetIndex, mlOperationIndex);
}

dna::ConstArrayView<uint16> RigLogicDNAReader::getMLOperationDependencyOperationIndices(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const
{
	return Reader->getMLOperationDependencyOperationIndices(mlTypeIndex, mlOperationSetIndex, mlOperationIndex);
}

uint16 RigLogicDNAReader::getMLOperationIndexListCount(uint16 mlTypeIndex, uint16 mlOperationSetIndex) const
{
	return Reader->getMLOperationIndexListCount(mlTypeIndex, mlOperationSetIndex);
}

dna::ConstArrayView<uint16> RigLogicDNAReader::getMLOperationIndicesForLOD(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 lod) const
{
	return Reader->getMLOperationIndicesForLOD(mlTypeIndex, mlOperationSetIndex, lod);
}

dna::ConstArrayView<uint16> RigLogicDNAReader::getMLJointsInputIndices() const
{
	return Reader->getMLJointsInputIndices();
}

dna::ConstArrayView<uint16> RigLogicDNAReader::getMLJointsOutputIndices() const
{
	return Reader->getMLJointsOutputIndices();
}

dna::ConstArrayView<uint16> RigLogicDNAReader::getMLJointsParameterKeys() const
{
	return Reader->getMLJointsParameterKeys();
}

dna::ConstArrayView<uint16> RigLogicDNAReader::getMLJointsParameterValues() const
{
	return Reader->getMLJointsParameterValues();
}

// Reader
void RigLogicDNAReader::unload(dna::DataLayer Layer)
{
	ensureMsgf(false, TEXT("Assets are not unloadable"));
}

void RigLogicDNAReader::destroy(dna::Reader* Pointer)
{
}
