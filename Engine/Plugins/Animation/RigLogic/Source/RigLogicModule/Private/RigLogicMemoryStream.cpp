// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicMemoryStream.h"

#include "Math/NumericLimits.h"

FRigLogicMemoryStream::FRigLogicMemoryStream(TArray<uint8>* Buffer)
{
	BitStreamBuffer = Buffer; //don't allocate new memory, just point to the existing one
}

void FRigLogicMemoryStream::seek(std::uint64_t Position)
{
	ensure(Position <= static_cast<std::uint64_t>(static_cast<size_t>(-1)));
	PositionInBuffer = static_cast<size_t>(Position);
}

std::uint64_t FRigLogicMemoryStream::tell()
{
	return PositionInBuffer;
}

void FRigLogicMemoryStream::open()
{
	PositionInBuffer = 0;
}

size_t FRigLogicMemoryStream::read(char* ReadToBuffer, size_t Size)
{
	FMemory::Memcpy(ReadToBuffer, BitStreamBuffer->GetData() + PositionInBuffer, Size);
	PositionInBuffer = PositionInBuffer + Size;
	return Size;
}

size_t FRigLogicMemoryStream::read(Writable* Destination, size_t Size)
{
	Destination->write(reinterpret_cast<char*>(BitStreamBuffer->GetData() + PositionInBuffer), Size);
	PositionInBuffer = PositionInBuffer + Size;
	return Size;
}

size_t FRigLogicMemoryStream::write(const char* WriteFromBuffer, size_t Size)
{
	Grow(PositionInBuffer + Size);
	FMemory::Memcpy(BitStreamBuffer->GetData() + PositionInBuffer, WriteFromBuffer, Size);
	PositionInBuffer = PositionInBuffer + Size;
	return Size;
}

size_t FRigLogicMemoryStream::write(Readable* Source, size_t Size)
{
	Grow(PositionInBuffer + Size);
	Source->read(reinterpret_cast<char*>(BitStreamBuffer->GetData() + PositionInBuffer), Size);
	PositionInBuffer = PositionInBuffer + Size;
	return Size;
}

std::uint64_t FRigLogicMemoryStream::size()
{
	const int32 Count = BitStreamBuffer->Num();
	ensure(Count >= 0);
	return static_cast<std::uint64_t>(Count);
}

void FRigLogicMemoryStream::Grow(size_t NewSize)
{
	const int32 BufferSize = BitStreamBuffer->Num();
	ensure(BufferSize >= 0);
	if (NewSize > static_cast<size_t>(BufferSize))
	{
		const int32 Difference = NewSize - BufferSize;
		BitStreamBuffer->AddUninitialized(Difference);
	}
}
