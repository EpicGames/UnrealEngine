// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "riglogic/RigLogic.h"

/** A simple implementation of in-memory stream for interfacing with RigLogic API,
  so RigLogic can consume DNA as a Stream either from file or memory */

class FRigLogicMemoryStream: public rl4::BoundedIOStream
{
public:
	/** The buffer is not copied, the pointer to it is stored inside this object**/
	explicit FRigLogicMemoryStream(TArray<uint8>* Buffer);

	void seek(std::uint64_t Position) override;
	std::uint64_t tell() override;
	void open() override;
	void close() override {}
	size_t read(char* ReadToBuffer, size_t Size) override;
	size_t read(Writable* Destination, size_t Size) override;
	size_t write(const char* WriteFromBuffer, size_t Size) override;
	size_t write(Readable* Source, size_t Size) override;
	std::uint64_t size() override;

private:
	void Grow(size_t NewSize);

private:
	TArray<uint8>* BitStreamBuffer; //doesn't contain the array, only points to the array given to it
	size_t PositionInBuffer = 0;
};
