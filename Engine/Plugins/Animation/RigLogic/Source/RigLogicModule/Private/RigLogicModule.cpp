// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigLogicModule.h"

#include "RigUnit_RigLogic.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "HAL/PlatformProcess.h"

#define LOCTEXT_NAMESPACE "FRigLogicModule"

DEFINE_LOG_CATEGORY(LogRigLogic);

void FRigLogicModule::StartupModule()
{

}

void FRigLogicModule::ShutdownModule()
{

}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FRigLogicModule, RigLogicModule)

