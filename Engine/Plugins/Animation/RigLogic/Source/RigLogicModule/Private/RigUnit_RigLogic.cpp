// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_RigLogic.h"

#include "ControlRig.h"
#include "DNAReader.h"
#include "DNA.h"
#include "DNAAsset.h"
#include "DNAAssetUserData.h"
#include "Engine/SkeletalMesh.h"
#include "RigInstance.h"
#include "RigLogic.h"
#include "SharedRigRuntimeContext.h"
#include "Components/SkeletalMeshComponent.h"
#include "Math/TransformNonVectorized.h"
#include "Units/RigUnitContext.h"

#include <tdm/TDM.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_RigLogic)

DEFINE_LOG_CATEGORY(LogRigLogicUnit);

const uint8 FRigUnit_RigLogic_Data::MAX_ATTRS_PER_JOINT = 10;

/** Constructs curve name from nameToSplit using formatString of form x<obj>y<attr>z **/
static FString ConstructCurveName(const FString& NameToSplit, const FString& FormatString)
{
	// constructs curve name from NameToSplit (always in form <obj>.<attr>)
	// using FormatString of form x<obj>y<attr>z
	// where x, y and z are arbitrary strings
	// example:
	// FormatString="mesh_<obj>_<attr>"
	// 'head.blink_L' becomes 'mesh_head_blink_L'

	FString ObjectName, AttributeName;
	if (!NameToSplit.Split(".", &ObjectName, &AttributeName))
	{

		UE_LOGF(LogRigLogicUnit, Error, "RigUnit_R: Missing '.' in '%ls'",
			*NameToSplit);
		return TEXT("");
	}

	FString CurveName = FormatString;
	CurveName = CurveName.Replace(TEXT("<obj>"), *ObjectName);
	CurveName = CurveName.Replace(TEXT("<attr>"), *AttributeName);
	return CurveName;
}

FRigUnit_RigLogic_Data::FRigUnit_RigLogic_Data()
: SkelMeshComponent(nullptr)
, LocalRigRuntimeContext(nullptr)
, RigInstance(nullptr)
, CurrentLOD(0)
{
}

FRigUnit_RigLogic_Data::~FRigUnit_RigLogic_Data()
{
	LocalRigRuntimeContext = nullptr;
}

FRigUnit_RigLogic_Data::FRigUnit_RigLogic_Data(const FRigUnit_RigLogic_Data& Other)
{
	*this = Other;
}

FRigUnit_RigLogic_Data& FRigUnit_RigLogic_Data::operator=(const FRigUnit_RigLogic_Data& Other)
{
	SkelMeshComponent = Other.SkelMeshComponent;
	LocalRigRuntimeContext = nullptr;
	RigInstance = nullptr;
	InputCurveIndices = Other.InputCurveIndices;
	MLMaskCurveIndices = Other.MLMaskCurveIndices;
	HierarchyBoneIndices = Other.HierarchyBoneIndices;
	MorphTargetCurveIndices = Other.MorphTargetCurveIndices;
	BlendShapeIndices = Other.BlendShapeIndices;
	CurveElementIndicesForAnimMaps = Other.CurveElementIndicesForAnimMaps;
	RigLogicIndicesForAnimMaps = Other.RigLogicIndicesForAnimMaps;
	CurrentLOD = Other.CurrentLOD;
	return *this;
}

bool FRigUnit_RigLogic_Data::IsRigLogicInitialized()
{
	return (LocalRigRuntimeContext != nullptr) && LocalRigRuntimeContext->RigLogic.IsValid() && RigInstance.IsValid();
}

void FRigUnit_RigLogic_Data::InitializeRigLogic(const URigHierarchy* InHierarchy, TSharedPtr<FSharedRigRuntimeContext> NewContext)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_InitializeRigLogic);

	if (!NewContext.IsValid())
	{
		UE_LOGF(LogRigLogicUnit, Warning, "No valid DNA file found, abort initialization.");
		return;
	}

	if (LocalRigRuntimeContext != NewContext)
	{
		LocalRigRuntimeContext = NewContext;
		RigInstance = nullptr;
	}

	if (!RigInstance.IsValid())
	{
		RigInstance = MakeUnique<FRigInstance>(LocalRigRuntimeContext->RigLogic.Get());
		RigInstance->SetLOD(CurrentLOD);
		CurrentLOD = RigInstance->GetLOD();

		const FRigLogicConfiguration& RigLogicConfig = LocalRigRuntimeContext->RigLogic->GetConfiguration();

		if (RigLogicConfig.LoadJoints)
		{
			MapJoints(InHierarchy);
		}
		if (RigLogicConfig.LoadTwistSwingBehavior || RigLogicConfig.LoadRBFBehavior)
		{
			MapDriverJoints(InHierarchy);
		}

		MapInputCurveIndices(InHierarchy);

		if (RigLogicConfig.LoadMachineLearnedBehavior)
		{
			MapMLMaskCurveIndices(InHierarchy);
		}
		if (RigLogicConfig.LoadBlendShapes)
		{
			MapMorphTargets(InHierarchy);
		}
		if (RigLogicConfig.LoadAnimatedMaps)
		{
			MapMaskMultipliers(InHierarchy);
		}
	}
}

//maps indices of input curves from dna file to control rig curves
void FRigUnit_RigLogic_Data::MapInputCurveIndices(const URigHierarchy* InHierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_MapInputCurveIndices);

	const IDNAReader* DNABehavior = LocalRigRuntimeContext->DNAReader.Get();
	const uint32 ControlCount = DNABehavior->GetRawControlCount();
	InputCurveIndices.Reset(ControlCount);
	for (uint32_t ControlIndex = 0; ControlIndex < ControlCount; ++ControlIndex)
	{
		const FString DNAControlName = DNABehavior->GetRawControlName(ControlIndex);
		const FString AnimatedControlName = ConstructCurveName(DNAControlName, TEXT("<obj>_<attr>"));
		if (AnimatedControlName == TEXT(""))
		{
			return;
		}
		const FName ControlFName(*AnimatedControlName);
		const int32 CurveIndex = InHierarchy ? InHierarchy->GetIndex(FRigElementKey(ControlFName, ERigElementType::Curve)) : INDEX_NONE;
		InputCurveIndices.Add(CurveIndex); //can be INDEX_NONE
	}
}

void FRigUnit_RigLogic_Data::MapMLMaskCurveIndices(const URigHierarchy* InHierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_MapMLMaskCurveIndices);

	const IDNAReader* DNABehavior = LocalRigRuntimeContext->DNAReader.Get();

	MLMaskCurveIndices.Empty();

	const uint16 MeshCount = DNABehavior->GetMeshCount();
	for (uint16 MeshIndex = 0; MeshIndex < MeshCount; ++MeshIndex)
	{
		const uint16 MeshRegionCount = DNABehavior->GetMeshRegionCount(MeshIndex);
		MLMaskCurveIndices.Reserve(MLMaskCurveIndices.Num() + MeshRegionCount);
		for (uint16 RegionIndex = 0; RegionIndex < MeshRegionCount; ++RegionIndex)
		{
			const FString& MeshRegionName = DNABehavior->GetMeshRegionName(MeshIndex, RegionIndex);
			const FString MaskCurveName = TEXT("CTRL_ML_") + MeshRegionName;
			const FName CurveFName(*MaskCurveName);
			const int32 CurveIndex = InHierarchy ? InHierarchy->GetIndex(FRigElementKey(CurveFName, ERigElementType::Curve)) : INDEX_NONE;
			MLMaskCurveIndices.Add(CurveIndex);
		}
	}
}

void FRigUnit_RigLogic_Data::MapJoints(const URigHierarchy* Hierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_MapJoints);

	const IDNAReader* DNABehavior = LocalRigRuntimeContext->DNAReader.Get();
	const uint16 JointCount = DNABehavior->GetJointCount();
	HierarchyBoneIndices.Reset(JointCount);
	for (uint16 JointIndex = 0; JointIndex < JointCount ; ++JointIndex)
	{
		const FString RLJointName = DNABehavior->GetJointName(JointIndex);
		const FName JointFName = FName(*RLJointName);
		const int32 BoneIndex = Hierarchy->GetIndex(FRigElementKey(JointFName, ERigElementType::Bone));
		HierarchyBoneIndices.Add(BoneIndex);
	}
}

void FRigUnit_RigLogic_Data::MapDriverJoints(const URigHierarchy* Hierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_MapDriverJoints);

	const IDNAReader* DNABehavior = LocalRigRuntimeContext->DNAReader.Get();

	auto FindJointIndex = [DNABehavior](const FString& JointName)
		{
			const uint16 JointCount = DNABehavior->GetJointCount();
			for (uint16 JointIndex = 0; JointIndex < JointCount; ++JointIndex)
			{
				const FString& RLJointName = DNABehavior->GetJointName(JointIndex);
				if (RLJointName == JointName)
				{
					return JointIndex;
				}
			}
			return static_cast<uint16>(-1);
		};

	const uint32 ControlCount = DNABehavior->GetRawControlCount();
	DriverJointsToControlAttributesMap.Empty();
	// This is a correct approximation as long as only 4 (rotation) attributes are used as driver joint attributes
	// and no regular raw controls are present in the DNA
	DriverJointsToControlAttributesMap.Reserve(ControlCount / 4);
	for (uint32_t ControlIndex = 0; ControlIndex < ControlCount; ++ControlIndex)
	{
		const FString DriverJointAttrName = DNABehavior->GetRawControlName(ControlIndex);
		if (DriverJointAttrName.Len() < 2)
		{
			continue;
		}

		const FString DriverJointName = DriverJointAttrName.Mid(0, DriverJointAttrName.Len() - 2);
		const FName BoneName = FName(*DriverJointName);
		const int32 BoneIndex = Hierarchy->GetIndex(FRigElementKey(BoneName, ERigElementType::Bone));
		if (BoneIndex == INDEX_NONE)
		{
			// Mixed DNAs will contain both driver joints and normal raw controls in this list, and those will
			// not be found in the joint hierarchy
			continue;
		}

		int32 MappingIndex = DriverJointsToControlAttributesMap.FindLastByPredicate([BoneIndex](const FBoneIndexControlAttributeMapping& Element)
			{
				return Element.BoneIndex == BoneIndex;
			});
		if (MappingIndex == INDEX_NONE)
		{
			FBoneIndexControlAttributeMapping NewMapping{BoneIndex, INDEX_NONE, INDEX_NONE , INDEX_NONE , INDEX_NONE, INDEX_NONE};
			// BoneIndex may be INDEX_NONE, but it's handled properly by the Evaluate method
			MappingIndex = DriverJointsToControlAttributesMap.Add(NewMapping);
		}

		FBoneIndexControlAttributeMapping& Mapping = DriverJointsToControlAttributesMap[MappingIndex];
		Mapping.DNAJointIndex = FindJointIndex(DriverJointName);

		if (DriverJointAttrName.EndsWith(TEXT(".x")))
		{
			Mapping.RotationX = ControlIndex;
		}
		else if (DriverJointAttrName.EndsWith(TEXT(".y")))
		{
			Mapping.RotationY = ControlIndex;
		}
		else if (DriverJointAttrName.EndsWith(TEXT(".z")))
		{
			Mapping.RotationZ = ControlIndex;
		}
		else if (DriverJointAttrName.EndsWith(TEXT(".w")))
		{
			Mapping.RotationW = ControlIndex;
		}
	}
}

void FRigUnit_RigLogic_Data::MapMorphTargets(const URigHierarchy* InHierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_MapMorphTargets);

	const IDNAReader* DNABehavior = LocalRigRuntimeContext->DNAReader.Get();
	const uint16 LODCount = DNABehavior->GetLODCount();

	MorphTargetCurveIndices.Reset();
	MorphTargetCurveIndices.AddDefaulted(LODCount);
	BlendShapeIndices.Reset();
	BlendShapeIndices.AddDefaulted(LODCount);

	for (uint16 LodIndex = 0; LodIndex < LODCount; ++LodIndex)
	{
		TArrayView<const uint16> BlendShapeChannelIndicesForLOD =
			DNABehavior->GetMeshBlendShapeChannelMappingIndicesForLOD(LodIndex);
		MorphTargetCurveIndices[LodIndex].Values.Reserve(BlendShapeChannelIndicesForLOD.Num());
		BlendShapeIndices[LodIndex].Values.Reserve(BlendShapeChannelIndicesForLOD.Num());
		for (uint16 MappingIndex: BlendShapeChannelIndicesForLOD)
		{
			const FMeshBlendShapeChannelMapping Mapping = DNABehavior->GetMeshBlendShapeChannelMapping(MappingIndex);
			const uint16 BlendShapeIndex = Mapping.BlendShapeChannelIndex;
			const uint16 MeshIndex = Mapping.MeshIndex;
			const FString BlendShapeStr = DNABehavior->GetBlendShapeChannelName(BlendShapeIndex);
			const FString MeshStr = DNABehavior->GetMeshName(MeshIndex);
			const FString MorphTargetStr = MeshStr + TEXT("__") + BlendShapeStr;
			const FName MorphTargetName(*MorphTargetStr);
			const int32 MorphTargetIndex = InHierarchy->GetIndex(FRigElementKey(MorphTargetName, ERigElementType::Curve));
			MorphTargetCurveIndices[LodIndex].Values.Add(MorphTargetIndex);
			BlendShapeIndices[LodIndex].Values.Add(Mapping.BlendShapeChannelIndex);
		}
	}
}

void FRigUnit_RigLogic_Data::MapMaskMultipliers(const URigHierarchy* InHierarchy)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_MapMaskMultipliers);

	const IDNAReader* DNABehavior = LocalRigRuntimeContext->DNAReader.Get();
	const uint16 LODCount = DNABehavior->GetLODCount();
	CurveElementIndicesForAnimMaps.Reset();
	CurveElementIndicesForAnimMaps.AddDefaulted(LODCount);

	RigLogicIndicesForAnimMaps.Reset();
	RigLogicIndicesForAnimMaps.AddDefaulted(LODCount);

	for (uint16 LodIndex = 0; LodIndex < LODCount; ++LodIndex)
	{
		TArrayView<const uint16> AnimMapIndicesPerLOD = DNABehavior->GetAnimatedMapIndicesForLOD(LodIndex);
		CurveElementIndicesForAnimMaps[LodIndex].Values.Reserve(AnimMapIndicesPerLOD.Num());
		RigLogicIndicesForAnimMaps[LodIndex].Values.Reserve(AnimMapIndicesPerLOD.Num());
		for (uint16 AnimMapIndexPerLOD: AnimMapIndicesPerLOD)
		{
			const FString AnimMapNameFStr = DNABehavior->GetAnimatedMapName(AnimMapIndexPerLOD);
			const FString MaskMultiplierNameStr = ConstructCurveName(AnimMapNameFStr, TEXT("<obj>_<attr>"));
			if (MaskMultiplierNameStr == "") {
				return;
			}
			const FName MaskMultiplierFName(*MaskMultiplierNameStr);
			const int32 CurveIndex = InHierarchy->GetIndex(FRigElementKey(MaskMultiplierFName, ERigElementType::Curve));
			CurveElementIndicesForAnimMaps[LodIndex].Values.Add(CurveIndex); //can be INDEX_NONE if curve was not found
			RigLogicIndicesForAnimMaps[LodIndex].Values.Add(AnimMapIndexPerLOD);
		}
	}
}

void FRigUnit_RigLogic_Data::CalculateRigLogic(const URigHierarchy* InHierarchy, TArrayView<const float> NeutralJointValues)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_Calculate);

	// LOD change is inexpensive
	RigInstance->SetLOD(CurrentLOD);
	CurrentLOD = RigInstance->GetLOD();

	const int32 RawControlCount = RigInstance->GetRawControlCount();
	for (int32 ControlIndex = 0; ControlIndex < RawControlCount; ++ControlIndex)
	{
		const uint32 CurveIndex = InputCurveIndices[ControlIndex];
		const float Value = InHierarchy->GetCurveValue(CurveIndex);
		RigInstance->SetRawControl(ControlIndex, FMath::Clamp(Value, 0.0, 1.0));
	}

	const FRigLogicConfiguration& RigLogicConfig = LocalRigRuntimeContext->RigLogic->GetConfiguration();
	if (RigLogicConfig.LoadRBFBehavior || RigLogicConfig.LoadTwistSwingBehavior)
	{
		for (int32 MappingIndex = 0; MappingIndex < DriverJointsToControlAttributesMap.Num(); ++MappingIndex)
		{
			const FBoneIndexControlAttributeMapping& Mapping = DriverJointsToControlAttributesMap[MappingIndex];
			const FTransform& PoseTransform = InHierarchy->GetLocalTransform(Mapping.BoneIndex);
			// Translation and Scale is currently not used here, so to avoid the overhead of checking them, they are simply ignored.
			// Should the need arise to use them as well, this code will need adjustment.
			const FQuat Rotation = PoseTransform.GetRotation();
			const int32 AttrIndex = Mapping.DNAJointIndex * MAX_ATTRS_PER_JOINT;
			const tdm::fquat NeutralRotation{ NeutralJointValues[AttrIndex + 3], NeutralJointValues[AttrIndex + 4], NeutralJointValues[AttrIndex + 5], NeutralJointValues[AttrIndex + 6] };
			const tdm::fquat AbsPoseRotation{ static_cast<float>(Rotation.X), static_cast<float>(Rotation.Y), static_cast<float>(Rotation.Z), static_cast<float>(Rotation.W) };
			const tdm::fquat DeltaPoseRotation = tdm::inverse(NeutralRotation) * AbsPoseRotation;

			if (Mapping.RotationX != INDEX_NONE)
			{
				RigInstance->SetRawControl(Mapping.RotationX, DeltaPoseRotation.x);
			}
			if (Mapping.RotationY != INDEX_NONE)
			{
				RigInstance->SetRawControl(Mapping.RotationY, DeltaPoseRotation.y);
			}
			if (Mapping.RotationZ != INDEX_NONE)
			{
				RigInstance->SetRawControl(Mapping.RotationZ, DeltaPoseRotation.z);
			}
			if (Mapping.RotationW != INDEX_NONE)
			{
				RigInstance->SetRawControl(Mapping.RotationW, DeltaPoseRotation.w);
			}
		}
	}

	if (RigLogicConfig.LoadMachineLearnedBehavior)
	{
		TArrayView<float> MaskBuffer = RigInstance->GetMLMaskValues();
		for (int32 MaskIndex = 0; MaskIndex < MLMaskCurveIndices.Num(); ++MaskIndex)
		{
			const int32 CurveIndex = MLMaskCurveIndices[MaskIndex];
			if (InHierarchy->IsCurveValueSetByIndex(CurveIndex))
			{
				MaskBuffer[MaskIndex] = InHierarchy->GetCurveValue(CurveIndex);
			}
		}
	}

	LocalRigRuntimeContext->RigLogic->Calculate(RigInstance.Get());
}

void FRigUnit_RigLogic_Data::UpdateJoints(URigHierarchy* Hierarchy, TArrayView<const float> NeutralJointValues, TArrayView<const float> DeltaJointValues)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_UpdateJoints);

	const float* N = NeutralJointValues.GetData();
	const float* D = DeltaJointValues.GetData();

	for (const uint16 JointIndex : LocalRigRuntimeContext->VariableJointIndicesPerLOD[CurrentLOD].Values)
	{
		const int32 BoneIndex = HierarchyBoneIndices[JointIndex];
		if (BoneIndex != INDEX_NONE)
		{
			const uint16 AttrIndex = JointIndex * MAX_ATTRS_PER_JOINT;
			const FTransform Transform
			{
				FQuat(N[AttrIndex + 3], N[AttrIndex + 4], N[AttrIndex + 5], N[AttrIndex + 6]) * FQuat(D[AttrIndex + 3], D[AttrIndex + 4], D[AttrIndex + 5], D[AttrIndex + 6]),
				FVector((N[AttrIndex + 0] + D[AttrIndex + 0]), (N[AttrIndex + 1] + D[AttrIndex + 1]), (N[AttrIndex + 2] + D[AttrIndex + 2])),
				FVector((N[AttrIndex + 7] + D[AttrIndex + 7]), (N[AttrIndex + 8] + D[AttrIndex + 8]), (N[AttrIndex + 9] + D[AttrIndex + 9]))
			};
			Hierarchy->SetLocalTransform(BoneIndex, Transform);
		}
	}
}

void FRigUnit_RigLogic_Data::UpdateBlendShapeCurves(URigHierarchy* InHierarchy, TArrayView<const float> BlendShapeValues)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_UpdateBlendShapeCurves);

	// set output blend shapes
	if (BlendShapeIndices.IsValidIndex(CurrentLOD) && MorphTargetCurveIndices.IsValidIndex(CurrentLOD))
	{
		const uint32 BlendShapePerLODCount = static_cast<uint32>(BlendShapeIndices[CurrentLOD].Values.Num());

		if (ensure(BlendShapePerLODCount == MorphTargetCurveIndices[CurrentLOD].Values.Num()))
		{
			for (uint32 MeshBlendIndex = 0; MeshBlendIndex < BlendShapePerLODCount; MeshBlendIndex++)
			{
				const int32 BlendShapeIndex = BlendShapeIndices[CurrentLOD].Values[MeshBlendIndex];
				const int32 MorphTargetCurveIndex = MorphTargetCurveIndices[CurrentLOD].Values[MeshBlendIndex];
				if (MorphTargetCurveIndex != INDEX_NONE)
				{
					const float Value = BlendShapeValues[BlendShapeIndex];
					InHierarchy->SetCurveValue(MorphTargetCurveIndex, Value);
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogRigLogicUnit, Warning, "Invalid LOD Index for the BlendShapes. Ensure your curve is set up correctly!");
	}
}

void FRigUnit_RigLogic_Data::UpdateAnimMapCurves(URigHierarchy* InHierarchy, TArrayView<const float> AnimMapOutputs)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_UpdateAnimMapCurves);

	// set output mask multipliers
	// In case curves are not imported yet into CL, AnimatedMapsCurveIndices will be empty, so we need to check
	// array bounds before trying to access it:
	if (RigLogicIndicesForAnimMaps.IsValidIndex(CurrentLOD) && CurveElementIndicesForAnimMaps.IsValidIndex(CurrentLOD))
	{
		const uint32 AnimMapPerLODCount = RigLogicIndicesForAnimMaps[CurrentLOD].Values.Num();
		for (uint32 AnimMapIndexForLOD = 0; AnimMapIndexForLOD < AnimMapPerLODCount; ++AnimMapIndexForLOD)
		{
			const int32 RigLogicAnimMapIndex = RigLogicIndicesForAnimMaps[CurrentLOD].Values[AnimMapIndexForLOD];
			const int32 InHierarchyAnimMapIndex = CurveElementIndicesForAnimMaps[CurrentLOD].Values[AnimMapIndexForLOD];
			if (InHierarchyAnimMapIndex != INDEX_NONE)
			{
				const float Value = AnimMapOutputs[RigLogicAnimMapIndex];
				InHierarchy->SetCurveValue(InHierarchyAnimMapIndex, Value);
			}
		}
	}
	else
	{
		UE_LOGF(LogRigLogicUnit, Warning, "Invalid LOD Index for the AnimationMaps. Ensure your curve is set up correctly!");
	}	
}

TSharedPtr<FSharedRigRuntimeContext> FRigUnit_RigLogic::GetSharedRigRuntimeContext(USkeletalMesh* SkelMesh)
{
	TSharedPtr<FSharedRigRuntimeContext> SharedRigRuntimeContext = nullptr;

	// We either have DNAAssetUserData on SKM and then use it to get DNA asset or we use legacy asset user data as a fallback option in else
	if (UDNAAssetUserData* DNAAssetUserData = SkelMesh->GetAssetUserData<UDNAAssetUserData>())
	{
		if (UDNA* DNA = DNAAssetUserData->DNAAsset)
		{
			SharedRigRuntimeContext = DNA->GetRigRuntimeContext();
		}
	}
	else
	{
		if (UDNAAsset* DNAAsset = SkelMesh->GetAssetUserData<UDNAAsset>())
		{
			SharedRigRuntimeContext = DNAAsset->GetRigRuntimeContext();
		}
	}

	return SharedRigRuntimeContext;
}

FRigUnit_RigLogic_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigUnit_RigLogic_Execute);

 	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy)
	{
		if(!bIsInitialized)
		{
			//const double startTime = FPlatformTime::Seconds();
			if (!Data.SkelMeshComponent.IsValid())
			{
				Data.SkelMeshComponent = ExecuteContext.UnitContext.DataSourceRegistry->RequestSource<USkeletalMeshComponent>(UControlRig::OwnerComponent);
				// In normal execution, Data.SkelMeshComponent will be nullptr at the beginning
				// however, during unit testing we cannot fetch it from DataSourceRegistry 
				// in that case, a mock version will be inserted into Data by unit test beforehand
			}
			if (!Data.SkelMeshComponent.IsValid() || Data.SkelMeshComponent->GetSkeletalMeshAsset() == nullptr)
			{
				return;
			}
			Data.CurrentLOD = Data.SkelMeshComponent->GetPredictedLODLevel();

			// Fetch shared runtime context of rig from DNAAsset
			TSharedPtr<FSharedRigRuntimeContext> RigRuntimeContext = GetSharedRigRuntimeContext(Data.SkelMeshComponent->GetSkeletalMeshAsset());
			// Context is initialized with a BehaviorReader, which can be imported into SkeletalMesh from DNA file
			// or overwritten by GeneSplicer when making a new character
			Data.InitializeRigLogic(Hierarchy, RigRuntimeContext);
			bIsInitialized = true;
			//const double delta = FPlatformTime::Seconds() - startTime;
			//UE_LOGF(LogRigLogicUnit, Warning, "RigLogic::Init execution time: %f", delta);
		}
		

		//const double startTime = FPlatformTime::Seconds();
		// Fetch shared runtime context of rig from DNAAsset
		if (!Data.SkelMeshComponent.IsValid() || !Data.IsRigLogicInitialized() || Hierarchy == nullptr)
		{
			return;
		}
		Data.CurrentLOD = Data.SkelMeshComponent->GetPredictedLODLevel();

		const FRigLogicConfiguration& RigLogicConfig = Data.LocalRigRuntimeContext->RigLogic->GetConfiguration();

		TArrayView<const float> NeutralJointValues = Data.LocalRigRuntimeContext->RigLogic->GetNeutralJointValues();
		TArrayView<const float> DeltaJointValues = Data.RigInstance->GetJointOutputs();

		Data.CalculateRigLogic(Hierarchy, NeutralJointValues);

		if (RigLogicConfig.LoadJoints)
		{
			Data.UpdateJoints(Hierarchy, NeutralJointValues, DeltaJointValues);
		}

		if (RigLogicConfig.LoadBlendShapes)
		{
			TArrayView<const float> BlendShapeValues = Data.RigInstance->GetBlendShapeOutputs();
			Data.UpdateBlendShapeCurves(Hierarchy, BlendShapeValues);
		}

		if (RigLogicConfig.LoadAnimatedMaps)
		{
			TArrayView<const float> AnimMapOutputs = Data.RigInstance->GetAnimatedMapOutputs();
			Data.UpdateAnimMapCurves(Hierarchy, AnimMapOutputs);
		}

		//const double delta = FPlatformTime::Seconds() - startTime;
		//UE_LOGF(LogRigLogicUnit, Warning, "RigLogic::Update execution time: %f", delta);
	}
}

