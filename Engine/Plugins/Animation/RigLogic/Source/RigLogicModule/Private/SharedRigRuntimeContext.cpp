// Copyright Epic Games, Inc. All Rights Reserved.

#include "SharedRigRuntimeContext.h"

#include "CoreMinimal.h"
#include "HAL/LowLevelMemTracker.h"
#include "DNAReader.h"
#include "RigLogic.h"

FSharedRigRuntimeContext::FSharedRigRuntimeContext() = default;

FSharedRigRuntimeContext::FSharedRigRuntimeContext(TSharedPtr<IDNAReader> InDNAReader, TSharedPtr<FRigLogic> InRigLogic) :
	DNAReader{InDNAReader},
	RigLogic{InRigLogic}
{
	CacheVariableJointIndices();
	if ((RigLogic->GetRBFSolverCount() != 0) || (RigLogic->GetSwingCount() != 0) || (RigLogic->GetTwistCount() != 0))
	{
		CacheInverseNeutralJointRotations();
	}
}

void FSharedRigRuntimeContext::CacheVariableJointIndices()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	const FRigLogicConfiguration& Config = RigLogic->GetConfiguration();
	const uint32 AttrCount = static_cast<uint8>(Config.TranslationType) + static_cast<uint8>(Config.RotationType) + static_cast<uint8>(Config.ScaleType);
	const uint16 LODCount = RigLogic->GetLODCount();
	VariableJointIndicesPerLOD.Reset();
	VariableJointIndicesPerLOD.AddDefaulted(LODCount);
	TSet<uint16> DistinctVariableJointIndices;
	for (uint16 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
	{
		TArrayView<const uint16> VariableAttributeIndices = RigLogic->GetJointVariableAttributeIndices(LODIndex);
		DistinctVariableJointIndices.Reset();
		DistinctVariableJointIndices.Reserve(VariableAttributeIndices.Num());
		for (const uint16 AttrIndex : VariableAttributeIndices)
		{
			const uint16 JointIndex = AttrIndex / AttrCount;
			DistinctVariableJointIndices.Add(JointIndex);
		}
		VariableJointIndicesPerLOD[LODIndex].Values = DistinctVariableJointIndices.Array();
	}
}

void FSharedRigRuntimeContext::CacheInverseNeutralJointRotations()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/RigLogic"));

	static constexpr uint32 JointAttributeCount = 10;
	TArrayView<const float> NeutralJointValues = RigLogic->GetNeutralJointValues();
	ensure(NeutralJointValues.Num() % JointAttributeCount == 0);

	const int32 JointCount = NeutralJointValues.Num() / JointAttributeCount;
	InverseNeutralJointRotations.Reset(JointCount);
	for (int32 JointIndex = 0; JointIndex < JointCount; ++JointIndex)
	{
		const int32 AttrIndex = JointIndex * JointAttributeCount;
		const tdm::fquat NeutralRotation{NeutralJointValues[AttrIndex + 3], NeutralJointValues[AttrIndex + 4], NeutralJointValues[AttrIndex + 5], NeutralJointValues[AttrIndex + 6]};
		const tdm::fquat InverseNeutralRotation = tdm::inverse(NeutralRotation);
		InverseNeutralJointRotations.Add(FQuat(InverseNeutralRotation.x, InverseNeutralRotation.y, InverseNeutralRotation.z, InverseNeutralRotation.w));
	}
}
