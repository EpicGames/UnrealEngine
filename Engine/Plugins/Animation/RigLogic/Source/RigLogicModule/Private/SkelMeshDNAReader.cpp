// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkelMeshDNAReader.h"

#include "DNAAsset.h"
#include "DNAReader.h"

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif
#include "Engine/AssetUserData.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY(LogSkelMeshDNAReader);

FSkelMeshDNAReader::FSkelMeshDNAReader(UDNAAsset* DNAAsset) : DNAReader{nullptr}
{
	DNAReader = DNAAsset->GetDNAReader();
}

/** HEADER READER **/
uint16 FSkelMeshDNAReader::GetFileFormatGeneration() const
{
	return DNAReader->GetFileFormatGeneration();
}

uint16 FSkelMeshDNAReader::GetFileFormatVersion() const
{
	return DNAReader->GetFileFormatVersion();
}

/** DESCRIPTOR READER **/

FString FSkelMeshDNAReader::GetName() const
{
	return DNAReader->GetName();
}

EArchetype FSkelMeshDNAReader::GetArchetype() const
{
	return DNAReader->GetArchetype();
}

EGender FSkelMeshDNAReader::GetGender() const
{
	return DNAReader->GetGender();
}

uint16 FSkelMeshDNAReader::GetAge() const
{
	return DNAReader->GetAge();
}

uint32 FSkelMeshDNAReader::GetMetaDataCount() const
{
	return DNAReader->GetMetaDataCount();
}

FString FSkelMeshDNAReader::GetMetaDataKey(uint32 Index) const
{
	return DNAReader->GetMetaDataKey(Index);
}

FString FSkelMeshDNAReader::GetMetaDataValue(const FString& Key) const
{
	return DNAReader->GetMetaDataValue(Key);
}

ETranslationUnit FSkelMeshDNAReader::GetTranslationUnit() const
{
	return DNAReader->GetTranslationUnit();
}

ERotationUnit FSkelMeshDNAReader::GetRotationUnit() const
{
	return DNAReader->GetRotationUnit();
}

FCoordinateSystem FSkelMeshDNAReader::GetCoordinateSystem() const
{
	return DNAReader->GetCoordinateSystem();
}

ERotationSequence FSkelMeshDNAReader::GetRotationSequence() const
{
	return DNAReader->GetRotationSequence();
}
FRotationSign FSkelMeshDNAReader::GetRotationSign() const
{
	return DNAReader->GetRotationSign();
}

EFaceWindingOrder FSkelMeshDNAReader::GetFaceWindingOrder() const
{
	return DNAReader->GetFaceWindingOrder();
}

uint16 FSkelMeshDNAReader::GetLODCount() const
{
	return DNAReader->GetLODCount();
}

uint16 FSkelMeshDNAReader::GetDBMaxLOD() const
{
	return DNAReader->GetDBMaxLOD();
}

FString FSkelMeshDNAReader::GetDBComplexity() const
{
	return DNAReader->GetDBComplexity();
}

FString FSkelMeshDNAReader::GetDBName() const
{
	return DNAReader->GetDBName();
}

/** DEFINITION READER **/

uint16 FSkelMeshDNAReader::GetGUIControlCount() const
{
	return DNAReader->GetGUIControlCount();
}

FString FSkelMeshDNAReader::GetGUIControlName(uint16 Index) const
{
	return DNAReader->GetGUIControlName(Index);
}

uint16 FSkelMeshDNAReader::GetRawControlCount() const
{
	return DNAReader->GetRawControlCount();
}

FString FSkelMeshDNAReader::GetRawControlName(uint16 Index) const
{
	return DNAReader->GetRawControlName(Index);
}

uint16 FSkelMeshDNAReader::GetJointCount() const
{
	return DNAReader->GetJointCount();
}

FString FSkelMeshDNAReader::GetJointName(uint16 Index) const
{
	return DNAReader->GetJointName(Index);
}

uint16 FSkelMeshDNAReader::GetJointIndexListCount() const
{
	return DNAReader->GetJointIndexListCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetJointIndicesForLOD(uint16 LOD) const
{
	return DNAReader->GetJointIndicesForLOD(LOD);
}

uint16 FSkelMeshDNAReader::GetBlendShapeChannelCount() const
{
	return DNAReader->GetBlendShapeChannelCount();
}

FString FSkelMeshDNAReader::GetBlendShapeChannelName(uint16 Index) const
{
	return DNAReader->GetBlendShapeChannelName(Index);
}

uint16 FSkelMeshDNAReader::GetBlendShapeChannelIndexListCount() const
{
	return DNAReader->GetBlendShapeChannelIndexListCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetBlendShapeChannelIndicesForLOD(uint16 LOD) const
{
	return DNAReader->GetBlendShapeChannelIndicesForLOD(LOD);
}

uint16 FSkelMeshDNAReader::GetAnimatedMapCount() const
{
	return DNAReader->GetAnimatedMapCount();
}

FString FSkelMeshDNAReader::GetAnimatedMapName(uint16 Index) const
{
	return DNAReader->GetAnimatedMapName(Index);
}

uint16 FSkelMeshDNAReader::GetAnimatedMapIndexListCount() const
{
	return DNAReader->GetAnimatedMapIndexListCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetAnimatedMapIndicesForLOD(uint16 LOD) const
{
	return DNAReader->GetAnimatedMapIndicesForLOD(LOD);
}

uint16 FSkelMeshDNAReader::GetMeshCount() const
{
	return DNAReader->GetMeshCount();
}

FString FSkelMeshDNAReader::GetMeshName(uint16 Index) const
{
	return DNAReader->GetMeshName(Index);
}

uint16 FSkelMeshDNAReader::GetMeshIndexListCount() const
{
	return DNAReader->GetMeshIndexListCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMeshIndicesForLOD(uint16 LOD) const
{
	return DNAReader->GetMeshIndicesForLOD(LOD);
}

uint16 FSkelMeshDNAReader::GetMeshBlendShapeChannelMappingCount() const
{
	return DNAReader->GetMeshBlendShapeChannelMappingCount();
}

FMeshBlendShapeChannelMapping FSkelMeshDNAReader::GetMeshBlendShapeChannelMapping(uint16 Index) const
{
	return DNAReader->GetMeshBlendShapeChannelMapping(Index);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const
{
	return DNAReader->GetMeshBlendShapeChannelMappingIndicesForLOD(LOD);
}

FVector FSkelMeshDNAReader::GetNeutralJointTranslation(uint16 Index) const
{
	return DNAReader->GetNeutralJointTranslation(Index);
}

FVector FSkelMeshDNAReader::GetNeutralJointRotation(uint16 Index) const
{
	return DNAReader->GetNeutralJointRotation(Index);
}

uint16 FSkelMeshDNAReader::GetJointParentIndex(uint16 Index) const
{
	return DNAReader->GetJointParentIndex(Index);
}

/** BEHAVIOR READER **/

TArrayView<const uint16> FSkelMeshDNAReader::GetGUIToRawInputIndices() const
{
	return DNAReader->GetGUIToRawInputIndices();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetGUIToRawOutputIndices() const
{
	return DNAReader->GetGUIToRawOutputIndices();
}

TArrayView<const float> FSkelMeshDNAReader::GetGUIToRawFromValues() const
{
	return DNAReader->GetGUIToRawFromValues();
}

TArrayView<const float> FSkelMeshDNAReader::GetGUIToRawToValues() const
{
	return  DNAReader->GetGUIToRawToValues();
}

TArrayView<const float> FSkelMeshDNAReader::GetGUIToRawSlopeValues() const
{
	return DNAReader->GetGUIToRawSlopeValues();
}

TArrayView<const float> FSkelMeshDNAReader::GetGUIToRawCutValues() const
{
	return DNAReader->GetGUIToRawCutValues();
}

uint16 FSkelMeshDNAReader::GetPSDCount() const
{
	return DNAReader->GetPSDCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetPSDRowIndices() const
{
	return DNAReader->GetPSDRowIndices();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetPSDColumnIndices() const
{
	return DNAReader->GetPSDColumnIndices();
}

TArrayView<const float> FSkelMeshDNAReader::GetPSDValues() const
{
	return DNAReader->GetPSDValues();
}

uint16 FSkelMeshDNAReader::GetJointRowCount() const
{
	return DNAReader->GetJointRowCount();
}

uint16 FSkelMeshDNAReader::GetJointColumnCount() const
{
	return DNAReader->GetJointColumnCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetJointGroupJointIndices(uint16 JointGroupIndex) const
{
	return DNAReader->GetJointGroupJointIndices(JointGroupIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetJointVariableAttributeIndices(uint16 LOD) const
{
	return DNAReader->GetJointVariableAttributeIndices(LOD);
}

uint16 FSkelMeshDNAReader::GetJointGroupCount() const
{
	return DNAReader->GetJointGroupCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetJointGroupLODs(uint16 JointGroupIndex) const
{
	return DNAReader->GetJointGroupLODs(JointGroupIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetJointGroupInputIndices(uint16 JointGroupIndex) const
{
	return DNAReader->GetJointGroupInputIndices(JointGroupIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetJointGroupOutputIndices(uint16 JointGroupIndex) const
{
	return DNAReader->GetJointGroupOutputIndices(JointGroupIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetJointGroupValues(uint16 JointGroupIndex) const
{
	return DNAReader->GetJointGroupValues(JointGroupIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetBlendShapeChannelLODs() const
{
	return DNAReader->GetBlendShapeChannelLODs();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetBlendShapeChannelInputIndices() const
{
	return DNAReader->GetBlendShapeChannelInputIndices();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetBlendShapeChannelOutputIndices() const
{
	return DNAReader->GetBlendShapeChannelOutputIndices();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetAnimatedMapLODs() const
{
	return DNAReader->GetAnimatedMapLODs();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetAnimatedMapInputIndices() const
{
	return DNAReader->GetAnimatedMapInputIndices();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetAnimatedMapOutputIndices() const
{
	return DNAReader->GetAnimatedMapOutputIndices();
}

TArrayView<const float> FSkelMeshDNAReader::GetAnimatedMapFromValues() const
{
	return DNAReader->GetAnimatedMapFromValues();
}

TArrayView<const float> FSkelMeshDNAReader::GetAnimatedMapToValues() const
{
	return DNAReader->GetAnimatedMapToValues();
}

TArrayView<const float> FSkelMeshDNAReader::GetAnimatedMapSlopeValues() const
{
	return DNAReader->GetAnimatedMapSlopeValues();
}

TArrayView<const float> FSkelMeshDNAReader::GetAnimatedMapCutValues() const
{
	return DNAReader->GetAnimatedMapCutValues();
}

/** GEOMETRY READER **/
uint32 FSkelMeshDNAReader::GetVertexPositionCount(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexPositionCount(MeshIndex);
	}
	return {};
}

FVector FSkelMeshDNAReader::GetVertexPosition(uint16 MeshIndex, uint32 PositionIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexPosition(MeshIndex, PositionIndex);
	}
	return {};
}

TArrayView<const float> FSkelMeshDNAReader::GetVertexPositionXs(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexPositionXs(MeshIndex);
	}
	return {};
}

TArrayView<const float> FSkelMeshDNAReader::GetVertexPositionYs(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexPositionYs(MeshIndex);
	}
	return {};
}

TArrayView<const float> FSkelMeshDNAReader::GetVertexPositionZs(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexPositionZs(MeshIndex);
	}
	return {};
}

uint32 FSkelMeshDNAReader::GetVertexTextureCoordinateCount(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexTextureCoordinateCount(MeshIndex);
	}
	return {};
}

FTextureCoordinate FSkelMeshDNAReader::GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexTextureCoordinate(MeshIndex, TextureCoordinateIndex);
	}
	return {};
}

uint32 FSkelMeshDNAReader::GetVertexNormalCount(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexNormalCount(MeshIndex);
	}
	return {};
}

FVector FSkelMeshDNAReader::GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexNormal(MeshIndex, NormalIndex);
	}
	return {};
}

/* not needed for gene splicer */
uint32 FSkelMeshDNAReader::GetVertexLayoutCount(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexLayoutCount(MeshIndex);
	}
	return {};
}

/* not needed for gene splicer */
FVertexLayout FSkelMeshDNAReader::GetVertexLayout(uint16 MeshIndex, uint32 VertexIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetVertexLayout(MeshIndex, VertexIndex);
	}
	return {};
}

/* not needed for gene splicer */
uint32 FSkelMeshDNAReader::GetFaceCount(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetFaceCount(MeshIndex);
	}
	return {};
}

/* not needed for gene splicer */
TArrayView<const uint32> FSkelMeshDNAReader::GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetFaceVertexLayoutIndices(MeshIndex, FaceIndex);
	}
	return {};
}

uint16 FSkelMeshDNAReader::GetMaximumInfluencePerVertex(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetMaximumInfluencePerVertex(MeshIndex);
	}
	return {};
}

uint32 FSkelMeshDNAReader::GetSkinWeightsCount(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetSkinWeightsCount(MeshIndex);
	}
	return {};
}

TArrayView<const float> FSkelMeshDNAReader::GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetSkinWeightsValues(MeshIndex, VertexIndex);
	}
	return {};
}

TArrayView<const uint16> FSkelMeshDNAReader::GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetSkinWeightsJointIndices(MeshIndex, VertexIndex);
	}
	return {};
}

uint16 FSkelMeshDNAReader::GetBlendShapeTargetCount(uint16 MeshIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeTargetCount(MeshIndex);
	}
	return {};
}

uint16 FSkelMeshDNAReader::GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeChannelIndex(MeshIndex, BlendShapeTargetIndex);
	}
	return {};
}

uint32 FSkelMeshDNAReader::GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeTargetDeltaCount(MeshIndex, BlendShapeIndex);
	}
	return {};
}

FVector FSkelMeshDNAReader::GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeIndex, uint32 DeltaIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeTargetDelta(MeshIndex, BlendShapeIndex, DeltaIndex);
	}
	return {};
}

TArrayView<const float> FSkelMeshDNAReader::GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeTargetDeltaXs(MeshIndex, BlendShapeTargetIndex);
	}
	return {};
}

TArrayView<const float> FSkelMeshDNAReader::GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeTargetDeltaYs(MeshIndex, BlendShapeTargetIndex);
	}
	return {};
}

TArrayView<const float> FSkelMeshDNAReader::GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeTargetDeltaZs(MeshIndex, BlendShapeTargetIndex);
	}
	return {};
}

TArrayView<const uint32> FSkelMeshDNAReader::GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeIndex) const
{
	if (DNAReader)
	{
		return DNAReader->GetBlendShapeTargetVertexIndices(MeshIndex, BlendShapeIndex);
	}
	return {};
}

/** MACHINE LEARNED BEHAVIOR READER **/
uint16 FSkelMeshDNAReader::GetMLControlCount() const
{
	return DNAReader->GetMLControlCount();
}

FString FSkelMeshDNAReader::GetMLControlName(uint16 Index) const
{
	return DNAReader->GetMLControlName(Index);
}

uint16 FSkelMeshDNAReader::GetNeuralNetworkCount() const
{
	return DNAReader->GetNeuralNetworkCount();
}

uint16 FSkelMeshDNAReader::GetNeuralNetworkIndexListCount() const
{
	return DNAReader->GetNeuralNetworkIndexListCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetNeuralNetworkIndicesForLOD(uint16 LOD) const
{
	return DNAReader->GetNeuralNetworkIndicesForLOD(LOD);
}

uint16 FSkelMeshDNAReader::GetMeshRegionCount(uint16 MeshIndex) const
{
	return DNAReader->GetMeshRegionCount(MeshIndex);
}

FString FSkelMeshDNAReader::GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const
{
	return DNAReader->GetMeshRegionName(MeshIndex, RegionIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const
{
	return DNAReader->GetNeuralNetworkIndicesForMeshRegion(MeshIndex, RegionIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetNeuralNetworkInputIndices(uint16 NetIndex) const
{
	return DNAReader->GetNeuralNetworkInputIndices(NetIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetNeuralNetworkOutputIndices(uint16 NetIndex) const
{
	return DNAReader->GetNeuralNetworkOutputIndices(NetIndex);
}

uint16 FSkelMeshDNAReader::GetNeuralNetworkLayerCount(uint16 NetIndex) const
{
	return DNAReader->GetNeuralNetworkLayerCount(NetIndex);
}

EActivationFunction FSkelMeshDNAReader::GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const
{
	return DNAReader->GetNeuralNetworkLayerActivationFunction(NetIndex, LayerIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const
{
	return DNAReader->GetNeuralNetworkLayerActivationFunctionParameters(NetIndex, LayerIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const
{
	return DNAReader->GetNeuralNetworkLayerBiases(NetIndex, LayerIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const
{
	return DNAReader->GetNeuralNetworkLayerWeights(NetIndex, LayerIndex);
}

ETranslationRepresentation FSkelMeshDNAReader::GetJointTranslationRepresentation(uint16 JointIndex) const
{
	return DNAReader->GetJointTranslationRepresentation(JointIndex);
}

ERotationRepresentation FSkelMeshDNAReader::GetJointRotationRepresentation(uint16 JointIndex) const
{
	return DNAReader->GetJointRotationRepresentation(JointIndex);
}

EScaleRepresentation FSkelMeshDNAReader::GetJointScaleRepresentation(uint16 JointIndex) const
{
	return DNAReader->GetJointScaleRepresentation(JointIndex);
}

uint16 FSkelMeshDNAReader::GetRBFPoseCount() const
{
	return DNAReader->GetRBFPoseCount();
}

FString FSkelMeshDNAReader::GetRBFPoseName(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseName(PoseIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFPoseJointOutputIndices(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseJointOutputIndices(PoseIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseBlendShapeChannelOutputIndices(PoseIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseAnimatedMapOutputIndices(PoseIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetRBFPoseJointOutputValues(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseJointOutputValues(PoseIndex);
}

float FSkelMeshDNAReader::GetRBFPoseScale(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseScale(PoseIndex);
}

uint16 FSkelMeshDNAReader::GetRBFPoseControlCount() const
{
	return DNAReader->GetRBFPoseControlCount();
}

FString FSkelMeshDNAReader::GetRBFPoseControlName(uint16 PoseControlIndex) const
{
	return DNAReader->GetRBFPoseControlName(PoseControlIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFPoseInputControlIndices(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseInputControlIndices(PoseIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFPoseOutputControlIndices(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseOutputControlIndices(PoseIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetRBFPoseOutputControlWeights(uint16 PoseIndex) const
{
	return DNAReader->GetRBFPoseOutputControlWeights(PoseIndex);
}

uint16 FSkelMeshDNAReader::GetRBFSolverCount() const
{
	return DNAReader->GetRBFSolverCount();
}

uint16 FSkelMeshDNAReader::GetRBFSolverIndexListCount() const
{
	return DNAReader->GetRBFSolverIndexListCount();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFSolverIndicesForLOD(uint16 LOD) const
{
	return DNAReader->GetRBFSolverIndicesForLOD(LOD);
}

FString FSkelMeshDNAReader::GetRBFSolverName(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverName(SolverIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFSolverRawControlIndices(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverRawControlIndices(SolverIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetRBFSolverPoseIndices(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverPoseIndices(SolverIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetRBFSolverRawControlValues(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverRawControlValues(SolverIndex);
}

ERBFSolverType FSkelMeshDNAReader::GetRBFSolverType(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverType(SolverIndex);
}

float FSkelMeshDNAReader::GetRBFSolverRadius(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverRadius(SolverIndex);
}

EAutomaticRadius FSkelMeshDNAReader::GetRBFSolverAutomaticRadius(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverAutomaticRadius(SolverIndex);
}

float FSkelMeshDNAReader::GetRBFSolverWeightThreshold(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverWeightThreshold(SolverIndex);
}

ERBFDistanceMethod FSkelMeshDNAReader::GetRBFSolverDistanceMethod(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverDistanceMethod(SolverIndex);
}

ERBFNormalizeMethod FSkelMeshDNAReader::GetRBFSolverNormalizeMethod(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverNormalizeMethod(SolverIndex);
}

ERBFFunctionType FSkelMeshDNAReader::GetRBFSolverFunctionType(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverFunctionType(SolverIndex);
}

ETwistAxis FSkelMeshDNAReader::GetRBFSolverTwistAxis(uint16 SolverIndex) const
{
	return DNAReader->GetRBFSolverTwistAxis(SolverIndex);
}

uint16 FSkelMeshDNAReader::GetTwistCount() const
{
	return DNAReader->GetTwistCount();
}

ETwistAxis FSkelMeshDNAReader::GetTwistSetupTwistAxis(uint16 TwistIndex) const
{
	return DNAReader->GetTwistSetupTwistAxis(TwistIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetTwistInputControlIndices(uint16 TwistIndex) const
{
	return DNAReader->GetTwistInputControlIndices(TwistIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetTwistOutputJointIndices(uint16 TwistIndex) const
{
	return DNAReader->GetTwistOutputJointIndices(TwistIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetTwistBlendWeights(uint16 TwistIndex) const
{
	return DNAReader->GetTwistBlendWeights(TwistIndex);
}

uint16 FSkelMeshDNAReader::GetSwingCount() const
{
	return DNAReader->GetSwingCount();
}

ETwistAxis FSkelMeshDNAReader::GetSwingSetupTwistAxis(uint16 SwingIndex) const
{
	return DNAReader->GetSwingSetupTwistAxis(SwingIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetSwingInputControlIndices(uint16 SwingIndex) const
{
	return DNAReader->GetSwingInputControlIndices(SwingIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetSwingOutputJointIndices(uint16 SwingIndex) const
{
	return DNAReader->GetSwingOutputJointIndices(SwingIndex);
}

TArrayView<const float> FSkelMeshDNAReader::GetSwingBlendWeights(uint16 SwingIndex) const
{
	return DNAReader->GetSwingBlendWeights(SwingIndex);
}

uint16 FSkelMeshDNAReader::GetMLTypeCount() const
{
	return DNAReader->GetMLTypeCount();
}

uint16 FSkelMeshDNAReader::GetMLOperationSetCount(uint16 MLTypeIndex) const
{
	return DNAReader->GetMLOperationSetCount(MLTypeIndex);
}

uint16 FSkelMeshDNAReader::GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return DNAReader->GetMLOperationCount(MLTypeIndex, MLOperationSetIndex);
}

EMachineLearnedBehaviorOperationType FSkelMeshDNAReader::GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return static_cast<EMachineLearnedBehaviorOperationType>(DNAReader->GetMLOperationType(MLTypeIndex, MLOperationSetIndex, MLOperationIndex));
}

TArrayView<const uint32> FSkelMeshDNAReader::GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return DNAReader->GetMLOperationParameters(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return DNAReader->GetMLOperationDependencyOperationSetIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return DNAReader->GetMLOperationDependencyOperationIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
}

uint16 FSkelMeshDNAReader::GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return DNAReader->GetMLOperationIndexListCount(MLTypeIndex, MLOperationSetIndex);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const
{
	return DNAReader->GetMLOperationIndicesForLOD(MLTypeIndex, MLOperationSetIndex, LOD);
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMLJointsInputIndices() const
{
	return DNAReader->GetMLJointsInputIndices();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMLJointsOutputIndices() const
{
	return DNAReader->GetMLJointsOutputIndices();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMLJointsParameterKeys() const
{
	return DNAReader->GetMLJointsParameterKeys();
}

TArrayView<const uint16> FSkelMeshDNAReader::GetMLJointsParameterValues() const
{
	return DNAReader->GetMLJointsParameterValues();
}

void FSkelMeshDNAReader::Unload(EDNADataLayer Layer)
{
	ensureMsgf(false, TEXT("Assest are not unloadable"));
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FDNAConfig& FSkelMeshDNAReader::GetConfig() const
{
	return DNAReader->GetConfig();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
