// Copyright Epic Games, Inc. All Rights Reserved.
#include "SkelMeshDNAUtils.h"
#include "DNAToSkelMeshMap.h"
#include "DNAAssetUserData.h"
#include "DNA.h"

#include "RenderResource.h"
#include "RHI.h"
#include "Async/ParallelFor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Misc/FileHelper.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/SkeletalMesh.h"
#include "AssetCompilingManager.h"
#include "ComponentReregisterContext.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#if WITH_EDITORONLY_DATA
#include "MeshUtilities.h"
#include "LODUtilities.h"
#endif // WITH_EDITORONLY_DATA
#include "Modules/ModuleManager.h"

#include "AnimationRuntime.h"
#include "BoneWeights.h"

#include "riglogic/RigLogic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkelMeshDNAUtils)

DEFINE_LOG_CATEGORY(LogDNAUtils);
/** compare based on base mesh source vertex indices */
struct FCompareMorphTargetDeltas
{
	FORCEINLINE bool operator()(const FMorphTargetDelta& A, const FMorphTargetDelta& B) const
	{
		return ((int32)A.SourceIdx - (int32)B.SourceIdx) < 0;
	}
};

USkelMeshDNAUtils::USkelMeshDNAUtils(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FDNAToSkelMeshMap* USkelMeshDNAUtils::CreateMapForUpdatingNeutralMesh(IDNAReader* InDNAReader, USkeletalMesh* InSkelMesh)
{
#if WITH_EDITORONLY_DATA
	FDNAToSkelMeshMap* DNAToSkelMeshMap = new FDNAToSkelMeshMap();

	//only vertex map is initialized in this pass so we can mix neutral meshes fast (e.g. on slider move);
	//playing animation on a such a mesh requires updating joints and skin weights
	//getting full quality animation requires mixing morph targets too
	DNAToSkelMeshMap->InitBaseMesh(InDNAReader, InSkelMesh);
	return DNAToSkelMeshMap;
#else
	return nullptr;
#endif // WITH_EDITORONLY_DATA
}

FDNAToSkelMeshMap* USkelMeshDNAUtils::CreateMapForUpdatingNeutralMesh(USkeletalMesh* InSkelMesh)
{
#if WITH_EDITORONLY_DATA
	FDNAToSkelMeshMap* DNAToSkelMeshMap = new FDNAToSkelMeshMap();

	DNAToSkelMeshMap->InitFromDNAAsset(InSkelMesh);
	return DNAToSkelMeshMap;
#else
	return nullptr;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
/** Updates bind pose using joint positions from DNA. */
void USkelMeshDNAUtils::UpdateJoints(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap)
{
	{	// Scoping of RefSkelModifier
		FReferenceSkeletonModifier RefSkelModifier(InSkelMesh->GetRefSkeleton(), InSkelMesh->GetSkeleton());

		// copy here
		TArray<FTransform> RawBonePose = InSkelMesh->GetRefSkeleton().GetRawRefBonePose();

		// When we are mounting the head to different bodies than female average, we need to use
		// component space, as the joint to which the head root is snapped to will be on a different
		// position than in the head rig!

		// calculate component space ahead of current transform
		TArray<FTransform> ComponentTransforms;
		FAnimationRuntime::FillUpComponentSpaceTransforms(InSkelMesh->GetRefSkeleton(), RawBonePose, ComponentTransforms);

		const TArray<FMeshBoneInfo>& RawBoneInfo = InSkelMesh->GetRefSkeleton().GetRawRefBoneInfo();

		// Skipping root joint (index 0) to avoid blinking of the mesh due to bounding box issue
		for (uint16 JointIndex = 0; JointIndex < InDNAReader->GetJointCount(); JointIndex++)
		{
			int32 BoneIndex = InDNAToSkelMeshMap->GetUEBoneIndex(JointIndex);

			FTransform DNATransform = FTransform::Identity;

			// Updating bind pose affects just translations.
			const FVector JointTranslation = InDNAReader->GetNeutralJointTranslation(JointIndex);
			const FVector JointRotation = InDNAReader->GetNeutralJointRotation(JointIndex);
			const ERotationUnit RotationUnit = InDNAReader->GetRotationUnit();
			const auto Angle2Deg = [RotationUnit](float Value) { return (RotationUnit == ERotationUnit::Degrees) ? tdm::fdeg(Value).value : tdm::fdeg(tdm::frad(Value)).value; };
			const FQuat Rotation = FQuat::MakeFromEuler({Angle2Deg(JointRotation.X), Angle2Deg(JointRotation.Y), Angle2Deg(JointRotation.Z)});
			DNATransform.SetTranslation(JointTranslation);
			DNATransform.SetRotation(Rotation);
			// Joint 0 (spine_04) is a root of GeneSplicer joint hierarchy, and is a special case
			// 1) it is parent to itself
			// 3) the head joints below it in the skeletal mesh are not spliced, as they are not in DNA,
			//    so they will retain female average transforms
			if (InDNAReader->GetJointParentIndex(JointIndex) == JointIndex)  //parent to itself
			{
				ComponentTransforms[BoneIndex] = DNATransform;
			}
			else
			{
				if (ensure(RawBoneInfo[BoneIndex].ParentIndex != INDEX_NONE))
				{
					ComponentTransforms[BoneIndex] = DNATransform * ComponentTransforms[RawBoneInfo[BoneIndex].ParentIndex];
				}
			}

			ComponentTransforms[BoneIndex].NormalizeRotation();
		}

		for (uint16 BoneIndex = 0; BoneIndex < RawBoneInfo.Num(); BoneIndex++)
		{
			FTransform LocalTransform;

			if (BoneIndex == 0)
			{
				LocalTransform = ComponentTransforms[BoneIndex];
			}
			else
			{
				LocalTransform = ComponentTransforms[BoneIndex].GetRelativeTransform(ComponentTransforms[RawBoneInfo[BoneIndex].ParentIndex]);
			}

			LocalTransform.NormalizeRotation();

			RefSkelModifier.UpdateRefPoseTransform(BoneIndex, LocalTransform);
		}
	}

	InSkelMesh->GetRefBasesInvMatrix().Reset();
	InSkelMesh->CalculateInvRefMatrices(); // Needs to be called after RefSkelModifier is destroyed
}

/** Updates base mesh vertices using data from DNA. */
void USkelMeshDNAUtils::UpdateBaseMesh(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap, ELodUpdateOption InUpdateOption)
{

	FSkeletalMeshModel* ImportedModel = InSkelMesh->GetImportedModel();
	// Expects vertex map to be initialized beforehand	
	int32 LODStart;
	int32 LODRangeSize;
	GetLODRange(InUpdateOption, ImportedModel->LODModels.Num(), LODStart, LODRangeSize);
	for (int32 LODIndex = LODStart; LODIndex < LODRangeSize; LODIndex++)
	{
		FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
		int32 SectionIndex = 0;
		for (FSkelMeshSection& Section : LODModel.Sections)
		{
			int32& DNAMeshIndex = InDNAToSkelMeshMap->ImportVtxToDNAMeshIndex[LODIndex][Section.GetVertexBufferIndex()];

			const int32 NumSoftVertices = Section.GetNumVertices();
			auto& OverlappingMap = InDNAToSkelMeshMap->OverlappingVertices[LODIndex][SectionIndex];
			int32 VertexBufferIndex = Section.GetVertexBufferIndex();
			for (int32 VertexIndex = 0; VertexIndex < NumSoftVertices; VertexIndex++)
			{
				int32& DNAVertexIndex = InDNAToSkelMeshMap->ImportVtxToDNAVtxIndex[LODIndex][VertexBufferIndex];

				if (DNAVertexIndex >= 0)
				{
					const FVector Position = InDNAReader->GetVertexPosition(DNAMeshIndex, DNAVertexIndex);
					FSoftSkinVertex& Vertex = Section.SoftVertices[VertexIndex];
					Vertex.Position = FVector3f{ Position };

					// Check if the current vertex has overlapping vertices, and then update them as well.
					TArray<int32>& OverlappedIndices = OverlappingMap[VertexIndex];
					int32 OverlappingCount = OverlappedIndices.Num();
					for (int32 OverlappingIndex = 0; OverlappingIndex < OverlappingCount; ++OverlappingIndex)
					{
						int32 OverlappingVertexIndex = OverlappedIndices[OverlappingIndex];
						FSoftSkinVertex& OverlappingVertex = Section.SoftVertices[OverlappingVertexIndex];
						OverlappingVertex.Position = FVector3f{ Position };
					}
				}
				VertexBufferIndex++;
			}
			SectionIndex++;
		}
	}
}

/** Updates Morph Targets using Blend Shapes from DNA.  */
void USkelMeshDNAUtils::UpdateMorphTargets(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap, ELodUpdateOption InUpdateOption)
{
	TArray<FDNABlendShapeTarget>& MeshBlendShapeTargets = InDNAToSkelMeshMap->GetMeshBlendShapeTargets();
	if (MeshBlendShapeTargets.Num() == 0)
	{
		UE_LOGF(LogDNAUtils, Warning, "No morph targets updated!");
		return;
	}
	FSkeletalMeshModel* ImportedModel = InSkelMesh->GetImportedModel();

	uint16 ChannelCount = InDNAReader->GetBlendShapeChannelCount();
	ParallelFor(InSkelMesh->GetMorphTargets().Num(), [&](int32 MorphIndex)
		{
			UMorphTarget* MorphTarget = InSkelMesh->GetMorphTargets()[MorphIndex];
			const FDNABlendShapeTarget& MeshTarget = MeshBlendShapeTargets[MorphIndex];
			// First get all DNA deltas for current Morph Target.
			TArrayView<const uint32> BlendShapeVertexIndices = InDNAReader->GetBlendShapeTargetVertexIndices(MeshTarget.MeshIndex, MeshTarget.TargetIndex);
			const int32 NumOfDeltas = BlendShapeVertexIndices.Num();
			if (NumOfDeltas > 0)
			{
				int32 LODStart;
				int32 LODRangeSize;
				GetLODRange(InUpdateOption, MorphTarget->GetMorphLODModels().Num(), LODStart, LODRangeSize);
				for (int32 LODIndex = LODStart; LODIndex < LODRangeSize; LODIndex++)
				{
					// MorphTarget vertex indices refer to full vertex index buffer of imported mesh.
					FMorphTargetLODModel& MorphLODModel = MorphTarget->GetMorphLODModels()[LODIndex];
					MorphLODModel.NumBaseMeshVerts = NumOfDeltas;
					MorphLODModel.bGeneratedByEngine = false;
					MorphLODModel.SectionIndices.Empty();
					MorphLODModel.Vertices.Reset(NumOfDeltas);
					TArrayView<const float> DeltaXs = InDNAReader->GetBlendShapeTargetDeltaXs(MeshTarget.MeshIndex, MeshTarget.TargetIndex);
					TArrayView<const float> DeltaYs = InDNAReader->GetBlendShapeTargetDeltaYs(MeshTarget.MeshIndex, MeshTarget.TargetIndex);
					TArrayView<const float> DeltaZs = InDNAReader->GetBlendShapeTargetDeltaZs(MeshTarget.MeshIndex, MeshTarget.TargetIndex);

					for (int32 DeltaIndex = 0; DeltaIndex < NumOfDeltas; DeltaIndex++)
					{
						FVector PositionDelta = FVector(DeltaXs[DeltaIndex], DeltaYs[DeltaIndex], DeltaZs[DeltaIndex]);
						FMorphTargetDelta MorphDelta;
						int32 DNAVertexIndex = BlendShapeVertexIndices[DeltaIndex];
						int32 UEVertexIndex = InDNAToSkelMeshMap->ImportDNAVtxToUEVtxIndex[LODIndex][MeshTarget.MeshIndex][DNAVertexIndex];
						if (UEVertexIndex > INDEX_NONE)
						{
							MorphDelta.SourceIdx = (uint32)UEVertexIndex;
							MorphDelta.PositionDelta = FVector3f{ PositionDelta };
							MorphDelta.TangentZDelta = FVector3f::ZeroVector;
							MorphLODModel.Vertices.Add(MorphDelta);
							// Find section indices that are involved in these morph deltas.
							int32 SectionIdx = InDNAToSkelMeshMap->UEVertexToSectionIndices[LODIndex][UEVertexIndex];
							if (!MorphLODModel.SectionIndices.Contains(SectionIdx) && SectionIdx > INDEX_NONE)
							{
								MorphLODModel.SectionIndices.Add(SectionIdx);
							}
						}
					}
					MorphLODModel.Vertices.Sort(FCompareMorphTargetDeltas());
				}
			}
			else
			{
				int32 LODStart;
				int32 LODRangeSize;
				GetLODRange(InUpdateOption, MorphTarget->GetMorphLODModels().Num(), LODStart, LODRangeSize);
				for (int32 LODIndex = LODStart; LODIndex < LODRangeSize; LODIndex++)
				{
					FMorphTargetLODModel& MorphLODModel = MorphTarget->GetMorphLODModels()[LODIndex];
					MorphLODModel.Reset();
				}
#ifdef DEBUG
				UE_LOGF(LogDNAUtils, Warning, " 0 deltas found for mesh %d and blend shape target %d", MeshTarget.MeshIndex, MeshTarget.TargetIndex);
#endif 
			}
		});
}

/* Updates Bone influences using Skin Weights from DNA. */
void USkelMeshDNAUtils::UpdateSkinWeights(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap, ELodUpdateOption InUpdateOption)
{
	FSkeletalMeshModel* ImportedModel = InSkelMesh->GetImportedModel();

	bool InfluenceMismatch = false;
	//Set a threshold a bit smaller then 1/255
	int32 LODStart;
	int32 LODRangeSize;
	GetLODRange(InUpdateOption, ImportedModel->LODModels.Num(), LODStart, LODRangeSize);
	for (int32 LODIndex = LODStart; LODIndex < LODRangeSize; ++LODIndex)
	{
		FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
		for (int32 SectionIndex = 0; SectionIndex < LODModel.Sections.Num(); SectionIndex++)
		{
			FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
			const int32 DNAMeshIndex = InDNAToSkelMeshMap->ImportVtxToDNAMeshIndex[LODIndex][Section.GetVertexBufferIndex()];

			const int32 NumEngineVertices = Section.GetNumVertices();
			for (int32 VertexIndex = 0; VertexIndex < NumEngineVertices; VertexIndex++)
			{
				const int32 VertexBufferIndex = VertexIndex + Section.GetVertexBufferIndex();
				const int32 DNAVertexIndex = InDNAToSkelMeshMap->ImportVtxToDNAVtxIndex[LODIndex][VertexBufferIndex];

				if (DNAVertexIndex < 0) continue; // Skip vertex not in DNA.

				TArrayView<const float> DNASkinWeights = InDNAReader->GetSkinWeightsValues(DNAMeshIndex, DNAVertexIndex);
				TArrayView<const uint16> DNASkinJoints = InDNAReader->GetSkinWeightsJointIndices(DNAMeshIndex, DNAVertexIndex);
				int32 SkinJointNum = DNASkinJoints.Num();

				FSoftSkinVertex& Vertex = Section.SoftVertices[VertexIndex];

				// Update skin weights only around eyes where blue channel is not 0.
				if (Vertex.Color.B == 0) {
					//UE_LOGF(LogDNAUtils, Log, "Skipping vertex UE: %d, DNA: %d", VertexIndex, DNAVertexIndex );
					continue;
				}
				
				FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES];
				float InfluenceWeights[MAX_TOTAL_INFLUENCES];

				int32 InfluenceIndex = 0;
				for (int32 Index = 0; Index < FMath::Min(SkinJointNum, MAX_TOTAL_INFLUENCES); ++Index)
				{
					// Find Engine bone for corresponding DNAJoint for the same influence.
					const int32 UpdatedBoneId = InDNAToSkelMeshMap->GetUEBoneIndex(DNASkinJoints[InfluenceIndex]);
					// BoneMap holds subset of bones belonging to current section.
					const int32 BoneMapIndex = Section.BoneMap.Find(UpdatedBoneId);

					// Update which bone in the subset influences this vertex.
					if (BoneMapIndex != INDEX_NONE)
					{
						InfluenceBones[InfluenceIndex] = BoneMapIndex;
						InfluenceWeights[InfluenceIndex] = DNASkinWeights[Index];
						InfluenceIndex++;
					}
				}

				using namespace UE::AnimationCore;
				FMemory::Memzero(Vertex.InfluenceBones);
				FMemory::Memzero(Vertex.InfluenceWeights);

				FBoneWeights BoneWeights = FBoneWeights::Create(InfluenceBones, InfluenceWeights, InfluenceIndex);
				InfluenceIndex = 0;
				for (FBoneWeight BoneWeight: BoneWeights)
				{
					Vertex.InfluenceBones[InfluenceIndex] = BoneWeight.GetBoneIndex();
					Vertex.InfluenceWeights[InfluenceIndex] = BoneWeight.GetRawWeight();
					InfluenceIndex++;
				}
			}
		}
	}
}

/** Rebuilds render data from LODModel and inits resources. */
void USkelMeshDNAUtils::RebuildRenderData(USkeletalMesh* InSkelMesh)
{
	FPlatformTime::InitTiming();

	// Before touching the render data, just make sure any asset depending on it are finished building.
	FAssetCompilingManager::Get().FinishCompilationForObjects({ InSkelMesh });

	double StartTime = FPlatformTime::Seconds();
	{
		InSkelMesh->FlushRenderState();
	}
	double TimeToFlush = FPlatformTime::Seconds();
	{
		FSkeletalMeshRenderData* RenderData = InSkelMesh->GetResourceForRendering();
		int32 LODIndex = 0;

		for (FSkeletalMeshLODRenderData& LODRenderData : RenderData->LODRenderData)
		{
			FSkeletalMeshLODModel& LODModelRef = InSkelMesh->GetImportedModel()->LODModels[LODIndex];
			for (int32 i = 0; i < LODModelRef.Sections.Num(); i++)
			{
				FSkelMeshSection& ModelSection = LODModelRef.Sections[i];
				ModelSection.CalcMaxBoneInfluences();
				ModelSection.CalcUse16BitBoneIndex();
			}

			const FSkeletalMeshLODModel* LODModelPtr = &LODModelRef;
			LODRenderData.BuildFromLODModel(LODModelPtr);
			LODIndex++;
		}
	}
	double TimeToRebuildModel = FPlatformTime::Seconds();
	{
		if (FApp::CanEverRender())
		{
			// Reinitialize the static mesh's resources.
			InSkelMesh->InitResources();
		}
	}
	double TimeToInitResources = FPlatformTime::Seconds();
	{
		// Re-register scope
		FScopedSkeletalMeshReregisterContexts ReregisterContext(InSkelMesh);
	}
}

void USkelMeshDNAUtils::RebuildRenderData_VertexPosition(USkeletalMesh* InSkelMesh, bool InRebuildTangents)
{
	if (!FApp::CanEverRender())
	{
		return;
	}

	{
		// Before touching the render data, just make sure any asset depending on it are finished building.
		FAssetCompilingManager::Get().FinishCompilationForObjects({ InSkelMesh });

		FSkeletalMeshModel* MeshModel = InSkelMesh->GetImportedModel();
		FSkeletalMeshRenderData* RenderData = InSkelMesh->GetResourceForRendering();

		for (int32 LODIdx = 0; LODIdx < RenderData->LODRenderData.Num(); ++LODIdx)
		{
			FSkeletalMeshLODModel& LODModel = MeshModel->LODModels[LODIdx];
			FSkeletalMeshLODRenderData& LODRenderData = RenderData->LODRenderData[LODIdx];

			ENQUEUE_RENDER_COMMAND(FSkelMeshDNAUpdatePositions)
				([&LODModel, &LODRenderData, InRebuildTangents](FRHICommandListImmediate& RHICmdList)
					{
						LLM_SCOPE(ELLMTag::SkeletalMesh);

						TArray<FSoftSkinVertex> Vertices;
						LODModel.GetVertices(Vertices);
						check(Vertices.Num() == LODRenderData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices());
						LODRenderData.StaticVertexBuffers.PositionVertexBuffer.Init(Vertices.Num());
						for (int32 i = 0; i < Vertices.Num(); i++)
						{
							LODRenderData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(i) = Vertices[i].Position;
							if (InRebuildTangents)
							{
								LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(i, Vertices[i].TangentX, Vertices[i].TangentY, Vertices[i].TangentZ);
							}
						}

						FPositionVertexBuffer& VertexBuffer = LODRenderData.StaticVertexBuffers.PositionVertexBuffer;
						void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetNumVertices() * VertexBuffer.GetStride(), RLM_WriteOnly);
						FMemory::Memcpy(VertexBufferData, VertexBuffer.GetVertexData(), VertexBuffer.GetNumVertices() * VertexBuffer.GetStride());
						RHICmdList.UnlockBuffer(VertexBuffer.VertexBufferRHI);

						if (InRebuildTangents)
						{
							FStaticMeshVertexBuffer& TangentsVertexBuffer = LODRenderData.StaticVertexBuffers.StaticMeshVertexBuffer;
							void* TangentsBufferData = RHICmdList.LockBuffer(TangentsVertexBuffer.TangentsVertexBuffer.VertexBufferRHI, 0, TangentsVertexBuffer.GetTangentSize(), RLM_WriteOnly);
							FMemory::Memcpy(TangentsBufferData, TangentsVertexBuffer.GetTangentData(), TangentsVertexBuffer.GetTangentSize());
							RHICmdList.UnlockBuffer(TangentsVertexBuffer.TangentsVertexBuffer.VertexBufferRHI);
						}
					});
		}
	}
}

void USkelMeshDNAUtils::UpdateJointBehavior(USkeletalMeshComponent* InSkelMeshComponent)
{
	//DNAAsset->SetBehaviorReader is called before calling this method
	//it is not here to avoid having DNAAsset in the API, as in future we might want
	//to generalize SkelMeshUpdate to be dna-independent

	//the rig behavior has changed, we need to force re-initializing of RigLogic
	//this will set RigLogic RigUnit to initial state
	InSkelMeshComponent->InitAnim(true);
}

UDNAAsset* USkelMeshDNAUtils::GetMeshDNA(USkeletalMesh* InSkelMesh)
{
	if (InSkelMesh)
	{
		return InSkelMesh->GetAssetUserData<UDNAAsset>();
	}

	return nullptr;
}

UDNA* USkelMeshDNAUtils::GetMeshDNAAsset(USkeletalMesh* InSkelMesh)
{
	if (InSkelMesh)
	{
		if (UDNAAssetUserData* DNAUserData = InSkelMesh->GetAssetUserData<UDNAAssetUserData>())
		{
			return DNAUserData->DNAAsset;
		}
	}
	return nullptr;
}

TSharedPtr<IDNAReader> USkelMeshDNAUtils::GetDNAReader(USkeletalMesh* InSkelMesh)
{
	if (InSkelMesh)
	{
		if (UDNA* DNAAsset = GetMeshDNAAsset(InSkelMesh))
		{
			return DNAAsset->GetDNAReader();
		}

		// Fallback to old DNA Asset
		if (UDNAAsset* OldDNAAsset = GetMeshDNA(InSkelMesh))
		{
			return OldDNAAsset->GetDNAReader();
		}
	}
	return nullptr;
}

void USkelMeshDNAUtils::SetDNAReader(USkeletalMesh* InSkelMesh, TSharedPtr<IDNAReader> InDNAReader, EDNACopyPolicy CopyPolicy, ERigLogicInitPolicy InitPolicy)
{
	if (InSkelMesh)
	{
		UDNA* DNAAsset = GetMeshDNAAsset(InSkelMesh);
		// Fallback to old DNA Asset (UDNAAsset is being phased out in favor of UDNA)
		UDNAAsset* OldDNAAsset = GetMeshDNA(InSkelMesh);

		// A skeletal mesh should only ever have one of UDNA or UDNAAsset attached. On asset
		// upgrade, UDNAAsset is replaced by UDNA. If both ever appear together, the InitPolicy
		// contract here is ambiguous: only one of them gets paired with a follow-up
		// RestoreLegacyUEMHCCompatibility at the call site, leaving the other with a stale or
		// uninitialized RigLogic context.
		ensureMsgf(!(DNAAsset && OldDNAAsset),
			TEXT("USkelMeshDNAUtils::SetDNAReader: skeletal mesh '%s' has both UDNA and UDNAAsset attached. ")
			TEXT("This is not expected -- callers using ERigLogicInitPolicy::Defer assume only one is present."),
			*InSkelMesh->GetName());

		if (DNAAsset)
		{
			DNAAsset->SetDNAReader(InDNAReader, CopyPolicy, InitPolicy);
		}

		if (OldDNAAsset)
		{
			OldDNAAsset->SetDNAReader(InDNAReader, CopyPolicy, InitPolicy);
		}
	}
}

#endif // WITH_EDITORONLY_DATA
