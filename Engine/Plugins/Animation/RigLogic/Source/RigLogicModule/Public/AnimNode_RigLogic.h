// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAReader.h"
#include "RigInstance.h"
#include "RigLogic.h"

#include "Animation/AnimNodeBase.h"
#include "Animation/SmartName.h"

#include "AnimNode_RigLogic.generated.h"

#define UE_API RIGLOGICMODULE_API

DECLARE_LOG_CATEGORY_EXTERN(LogRigLogicAnimNode, Log, All);

struct FSharedRigRuntimeContext;
struct FDNAIndexMapping;

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RigLogic : public FAnimNode_Base
{
public:
	GENERATED_USTRUCT_BODY()

	UE_API FAnimNode_RigLogic();
	UE_API ~FAnimNode_RigLogic();

	UE_API void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API void GatherDebugData(FNodeDebugData& DebugData) override;
	int32 GetLODThreshold() const override { return LODThreshold; }

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink AnimSequence;

	/*
	 * Since the order of anim curves may change even on a frame by frame basis, it is not safe to cache and
	 * rely on cached indices by default, but if the blueprints feeding anim curves into RigLogic are set up
	 * in a controlled manner, such that no such runtime changes are expected to the order or number of anim
	 * curves, caching may improve the performance of the node, especially in low-LOD evaluations.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RigLogic)
	bool CacheAnimCurveNames = false;

private:
	using FCachedIndexedCurve = TBaseBlendedCurve<FDefaultAllocator, UE::Anim::FCurveElementIndexed>;

private:
	UE_API void CacheVariableJointAttributes(const FBoneContainer& RequiredBones);
	UE_API void CacheDriverJoints(const FBoneContainer& RequiredBones);
	UE_API void CachePoseCurvesToRigLogicControlsMap(const FPoseContext& InputContext, const FCachedIndexedCurve& IndexedCurves, TArray<int32>& Indices);
	UE_API void UpdateRawControls(const FPoseContext& InputContext);
	UE_API void UpdateRawControlsCached(const FPoseContext& InputContext);
	UE_API void UpdateSparseDriverJointDrivenControlCurves(const FPoseContext& InputContext);
	UE_API void UpdateDenseDriverJointDrivenControlCurves(const FPoseContext& InputContext);
	UE_API void UpdateMLMaskCurves(const FPoseContext& InputContext);
	UE_API void UpdateMLMaskCurvesCached(const FPoseContext& InputContext);
	UE_API void UpdateControlCurves(const FPoseContext& InputContext);
	UE_API void CalculateRigLogic();
	UE_API void UpdateJoints(FPoseContext& OutputContext);
	UE_API void UpdateBlendShapeCurves(FPoseContext& OutputContext);
	UE_API void UpdateAnimMapCurves(FPoseContext& OutputContext);

private:
	struct FJointCompactPoseBoneMapping
	{
		uint16 JointIndex;
		FCompactPoseBoneIndex CompactPoseBoneIndex;
	};

	struct FCompactPoseBoneControlAttributeMapping
	{
		FCompactPoseBoneIndex CompactPoseBoneIndex;
		int32 DNAJointIndex;
		int32 RotationX;
		int32 RotationY;
		int32 RotationZ;
		int32 RotationW;
	};

	struct FCachedJointMapping
	{
		TArray<FJointCompactPoseBoneMapping> JointsMapDNAIndicesToCompactPoseBoneIndices;
		TArray<FCompactPoseBoneControlAttributeMapping> SparseDriverJointsToControlAttributesMap;
		TArray<FCompactPoseBoneControlAttributeMapping> DenseDriverJointsToControlAttributesMap;
		int32 BoneCount = -1;
	};

	template <typename T>
	struct TArrayWrapper
	{
		TArray<T> Values;
	};

	struct FCurveElementControlAttributeMapping
	{
		TArray<int32> RawControlIndices;
		TArray<int32> MLMaskCurveIndices;
	};

private:
	/*
	 * Max LOD level that RigLogic Node is evaluated.
	 * For example if you have the threshold set to 2, it will evaluate until including LOD 2 (based on 0 index). In case the LOD level gets set to 3, it will stop evaluating the Rig Logic.
	 * Setting it to -1 will always evaluate it.
	 */
	UPROPERTY(EditAnywhere, Category = RigLogic, meta = (PinHiddenByDefault))
	int32 LODThreshold = INDEX_NONE;

private:
	TSharedPtr<FSharedRigRuntimeContext> LocalRigRuntimeContext;
	TSharedPtr<FDNAIndexMapping> LocalDNAIndexMapping;
	FRigInstance* RigInstance;
	TArray<FCachedJointMapping> LocalJointMappingsPerLOD;
	TArray<FCurveElementControlAttributeMapping> PoseCurvesToRigLogicControlsMap;
};

#undef UE_API
