// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectSaveContext.h"

#include "Misc/TransactionallySafeRWLock.h"

#include "Engine/AssetUserData.h"
#include "Interfaces/Interface_AssetUserData.h"

#include "DNACommon.h"
#include "RigLogic.h"

#include "DNA.generated.h"

#define UE_API RIGLOGICMODULE_API

DECLARE_LOG_CATEGORY_EXTERN(LogDNA, Log, All);

class IDNAReader;
class IBehaviorReader;
class IGeometryReader;
class FRigLogicMemoryStream;
class UAssetUserData;
class USkeleton;
class USkeletalMesh;
class USkeletalMeshComponent;
struct FDNAIndexMapping;
struct FSharedRigRuntimeContext;
enum class EDNADataLayer: uint8;

 /** An asset holding the data needed to generate/update/animate a RigLogic character
  * It is imported from character's DNA file as a bit stream, and separated out it into runtime (behavior) and design-time chunks;
  * Currently, the design-time part still loads the geometry, as it is needed for the skeletal mesh update; once SkeletalMeshDNAReader is
  * fully implemented, it will be able to read the geometry directly from the SkeletalMesh and won't load it into this asset 
  **/
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "DNA"))
class UDNA : public UObject
{
	GENERATED_BODY()

public:
	UE_API UDNA();
	UE_API ~UDNA();

#if WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;
#endif

	UE_API TSharedPtr<IDNAReader> GetDNAReader();

	UPROPERTY(EditAnywhere, Category = "DNA")
	FDNAConfig DNAConfig;
	/** In non-editor builds, the DNA source data will be unloaded to save memory after the runtime
	  * data has been initialized from it.
	  * 
	  * Set this property to true to keep the DNA in memory, e.g. if you need to modify it at
	  * runtime. For most use cases, this shouldn't be needed.
	 **/
	UPROPERTY(EditAnywhere, Category = RuntimeSettings)
	bool bKeepDNAAfterInitialization;

	/** Instead of serializing the DNA data and initialize RigLogic from it in runtime, dump the RigLogic
	  * state directly into the cooked asset, skipping the expensive initialization at runtime entirely.
	 **/
	UPROPERTY(EditAnywhere, Category = RuntimeSettings)
	bool bUseOptimizedCooking = true;

	/** Collection of runtime metadata related to a specific character.
	  * The value field is a FString and requires casting for a derived types.
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RuntimeSettings)
	TMap<FString, FString> MetaData;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RuntimeSettings)
	FRigLogicConfiguration RigLogicConfiguration;

#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UE_DEPRECATED(5.8, "This function has been deprecated, use the overload that also accepts a DNAConfig.")
	UE_API bool Init(const FString& Filename);
	UE_API bool Init(const FString& Filename, const FDNAConfig& InDNAConfig);
	UE_API void Serialize(FArchive& Ar) override;
	UE_API void PreSave(FObjectPreSaveContext SaveContext);
	UE_API void PostLoad() override;

	/** Alias or Copy DNA Reader and optionally reinitialize RigLogic with the new DNA data.
	*
	* Pass EDNACopyPolicy::Alias to prevent creation of a deep-copy of the given DNA and
	* instead just take the given shared pointer as-is.
	*
	* Pass ERigLogicInitPolicy::Defer when this call is immediately followed by another
	* operation that rebuilds the runtime context (e.g. RestoreLegacyUEMHCCompatibility)
	* to avoid constructing an FRigLogic that will be thrown away.
	**/
	UE_API void SetDNAReader(TSharedPtr<IDNAReader> SourceDNAReader,
							 EDNACopyPolicy CopyPolicy = EDNACopyPolicy::Copy,
							 ERigLogicInitPolicy InitPolicy = ERigLogicInitPolicy::Initialize);

	/** Initialize this object for use at runtime from another instance that has already been
	  * initialized. 
	  * 
	  * Overwrites all member variables. Only data needed for runtime evaluation will be copied.
	  * 
	  * Performs a shallow copy, so the runtime data is shared between the two instances and the
	  * memory cost of the copied UDNA is very low.
	  *
	  * Note that the reference to the shared runtime data will be dropped if the source DNA is
	  * modified, so the two instances are effectively independent and can safely be modified or
	  * deleted without affecting the other.
	 **/
	UE_API void InitializeForRuntimeFrom(UDNA* Other);
	UE_API TSharedPtr<FSharedRigRuntimeContext> GetRigRuntimeContext();
	UE_API TSharedPtr<FDNAIndexMapping> GetDNAIndexMapping(const USkeleton* Skeleton, const USkeletalMesh* SkeletalMesh);
	/**
	 * Fallback for users who have not updated to the new conversion workflow.
	 * Converts DNA reader data back to its original source space (Maya coordinate space)
	 * and provides RigLogic with the same transformed data as in the legacy pipeline.
	**/
	UE_API void RestoreLegacyUEMHCCompatibility();

private:
	friend struct FAnimNode_RigLogic;
	friend struct FRigUnit_RigLogic;

	void InvalidateRigRuntimeContext();
	void InitializeRigRuntimeContext(TSharedPtr<IDNAReader> ReaderForRigLogic);

private:
	struct FSkeletalMeshSkeletonToDNAIndexMappingKey
	{
		TWeakObjectPtr<const USkeletalMesh> SkeletalMesh;
		TWeakObjectPtr<const USkeleton> Skeleton;

		bool operator==(const FSkeletalMeshSkeletonToDNAIndexMappingKey& Other) const
		{
			return (SkeletalMesh == Other.SkeletalMesh) && (Skeleton == Other.Skeleton);
		}
		bool operator!=(const FSkeletalMeshSkeletonToDNAIndexMappingKey& Other) const
		{
			return !(*this == Other);
		}
		friend uint32 GetTypeHash(const FSkeletalMeshSkeletonToDNAIndexMappingKey& Instance)
		{
			return HashCombine(GetTypeHash(Instance.SkeletalMesh), GetTypeHash(Instance.Skeleton));
		}
	};

private:
	// Synchronize DNA updates
	FTransactionallySafeRWLock DNAUpdateLock;

	// Synchronize Rig Runtime Context updates
	FTransactionallySafeRWLock RigRuntimeContextUpdateLock;

	// Synchronize DNA Index Mapping updates
	FTransactionallySafeRWLock DNAIndexMappingUpdateLock;

	// Keep ref-count to shared memory resource alive in case the DNA asset is GCed during engine shutdown
	TSharedPtr<rl4::MemoryResource> MemoryResource;

	/**
	*	Unified DNA Reader used for run-time execution of RigLogic and updating of SkeletalMesh geometry
	**/
	TSharedPtr<IDNAReader> DNAReader;

	/** Runtime data necessary for rig computations that is shared between
	  * multiple rig instances based on the same DNA.
	**/
	TSharedPtr<FSharedRigRuntimeContext> RigRuntimeContext;

	/** Container for Skeleton <-> DNAAsset index mappings
	  * The mapping object owners will be the SkeletalMeshes, and periodic cleanups will
	  * ensure that dead objects are deleted from the map.
	 **/
	TMap<FSkeletalMeshSkeletonToDNAIndexMappingKey, TSharedPtr<FDNAIndexMapping>> DNAIndexMappingContainer;

};

#undef UE_API
