// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "Engine/AssetUserData.h"
#include "Engine/SkeletalMesh.h"

#include "DNAAssetUserData.generated.h"


class UDNA;
class USkeletalMesh;


UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "MetaHuman DNA Data"))
class UDNAAssetUserData: public UAssetUserData 
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Asset", meta = (DisplayName = "DNA"))
	TObjectPtr<UDNA> DNAAsset;
};