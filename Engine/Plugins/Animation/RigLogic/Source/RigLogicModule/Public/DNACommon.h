// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PerPlatformProperties.h"
#include "RBF/RBFSolver.h"

#include "DNACommon.generated.h"

UENUM(BlueprintType)
enum class EArchetype: uint8
{
	Asian,
	Black,
	Caucasian,
	Hispanic,
	Alien,
	Other
};

UENUM(BlueprintType)
enum class EGender: uint8
{
	Male,
	Female,
	Other
};

UENUM(BlueprintType)
enum class ETranslationUnit: uint8
{
	CM,
	M
};

UENUM(BlueprintType)
enum class ERotationUnit: uint8
{
	Degrees,
	Radians
};

UENUM(BlueprintType)
enum class EDirection: uint8
{
	Left,
	Right,
	Up,
	Down,
	Front,
	Back
};

UENUM(BlueprintType)
enum class ERotationSequence: uint8
{
	XYZ,
	XZY,
	YXZ,
	YZX,
	ZXY,
	ZYX
};

UENUM(BlueprintType)
enum class ERotationDirection: uint8 {
	None = 0 UMETA(Hidden),
	Positive = 1,
	Negative = 255
};

// Face vertex winding order, viewed along the outward surface normal.
// CCW: cross product of consecutive face vertices points along the stored normal direction (Maya / OpenGL convention).
// CW:  cross product of consecutive face vertices opposes the stored normal direction (DirectX / UE renderer convention).
UENUM(BlueprintType)
enum class EFaceWindingOrder : uint8
{
	CCW = 0,
	CW = 1
};

UENUM(BlueprintType)
enum class ETranslationRepresentation : uint8
{
	Vector
};

UENUM(BlueprintType)
enum class ERotationRepresentation : uint8
{
	EulerAngles,
	Quaternion
};

UENUM(BlueprintType)
enum class EScaleRepresentation : uint8
{
	Vector
};

UENUM(BlueprintType)
enum class EMachineLearnedBehaviorOperationType : uint8
{
	Unspecified,
	Gather,
	Scatter,
	MLP,
	Average
};

/*
UENUM(BlueprintType)
enum class ERBFSolverType : uint8
{
	Additive,
	Interpolative
};

UENUM(BlueprintType)
enum class ERBFFunctionType : uint8
{
	Gaussian,
	Exponential,
	Linear,
	Cubic,
	Quintic,
};

UENUM(BlueprintType)
enum class ERBFDistanceMethod : uint8
{
	Euclidean,
	Quaternion,
	SwingAngle,
	TwistAngle,
};

UENUM(BlueprintType)
enum class ERBFNormalizeMethod : uint8
{
	OnlyNormalizeAboveOne,
	AlwaysNormalize
};
*/

UENUM(BlueprintType)
enum class EAutomaticRadius : uint8
{
	On,
	Off
};

UENUM(BlueprintType)
enum class ETwistAxis : uint8
{
	X,
	Y,
	Z
};

UENUM(BlueprintType)
enum class EDNADataLayer : uint8
{
	None,
	Descriptor = 1,
	Definition = 2 | Descriptor,  // Implicitly loads Descriptor
	Behavior = 4 | Definition,  // Implicitly loads Descriptor and Definition
	Geometry = 8 | Definition,  // Implicitly loads Descriptor and Definition
	GeometryWithoutBlendShapes = 16 | Definition,  // Implicitly loads Descriptor and Definition
	MachineLearnedBehavior = 32 | Definition,  // Implicitly loads Definition
	RBFBehavior = 64 | Behavior,  // Implicitly loads Behavior and all body-rig related layers
	All = RBFBehavior | Geometry | MachineLearnedBehavior
};
ENUM_CLASS_FLAGS(EDNADataLayer);

UENUM(BlueprintType, meta = (Bitflags))
enum class EDNADataLayerUIProxy : uint8
{
	Descriptor = 0,
	Definition = 1,
	Behavior = 2,
	Geometry = 3,
	GeometryNoBlendShapes = 4,
	MLBehavior = 5,
	RBFBehavior = 6
};

UENUM(BlueprintType)
enum class ECoordinateSystemTransformPolicy : uint8 {
	Preserve,
	Transform
};

/** Controls whether SetDNAReader rebuilds the RigLogic runtime context.
 *  Use Defer when SetDNAReader is immediately followed by another mutation
 *  (e.g. RestoreLegacyUEMHCCompatibility) that will rebuild the context anyway,
 *  to avoid the wasted FRigLogic construction.
 */
UENUM(BlueprintType)
enum class ERigLogicInitPolicy : uint8 {
	Initialize,
	Defer
};

/** Controls whether SetDNAReader takes ownership of an independent copy of the source reader
 *  or aliases the input shared pointer directly.
 *
 *  Copy (default): the asset performs a deep copy of the source reader. This isolates the 
 *  asset from caller-side mutation.
 *
 *  Alias: the asset stores the input TSharedPtr<IDNAReader> directly. Cheaper, but the asset's
 *  reader is then sensitive to caller-side mutation and retains all layers the source had.
 *  It is safe today because IDNAReader exposes only one non-const method (Unload),
 *  which is reachable only in non-editor builds. If a new mutator is added to IDNAReader,
 *  every Alias call site must be re-audited for cross-asset mutation hazards.
 */
UENUM(BlueprintType)
enum class EDNACopyPolicy : uint8 {
	Copy,
	Alias
};

UENUM(BlueprintType)
enum class EActivationFunction : uint8
{
	Linear,
	ReLU,
	LeakyReLU,
	Tanh,
	Sigmoid
};

USTRUCT(BlueprintType)
struct FRotationSign
{
	GENERATED_BODY()

	FRotationSign() : XAxis(), YAxis(), ZAxis()
	{
	}

	FRotationSign(ERotationDirection XAxis, ERotationDirection YAxis, ERotationDirection ZAxis) : XAxis(XAxis), YAxis(YAxis), ZAxis(ZAxis)
	{
	}

	bool operator==(const FRotationSign& Other) const
	{
		return (XAxis == Other.XAxis) && (YAxis == Other.YAxis) && (ZAxis == Other.ZAxis);
	}

	bool operator!=(const FRotationSign& Other) const
	{
		return !(*this == Other);
	}

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "RigLogic")
	ERotationDirection XAxis;
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "RigLogic")
	ERotationDirection YAxis;
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "RigLogic")
	ERotationDirection ZAxis;
};

USTRUCT(BlueprintType)
struct FCoordinateSystem
{
	GENERATED_BODY()

	FCoordinateSystem() : XAxis(), YAxis(), ZAxis()
	{
	}

	FCoordinateSystem(EDirection XAxis, EDirection YAxis, EDirection ZAxis) : XAxis(XAxis), YAxis(YAxis), ZAxis(ZAxis)
	{
	}

	bool operator==(const FCoordinateSystem& Other) const
	{
		return (XAxis == Other.XAxis) && (YAxis == Other.YAxis) && (ZAxis == Other.ZAxis);
	}

	bool operator!=(const FCoordinateSystem& Other) const
	{
		return !(*this == Other);
	}

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "RigLogic")
	EDirection XAxis;
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "RigLogic")
	EDirection YAxis;
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "RigLogic")
	EDirection ZAxis;
};

USTRUCT(BlueprintType)
struct FDNAConfig
{
	GENERATED_BODY()

	FDNAConfig() = default;

	FDNAConfig(EDNADataLayer Layers, FPerPlatformInt MaxLOD, FPerPlatformInt MinLOD, TArrayView<const uint16> ExactLODs, ECoordinateSystemTransformPolicy CoordinateSystemTransformPolicy, FCoordinateSystem CoordinateSystem, FRotationSign RotationSign, ERotationSequence RotationSequence) :
		Layers(static_cast<int32>(Layers)),
		MaxLODPerPlatform(MaxLOD),
		MinLODPerPlatform(MinLOD),
		ExactLODs(ExactLODs),
		CoordinateSystemTransformPolicy(CoordinateSystemTransformPolicy),
		CoordinateSystem(CoordinateSystem),
		RotationSign(RotationSign),
		RotationSequence(RotationSequence)
	{
	}

	static FDNAConfig Legacy(ECoordinateSystemTransformPolicy CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Preserve)
	{
		// Use old coordinate system parameters to match old behavior of already imported/serialized assets, preserving backward compatibility
		FDNAConfig DNAConfig;
		DNAConfig.CoordinateSystemTransformPolicy = CoordinateSystemTransformPolicy;
		DNAConfig.CoordinateSystem = {EDirection::Left, EDirection::Down, EDirection::Front};
		DNAConfig.RotationSign = {ERotationDirection::Positive, ERotationDirection::Positive, ERotationDirection::Positive};
		DNAConfig.RotationSequence = ERotationSequence::XYZ;
		DNAConfig.FaceWindingOrder = EFaceWindingOrder::CW;
		return DNAConfig;
	}

	static FDNAConfig Source(ECoordinateSystemTransformPolicy CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Preserve)
	{
		// Use Source (Maya) coordinate system (used for restoring a DNA to its original data)
		FDNAConfig DNAConfig;
		DNAConfig.CoordinateSystemTransformPolicy = CoordinateSystemTransformPolicy;
		DNAConfig.CoordinateSystem = {EDirection::Left, EDirection::Up, EDirection::Front};
		DNAConfig.RotationSequence = ERotationSequence::XYZ;
		DNAConfig.RotationSign = {ERotationDirection::Positive, ERotationDirection::Positive, ERotationDirection::Positive};
		DNAConfig.FaceWindingOrder = EFaceWindingOrder::CCW;
		return DNAConfig;
	}

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA", meta = (Bitmask, BitmaskEnum = "/Script/RigLogicModule.EDNADataLayerUIProxy"))
	int32 Layers = static_cast<int32>(EDNADataLayer::All);

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA", meta = (DisplayName = "Max LOD"))
	FPerPlatformInt MaxLODPerPlatform;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA", meta = (DisplayName = "Min LOD"))
	FPerPlatformInt MinLODPerPlatform = {static_cast<int32>(-1)};

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA")
	TArray<uint8> ExactLODs;
	// This is a hidden copy of the above blueprint-exposed exact LOD list so it can be safely converted to the lower level config
	UPROPERTY(Transient)
	mutable TArray<uint16> InternalExactLODs;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA")
	ECoordinateSystemTransformPolicy CoordinateSystemTransformPolicy = ECoordinateSystemTransformPolicy::Preserve;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA")
	FCoordinateSystem CoordinateSystem = {EDirection::Left, EDirection::Front, EDirection::Up};

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA")
	FRotationSign RotationSign = {ERotationDirection::Negative, ERotationDirection::Negative, ERotationDirection::Positive};

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA")
	ERotationSequence RotationSequence = ERotationSequence::XYZ;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "DNA")
	EFaceWindingOrder FaceWindingOrder = EFaceWindingOrder::CW;
};

UCLASS()
class UDNAConfigHolder : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FDNAConfig Config;
};

USTRUCT(BlueprintType)
struct FMeshBlendShapeChannelMapping
{
	GENERATED_BODY()

	FMeshBlendShapeChannelMapping() : MeshIndex(), BlendShapeChannelIndex()
	{
	}

	FMeshBlendShapeChannelMapping(int32 MeshIndex, int32 BlendShapeChannelIndex) : MeshIndex(MeshIndex), BlendShapeChannelIndex(BlendShapeChannelIndex)
	{
	}

	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	int32 MeshIndex;
	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	int32 BlendShapeChannelIndex;
};

USTRUCT(BlueprintType)
struct FTextureCoordinate
{
	GENERATED_BODY()

	FTextureCoordinate() : U(), V()
	{
	}

	FTextureCoordinate(float U, float V) : U(U), V(V)
	{
	}

	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	float U;
	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	float V;
};

USTRUCT(BlueprintType)
struct FVertexLayout
{
	GENERATED_BODY()

	FVertexLayout() : Position(), TextureCoordinate(), Normal()
	{
	}

	FVertexLayout(int32 Position, int32 TextureCoordinate, int32 Normal) : Position(Position), TextureCoordinate(TextureCoordinate), Normal(Normal)
	{
	}

	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	int32 Position;
	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	int32 TextureCoordinate;
	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	int32 Normal;
};
