// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoneIndices.h"
#include "Engine/AssetUserData.h"
#include "Animation/AnimCurveTypes.h"

#define UE_API RIGLOGICMODULE_API

class IDNAReader;
class USkeleton;
class USkeletalMesh;

struct FDNAIndexMapping
{
	template <typename T>
	struct TArrayWrapper
	{
		TArray<T> Values;
	};
	
	struct FMeshPoseBoneControlAttributeMapping
	{
		FMeshPoseBoneIndex MeshPoseBoneIndex;
		int32 DNAJointIndex;
		int32 RotationX;
		int32 RotationY;
		int32 RotationZ;
		int32 RotationW;
	};

	using FCachedIndexedCurve = TBaseBlendedCurve<FDefaultAllocator, UE::Anim::FCurveElementIndexed>;

	FGuid SkeletonGuid;

	// all the control attributes that we will need to extract, alongside their control index
	FCachedIndexedCurve ControlAttributeCurves;
	FCachedIndexedCurve MLMaskCurves;
	TArray<FMeshPoseBoneControlAttributeMapping> DriverJointsToControlAttributesMap;
	TArray<FMeshPoseBoneIndex> JointsMapDNAIndicesToMeshPoseBoneIndices;
	TArray<FCachedIndexedCurve> MorphTargetCurvesPerLOD;
	TArray<FCachedIndexedCurve> MaskMultiplierCurvesPerLOD;

	UE_API void Init(const IDNAReader* DNAReader, const USkeleton* Skeleton, const USkeletalMesh* SkeletalMesh);

	UE_API void MapControlCurves(const IDNAReader* DNAReader, const USkeleton* Skeleton);
	UE_API void MapMLMaskCurves(const IDNAReader* DNAReader, const USkeleton* Skeleton);
	UE_API void MapDriverJoints(const IDNAReader* DNAReader, const USkeletalMesh* SkeletalMesh);
	UE_API void MapJoints(const IDNAReader* DNAReader, const USkeletalMesh* SkeletalMesh);
	UE_API void MapMorphTargets(const IDNAReader* DNAReader, const USkeleton* Skeleton, const USkeletalMesh* SkeletalMesh);
	UE_API void MapMaskMultipliers(const IDNAReader* DNAReader, const USkeleton* Skeleton);

};

#undef UE_API
