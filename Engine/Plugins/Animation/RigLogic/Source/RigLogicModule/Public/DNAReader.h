// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "DNACommon.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDNAReader, Log, All);

namespace dna
{

class Reader;

}  // namespace dna

class IDNAReaderBase
{
protected:
	virtual ~IDNAReaderBase() = default;

public:
	virtual dna::Reader* Unwrap() const = 0;
	virtual void Unload(EDNADataLayer Layer) = 0;

	/** Returns the FDNAConfig the reader was constructed with, with all per-platform
	 *  fields resolved to scalar values for the host platform that built it.
	 *  Use this to compare host config against a cook-target config without going
	 *  through TPerPlatformProperty::GetValue() (which reads OnGetPreviewPlatform,
	 *  an FNotThreadSafeDelegate that the cook server may rebind concurrently).
	 */
	virtual const FDNAConfig& GetConfig() const = 0;

	/** Returns true if this reader is wrapped in the legacy adapter (FLegacyDNAReader),
	 *  which applies the legacy Maya->UE coordinate swizzle on geometry queries regardless
	 *  of FDNAConfig. Used by callers that must preserve UEMHC backwards-compatible behavior
	 *  (e.g. RestoreLegacyUEMHCCompatibility) to detect when re-wrapping is required.
	 */
	virtual bool IsLegacyWrapped() const { return false; }

};

class IHeaderReader : public IDNAReaderBase
{
protected:
	virtual ~IHeaderReader() = default;

public:
	virtual uint16 GetFileFormatGeneration() const = 0;
	virtual uint16 GetFileFormatVersion() const = 0;

};

class IDescriptorReader : public IHeaderReader
{
protected:
	virtual ~IDescriptorReader() = default;

public:
	virtual FString GetName() const = 0;
	virtual EArchetype GetArchetype() const = 0;
	virtual EGender GetGender() const = 0;
	virtual uint16 GetAge() const = 0;
	virtual uint32 GetMetaDataCount() const = 0;
	virtual FString GetMetaDataKey(uint32 Index) const = 0;
	virtual FString GetMetaDataValue(const FString& Key) const = 0;
	virtual ETranslationUnit GetTranslationUnit() const = 0;
	virtual ERotationUnit GetRotationUnit() const = 0;
	virtual FCoordinateSystem GetCoordinateSystem() const = 0;
	virtual ERotationSequence GetRotationSequence() const = 0;
	virtual FRotationSign GetRotationSign() const = 0;
	virtual EFaceWindingOrder GetFaceWindingOrder() const = 0;
	virtual uint16 GetLODCount() const = 0;
	virtual uint16 GetDBMaxLOD() const = 0;
	virtual FString GetDBComplexity() const = 0;
	virtual FString GetDBName() const = 0;

};

class IDefinitionReader : public IDescriptorReader
{
protected:
	virtual ~IDefinitionReader() = default;

public:
	virtual uint16 GetGUIControlCount() const = 0;
	virtual FString GetGUIControlName(uint16 Index) const = 0;
	virtual uint16 GetRawControlCount() const = 0;
	virtual FString GetRawControlName(uint16 Index) const = 0;
	virtual uint16 GetJointCount() const = 0;
	virtual FString GetJointName(uint16 Index) const = 0;
	virtual uint16 GetJointIndexListCount() const = 0;
	virtual TArrayView<const uint16> GetJointIndicesForLOD(uint16 LOD) const = 0;
	virtual uint16 GetJointParentIndex(uint16 Index) const = 0;
	virtual uint16 GetBlendShapeChannelCount() const = 0;
	virtual FString GetBlendShapeChannelName(uint16 Index) const = 0;
	virtual uint16 GetBlendShapeChannelIndexListCount() const = 0;
	virtual TArrayView<const uint16> GetBlendShapeChannelIndicesForLOD(uint16 LOD) const = 0;
	virtual uint16 GetAnimatedMapCount() const = 0;
	virtual FString GetAnimatedMapName(uint16 Index) const = 0;
	virtual uint16 GetAnimatedMapIndexListCount() const = 0;
	virtual TArrayView<const uint16> GetAnimatedMapIndicesForLOD(uint16 LOD) const = 0;
	virtual uint16 GetMeshCount() const = 0;
	virtual FString GetMeshName(uint16 Index) const = 0;
	virtual uint16 GetMeshIndexListCount() const = 0;
	virtual TArrayView<const uint16> GetMeshIndicesForLOD(uint16 LOD) const = 0;
	virtual uint16 GetMeshBlendShapeChannelMappingCount() const = 0;
	virtual FMeshBlendShapeChannelMapping GetMeshBlendShapeChannelMapping(uint16 Index) const = 0;
	virtual TArrayView<const uint16> GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const = 0;
	virtual FVector GetNeutralJointTranslation(uint16 Index) const = 0;
	virtual FVector GetNeutralJointRotation(uint16 Index) const = 0;
};

class IBehaviorReader : public virtual IDefinitionReader
{
protected:
	virtual ~IBehaviorReader() = default;

public:
	virtual TArrayView<const uint16> GetGUIToRawInputIndices() const = 0;
	virtual TArrayView<const uint16> GetGUIToRawOutputIndices() const = 0;
	virtual TArrayView<const float> GetGUIToRawFromValues() const = 0;
	virtual TArrayView<const float> GetGUIToRawToValues() const = 0;
	virtual TArrayView<const float> GetGUIToRawSlopeValues() const = 0;
	virtual TArrayView<const float> GetGUIToRawCutValues() const = 0;
	virtual uint16 GetPSDCount() const = 0;
	virtual TArrayView<const uint16> GetPSDRowIndices() const = 0;
	virtual TArrayView<const uint16> GetPSDColumnIndices() const = 0;
	virtual TArrayView<const float> GetPSDValues() const = 0;
	virtual uint16 GetJointRowCount() const = 0;
	virtual uint16 GetJointColumnCount() const = 0;
	virtual TArrayView<const uint16> GetJointVariableAttributeIndices(uint16 LOD) const = 0;
	virtual uint16 GetJointGroupCount() const = 0;
	virtual TArrayView<const uint16> GetJointGroupLODs(uint16 JointGroupIndex) const = 0;
	virtual TArrayView<const uint16> GetJointGroupInputIndices(uint16 JointGroupIndex) const = 0;
	virtual TArrayView<const uint16> GetJointGroupOutputIndices(uint16 JointGroupIndex) const = 0;
	virtual TArrayView<const float> GetJointGroupValues(uint16 JointGroupIndex) const = 0;
	virtual TArrayView<const uint16> GetJointGroupJointIndices(uint16 JointGroupIndex) const = 0;
	virtual TArrayView<const uint16> GetBlendShapeChannelLODs() const = 0;
	virtual TArrayView<const uint16> GetBlendShapeChannelOutputIndices() const = 0;
	virtual TArrayView<const uint16> GetBlendShapeChannelInputIndices() const = 0;
	virtual TArrayView<const uint16> GetAnimatedMapLODs() const = 0;
	virtual TArrayView<const uint16> GetAnimatedMapInputIndices() const = 0;
	virtual TArrayView<const uint16> GetAnimatedMapOutputIndices() const = 0;
	virtual TArrayView<const float> GetAnimatedMapFromValues() const = 0;
	virtual TArrayView<const float> GetAnimatedMapToValues() const = 0;
	virtual TArrayView<const float> GetAnimatedMapSlopeValues() const = 0;
	virtual TArrayView<const float> GetAnimatedMapCutValues() const = 0;
};

class IGeometryReader : public virtual IDefinitionReader
{
protected:
	virtual ~IGeometryReader() = default;

public:
	virtual uint32 GetVertexPositionCount(uint16 MeshIndex) const = 0;
	virtual FVector GetVertexPosition(uint16 MeshIndex, uint32 VertexIndex) const = 0;
	virtual TArrayView<const float> GetVertexPositionXs(uint16 MeshIndex) const = 0;
	virtual TArrayView<const float> GetVertexPositionYs(uint16 MeshIndex) const = 0;
	virtual TArrayView<const float> GetVertexPositionZs(uint16 MeshIndex) const = 0;
	virtual uint32 GetVertexTextureCoordinateCount(uint16 MeshIndex) const = 0;
	virtual FTextureCoordinate GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const = 0;
	virtual uint32 GetVertexNormalCount(uint16 MeshIndex) const = 0;
	virtual FVector GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const = 0;
	virtual uint32 GetFaceCount(uint16 MeshIndex) const = 0;
	virtual TArrayView<const uint32> GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const = 0;
	virtual uint32 GetVertexLayoutCount(uint16 MeshIndex) const = 0;
	virtual FVertexLayout GetVertexLayout(uint16 MeshIndex, uint32 LayoutIndex) const = 0;
	virtual uint16 GetMaximumInfluencePerVertex(uint16 MeshIndex) const = 0;
	virtual uint32 GetSkinWeightsCount(uint16 MeshIndex) const = 0;
	virtual TArrayView<const float> GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const = 0;
	virtual TArrayView<const uint16> GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const = 0;
	virtual uint16 GetBlendShapeTargetCount(uint16 MeshIndex) const = 0;
	virtual uint16 GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const = 0;
	virtual uint32 GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const = 0;
	virtual FVector GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeTargetIndex, uint32 DeltaIndex) const = 0;
	virtual TArrayView<const float> GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const = 0;
	virtual TArrayView<const float> GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const = 0;
	virtual TArrayView<const float> GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const = 0;
	virtual TArrayView<const uint32> GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const = 0;
};

class IMachineLearnedBehaviorReader : public virtual IDefinitionReader
{
protected:
	virtual ~IMachineLearnedBehaviorReader() = default;

public:
	virtual uint16 GetMLControlCount() const = 0;
	virtual FString GetMLControlName(uint16 Index) const = 0;
	virtual uint16 GetNeuralNetworkCount() const = 0;
	virtual uint16 GetNeuralNetworkIndexListCount() const = 0;
	virtual TArrayView<const uint16> GetNeuralNetworkIndicesForLOD(uint16 LOD) const = 0;
	virtual uint16 GetMeshRegionCount(uint16 MeshIndex) const = 0;
	virtual FString GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const = 0;
	virtual TArrayView<const uint16> GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const = 0;
	virtual TArrayView<const uint16> GetNeuralNetworkInputIndices(uint16 NetIndex) const = 0;
	virtual TArrayView<const uint16> GetNeuralNetworkOutputIndices(uint16 NetIndex) const = 0;
	virtual uint16 GetNeuralNetworkLayerCount(uint16 NetIndex) const = 0;
	virtual EActivationFunction GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const = 0;
	virtual TArrayView<const float> GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const = 0;
	virtual TArrayView<const float> GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const = 0;
	virtual TArrayView<const float> GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const = 0;

};

class IJointBehaviorMetadataReader : public virtual IDefinitionReader
{
protected:
	virtual ~IJointBehaviorMetadataReader() = default;

public:
	virtual ETranslationRepresentation GetJointTranslationRepresentation(uint16 JointIndex) const = 0;
	virtual ERotationRepresentation GetJointRotationRepresentation(uint16 JointIndex) const = 0;
	virtual EScaleRepresentation GetJointScaleRepresentation(uint16 JointIndex) const = 0;

};

class IRBFBehaviorReader : public virtual IBehaviorReader
{
protected:
	virtual ~IRBFBehaviorReader() = default;

public:
	virtual uint16 GetRBFPoseCount() const = 0;
	virtual FString GetRBFPoseName(uint16 PoseIndex) const = 0;
	virtual TArrayView<const uint16> GetRBFPoseJointOutputIndices(uint16 PoseIndex) const = 0;
	virtual TArrayView<const uint16> GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const = 0;
	virtual TArrayView<const uint16> GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const = 0;
	virtual TArrayView<const float> GetRBFPoseJointOutputValues(uint16 PoseIndex) const = 0;
	virtual float GetRBFPoseScale(uint16 PoseIndex) const = 0;
	virtual uint16 GetRBFPoseControlCount() const = 0;
	virtual FString GetRBFPoseControlName(uint16 PoseControlIndex) const = 0;
	virtual TArrayView<const uint16> GetRBFPoseInputControlIndices(uint16 PoseIndex) const = 0;
	virtual TArrayView<const uint16> GetRBFPoseOutputControlIndices(uint16 PoseIndex) const = 0;
	virtual TArrayView<const float> GetRBFPoseOutputControlWeights(uint16 PoseIndex) const = 0;
	virtual uint16 GetRBFSolverCount() const = 0;
	virtual uint16 GetRBFSolverIndexListCount() const = 0;
	virtual TArrayView<const uint16> GetRBFSolverIndicesForLOD(uint16 LOD) const = 0;
	virtual FString GetRBFSolverName(uint16 SolverIndex) const = 0;
	virtual TArrayView<const uint16> GetRBFSolverRawControlIndices(uint16 SolverIndex) const = 0;
	virtual TArrayView<const uint16> GetRBFSolverPoseIndices(uint16 SolverIndex) const = 0;
	virtual TArrayView<const float> GetRBFSolverRawControlValues(uint16 SolverIndex) const = 0;
	virtual ERBFSolverType GetRBFSolverType(uint16 SolverIndex) const = 0;
	virtual float GetRBFSolverRadius(uint16 SolverIndex) const = 0;
	virtual EAutomaticRadius GetRBFSolverAutomaticRadius(uint16 SolverIndex) const = 0;
	virtual float GetRBFSolverWeightThreshold(uint16 SolverIndex) const = 0;
	virtual ERBFDistanceMethod GetRBFSolverDistanceMethod(uint16 SolverIndex) const = 0;
	virtual ERBFNormalizeMethod GetRBFSolverNormalizeMethod(uint16 SolverIndex) const = 0;
	virtual ERBFFunctionType GetRBFSolverFunctionType(uint16 SolverIndex) const = 0;
	virtual ETwistAxis GetRBFSolverTwistAxis(uint16 SolverIndex) const = 0;

};

class ITwistSwingBehaviorReader : public virtual IDefinitionReader
{
protected:
	virtual ~ITwistSwingBehaviorReader() = default;

public:
	virtual uint16 GetTwistCount() const = 0;
	virtual ETwistAxis GetTwistSetupTwistAxis(uint16 TwistIndex) const = 0;
	virtual TArrayView<const uint16> GetTwistInputControlIndices(uint16 TwistIndex) const = 0;
	virtual TArrayView<const uint16> GetTwistOutputJointIndices(uint16 TwistIndex) const = 0;
	virtual TArrayView<const float> GetTwistBlendWeights(uint16 TwistIndex) const = 0;
	virtual uint16 GetSwingCount() const = 0;
	virtual ETwistAxis GetSwingSetupTwistAxis(uint16 SwingIndex) const = 0;
	virtual TArrayView<const uint16> GetSwingInputControlIndices(uint16 SwingIndex) const = 0;
	virtual TArrayView<const uint16> GetSwingOutputJointIndices(uint16 SwingIndex) const = 0;
	virtual TArrayView<const float> GetSwingBlendWeights(uint16 SwingIndex) const = 0;

};

class IMachineLearnedBehaviorExtReader : public virtual IMachineLearnedBehaviorReader
{
protected:
	virtual ~IMachineLearnedBehaviorExtReader() = default;

public:
	virtual uint16 GetMLTypeCount() const = 0;
	virtual uint16 GetMLOperationSetCount(uint16 MLTypeIndex) const = 0;
	virtual uint16 GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const = 0;
	virtual EMachineLearnedBehaviorOperationType GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const = 0;
	virtual TArrayView<const uint32> GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const = 0;
	virtual TArrayView<const uint16> GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const = 0;
	virtual TArrayView<const uint16> GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const = 0;
	virtual uint16 GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const = 0;
	virtual TArrayView<const uint16> GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const = 0;
	virtual TArrayView<const uint16> GetMLJointsInputIndices() const = 0;
	virtual TArrayView<const uint16> GetMLJointsOutputIndices() const = 0;
	virtual TArrayView<const uint16> GetMLJointsParameterKeys() const = 0;
	virtual TArrayView<const uint16> GetMLJointsParameterValues() const = 0;

};

/**
	@brief UE interface for DNA Reader wrappers.
*/
class IDNAReader : public IRBFBehaviorReader, public IGeometryReader, public IMachineLearnedBehaviorExtReader, public IJointBehaviorMetadataReader, public ITwistSwingBehaviorReader
{
public:
	virtual ~IDNAReader() = default;
};
