// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Serialization/Archive.h"
#include "UObject/NoExportTypes.h"

#include "DNAReader.h"

#define UE_API RIGLOGICMODULE_API

DECLARE_LOG_CATEGORY_EXTERN(LogDNAToSkelMeshMap, Log, All);

class USkeletalMesh;


struct FDNABlendShapeTarget
{
	FDNABlendShapeTarget()
		: MeshIndex(INDEX_NONE)
		, TargetIndex(INDEX_NONE)
	{
	}

	FDNABlendShapeTarget(const int16 InMeshIndex, const int16 InTargetIndex)
		: MeshIndex(InMeshIndex)
		, TargetIndex(InTargetIndex)
	{
	}

	int16 MeshIndex;
	int16 TargetIndex;
};

/** An object holding the mappings from DNA to UE indices, needed for updating a character from DNA file */
class FDNAToSkelMeshMap
{
public:

	FDNAToSkelMeshMap() {}

#if WITH_EDITORONLY_DATA
	/** Creates vertex map between source DNA and target SkelMesh, and stores DNAReader
	  * Enough for updating neutral meshes, but not for updating joints and morphtargets; call the next method for that
	  * TODO: When ImportedModel is added to runtime, WITH_EDITORONLY_DATA should be removed**/
	UE_API void InitBaseMesh(IDNAReader* InSourceDNAReader, USkeletalMesh* InTargetSkeletalMesh);
	/** Creates map for updating joints; needs to be called before playing animation on the rig **/
	UE_API void MapJoints(IDNAReader* InDNAReader);
	/** Number of DNA joints currently mapped to UE bone indices. **/
	int32 GetMappedJointCount() const
	{
		return RLJointToUEBoneIndices.Num();
	}
	/** Creates map for updating morph targets; needs to be called before playing animation on the rig **/
	UE_API void MapMorphTargets(IDNAReader* InDNAReader);
#endif //WITH_EDITORONLY_DATA
	/** Creates mappings between source DNA stored in DNAAsset AssetUserData attached to SkeletalMesh;
		Uses FSkeletalMeshDNAReader to be able to read geometry which, in this case, is NOT present in the DNAAsset **/
	UE_API bool InitFromDNAAsset(USkeletalMesh* InSkelMesh);
	UE_API void InitVertexMap(IDNAReader* InDNAReader);


	int32 GetUEBoneIndex(int32 InRLJointIndex)
	{
		return RLJointToUEBoneIndices[InRLJointIndex];
	}

	TArray<FDNABlendShapeTarget>& GetMeshBlendShapeTargets()
	{
		return MeshBlendShapeTargets;
	}

	TArray<TArray<int32>> ImportVtxToDNAMeshIndex;
	TArray<TArray<int32>> ImportVtxToDNAVtxIndex;
	TArray<TArray<TArray<int32>>> ImportDNAVtxToUEVtxIndex;
	/** Map for keeping all overlapping vertices for each vertex of each section of each LOD Model */
	TArray<TArray<TArray<TArray<int32>>>> OverlappingVertices;
	TArray<TArray<int32>> UEVertexToSectionIndices;

private:

	USkeletalMesh* TargetSkelMesh = nullptr;

	TArray<int32> RLJointToUEBoneIndices;  // UFC joint index to UE bone index
	TArray<FDNABlendShapeTarget> MeshBlendShapeTargets; // Reverse mapping.
	TArray<uint16> DNASkinnedJointIndices;

};

#undef UE_API
