// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAReader.h"

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

namespace trio
{

class BoundedIOStream;

}  // namespace trio

namespace dna
{

enum class DataLayer : uint32;
using trio::BoundedIOStream;

}  // namespace dna

RIGLOGICMODULE_API dna::DataLayer CalculateDNADataLayerBitmask(EDNADataLayer Layer);

RIGLOGICMODULE_API TSharedPtr<IDNAReader> LoadDNAFromStream(dna::BoundedIOStream* Stream, const FDNAConfig& DNAConfig = {});
RIGLOGICMODULE_API TSharedPtr<IDNAReader> LoadDNAFromBuffer(TArray<uint8>* DNABuffer, const FDNAConfig& DNAConfig = {});
RIGLOGICMODULE_API TSharedPtr<IDNAReader> LoadDNAFromFile(const FString& Path, const FDNAConfig& DNAConfig = {});

RIGLOGICMODULE_API void SaveDNAToStream(const IDNAReader* Reader, EDNADataLayer Layer, TNotNull<dna::BoundedIOStream*> Stream);
RIGLOGICMODULE_API void SaveDNAToBuffer(const IDNAReader* Reader, EDNADataLayer Layer, TArray<uint8>& DNABuffer);
RIGLOGICMODULE_API void SaveDNAToFile(const IDNAReader* Reader, EDNADataLayer Layer, const FString& Path);

// To be used as legacy (returning FLegacyDNAReader)
RIGLOGICMODULE_API TSharedPtr<IDNAReader> ReadDNAFromStream(dna::BoundedIOStream* Stream, const FDNAConfig& DNAConfig = {});
RIGLOGICMODULE_API TSharedPtr<IDNAReader> ReadDNAFromFile(const FString& Path, EDNADataLayer Layer = EDNADataLayer::All, uint16_t MaxLOD = 0);
RIGLOGICMODULE_API TSharedPtr<IDNAReader> ReadDNAFromFile(const FString& Path, EDNADataLayer Layer, TArrayView<uint16_t> LODs);
RIGLOGICMODULE_API TSharedPtr<IDNAReader> ReadDNAFromBuffer(TArray<uint8>* DNABuffer, EDNADataLayer Layer = EDNADataLayer::All, uint16_t MaxLOD = 0);
RIGLOGICMODULE_API TSharedPtr<IDNAReader> ReadDNAFromBuffer(TArray<uint8>* DNABuffer, EDNADataLayer Layer, TArrayView<uint16_t> LODs);
UE_DEPRECATED(5.8, "This function has been deprecated, use SaveDNAToBuffer instead.")
RIGLOGICMODULE_API TArray<uint8> ReadStreamFromDNA(const IDNAReader* Reader, EDNADataLayer Layer);
RIGLOGICMODULE_API void WriteDNAToFile(const IDNAReader* Reader, EDNADataLayer Layer, const FString& Path);

/** Creates a new DNA asset with behavior/geometry data set. FileHelper is used to load dna data from specified file */
RIGLOGICMODULE_API TObjectPtr<class UDNAAsset> LoadDNAAssetFromFile(const FString& InFilePath, UObject* InOuter, EDNADataLayer InLayer = EDNADataLayer::All);
//UE_DEPRECATED(5.8, "Use GetDNAFromFile instead.")
RIGLOGICMODULE_API TObjectPtr<class UDNAAsset> GetDNAAssetFromFile(const FString& InFilePath, UObject* InOuter, EDNADataLayer InLayer = EDNADataLayer::All);
RIGLOGICMODULE_API TObjectPtr<class UDNA> ReadDNAAssetFromFile(const FString& InFilePath, UObject* InOuter, EDNADataLayer InLayer = EDNADataLayer::All);
