// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAReader.h"
#include "DNAUtils.h"
#include "FMemoryResource.h"

// Legacy DNA reader adapter performs legacy data transforms to UE's coordinate system.
// This is used by code such as MHC that hasn't switched to using the coordinate system
// functionality implemented in the low level DNA lib.
template <class TWrappedReader>
class FLegacyDNAReader : public IDNAReader
{
public:
	FLegacyDNAReader(TWrappedReader* Source, const FDNAConfig& InConfig);
	~FLegacyDNAReader() = default;

	FLegacyDNAReader(const FLegacyDNAReader&) = delete;
	FLegacyDNAReader& operator=(const FLegacyDNAReader&) = delete;

	FLegacyDNAReader(FLegacyDNAReader&&) = default;
	FLegacyDNAReader& operator=(FLegacyDNAReader&&) = default;

	// Header
	uint16 GetFileFormatGeneration() const override;
	uint16 GetFileFormatVersion() const override;
	// Descriptor
	FString GetName() const override;
	EArchetype GetArchetype() const override;
	EGender GetGender() const override;
	uint16 GetAge() const override;
	uint32 GetMetaDataCount() const override;
	FString GetMetaDataKey(uint32 Index) const override;
	FString GetMetaDataValue(const FString& Key) const override;
	ETranslationUnit GetTranslationUnit() const override;
	ERotationUnit GetRotationUnit() const override;
	FCoordinateSystem GetCoordinateSystem() const override;
	ERotationSequence GetRotationSequence() const override;
	FRotationSign GetRotationSign() const override;
	EFaceWindingOrder GetFaceWindingOrder() const override;
	uint16 GetLODCount() const override;
	uint16 GetDBMaxLOD() const override;
	FString GetDBComplexity() const override;
	FString GetDBName() const override;
	// Definition
	uint16 GetGUIControlCount() const override;
	FString GetGUIControlName(uint16 Index) const override;
	uint16 GetRawControlCount() const override;
	FString GetRawControlName(uint16 Index) const override;
	uint16 GetJointCount() const override;
	FString GetJointName(uint16 Index) const override;
	uint16 GetJointIndexListCount() const override;
	TArrayView<const uint16> GetJointIndicesForLOD(uint16 LOD) const override;
	uint16 GetJointParentIndex(uint16 Index) const override;
	uint16 GetBlendShapeChannelCount() const override;
	FString GetBlendShapeChannelName(uint16 Index) const override;
	uint16 GetBlendShapeChannelIndexListCount() const override;
	TArrayView<const uint16> GetBlendShapeChannelIndicesForLOD(uint16 LOD) const override;
	uint16 GetAnimatedMapCount() const override;
	FString GetAnimatedMapName(uint16 Index) const override;
	uint16 GetAnimatedMapIndexListCount() const override;
	TArrayView<const uint16> GetAnimatedMapIndicesForLOD(uint16 LOD) const override;
	uint16 GetMeshCount() const override;
	FString GetMeshName(uint16 Index) const override;
	uint16 GetMeshIndexListCount() const override;
	TArrayView<const uint16> GetMeshIndicesForLOD(uint16 LOD) const override;
	uint16 GetMeshBlendShapeChannelMappingCount() const override;
	FMeshBlendShapeChannelMapping GetMeshBlendShapeChannelMapping(uint16 Index) const override;
	TArrayView<const uint16> GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const override;
	FVector GetNeutralJointTranslation(uint16 Index) const override;
	FVector GetNeutralJointRotation(uint16 Index) const override;
	// Behavior
	TArrayView<const uint16> GetGUIToRawInputIndices() const override;
	TArrayView<const uint16> GetGUIToRawOutputIndices() const override;
	TArrayView<const float> GetGUIToRawFromValues() const override;
	TArrayView<const float> GetGUIToRawToValues() const override;
	TArrayView<const float> GetGUIToRawSlopeValues() const override;
	TArrayView<const float> GetGUIToRawCutValues() const override;
	uint16 GetPSDCount() const override;
	TArrayView<const uint16> GetPSDRowIndices() const override;
	TArrayView<const uint16> GetPSDColumnIndices() const override;
	TArrayView<const float> GetPSDValues() const override;
	uint16 GetJointRowCount() const override;
	uint16 GetJointColumnCount() const override;
	TArrayView<const uint16> GetJointVariableAttributeIndices(uint16 LOD) const override;
	uint16 GetJointGroupCount() const override;
	TArrayView<const uint16> GetJointGroupLODs(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetJointGroupInputIndices(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetJointGroupOutputIndices(uint16 JointGroupIndex) const override;
	TArrayView<const float> GetJointGroupValues(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetJointGroupJointIndices(uint16 JointGroupIndex) const override;
	TArrayView<const uint16> GetBlendShapeChannelLODs() const override;
	TArrayView<const uint16> GetBlendShapeChannelOutputIndices() const override;
	TArrayView<const uint16> GetBlendShapeChannelInputIndices() const override;
	TArrayView<const uint16> GetAnimatedMapLODs() const override;
	TArrayView<const uint16> GetAnimatedMapInputIndices() const override;
	TArrayView<const uint16> GetAnimatedMapOutputIndices() const override;
	TArrayView<const float> GetAnimatedMapFromValues() const override;
	TArrayView<const float> GetAnimatedMapToValues() const override;
	TArrayView<const float> GetAnimatedMapSlopeValues() const override;
	TArrayView<const float> GetAnimatedMapCutValues() const override;
	// Geometry
	uint32 GetVertexPositionCount(uint16 MeshIndex) const override;
	FVector GetVertexPosition(uint16 MeshIndex, uint32 VertexIndex) const override;
	TArrayView<const float> GetVertexPositionXs(uint16 MeshIndex) const override;
	TArrayView<const float> GetVertexPositionYs(uint16 MeshIndex) const override;
	TArrayView<const float> GetVertexPositionZs(uint16 MeshIndex) const override;
	uint32 GetVertexTextureCoordinateCount(uint16 MeshIndex) const override;
	FTextureCoordinate GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const override;
	uint32 GetVertexNormalCount(uint16 MeshIndex) const override;
	FVector GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const override;
	uint32 GetFaceCount(uint16 MeshIndex) const override;
	TArrayView<const uint32> GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const override;
	uint32 GetVertexLayoutCount(uint16 MeshIndex) const override;
	FVertexLayout GetVertexLayout(uint16 MeshIndex, uint32 LayoutIndex) const override;
	uint16 GetMaximumInfluencePerVertex(uint16 MeshIndex) const override;
	uint32 GetSkinWeightsCount(uint16 MeshIndex) const override;
	TArrayView<const float> GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const override;
	TArrayView<const uint16> GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const override;
	uint16 GetBlendShapeTargetCount(uint16 MeshIndex) const override;
	uint16 GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	uint32 GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	FVector GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeTargetIndex, uint32 DeltaIndex) const override;
	TArrayView<const float> GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	TArrayView<const float> GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	TArrayView<const float> GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	TArrayView<const uint32> GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const override;
	// Machine Learned Behavior
	uint16 GetMLControlCount() const override;
	FString GetMLControlName(uint16 Index) const override;
	uint16 GetNeuralNetworkCount() const override;
	uint16 GetNeuralNetworkIndexListCount() const override;
	TArrayView<const uint16> GetNeuralNetworkIndicesForLOD(uint16 LOD) const override;
	uint16 GetMeshRegionCount(uint16 MeshIndex) const override;
	FString GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const override;
	TArrayView<const uint16> GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const override;
	TArrayView<const uint16> GetNeuralNetworkInputIndices(uint16 NetIndex) const override;
	TArrayView<const uint16> GetNeuralNetworkOutputIndices(uint16 NetIndex) const override;
	uint16 GetNeuralNetworkLayerCount(uint16 NetIndex) const override;
	EActivationFunction GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const override;
	TArrayView<const float> GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const override;
	TArrayView<const float> GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const override;
	TArrayView<const float> GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const override;
	// JointBehaviorMetadataReader
	ETranslationRepresentation GetJointTranslationRepresentation(uint16 JointIndex) const override;
	ERotationRepresentation GetJointRotationRepresentation(uint16 JointIndex) const override;
	EScaleRepresentation GetJointScaleRepresentation(uint16 JointIndex) const override;
	// RBFBehavior
	uint16 GetRBFPoseCount() const override;
	FString GetRBFPoseName(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseJointOutputIndices(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const override;
	TArrayView<const float> GetRBFPoseJointOutputValues(uint16 PoseIndex) const override;
	float GetRBFPoseScale(uint16 PoseIndex) const override;
	uint16 GetRBFPoseControlCount() const override;
	FString GetRBFPoseControlName(uint16 PoseControlIndex) const override;
	TArrayView<const uint16> GetRBFPoseInputControlIndices(uint16 PoseIndex) const override;
	TArrayView<const uint16> GetRBFPoseOutputControlIndices(uint16 PoseIndex) const override;
	TArrayView<const float> GetRBFPoseOutputControlWeights(uint16 PoseIndex) const override;
	uint16 GetRBFSolverCount() const override;
	uint16 GetRBFSolverIndexListCount() const override;
	TArrayView<const uint16> GetRBFSolverIndicesForLOD(uint16 LOD) const override;
	FString GetRBFSolverName(uint16 SolverIndex) const override;
	TArrayView<const uint16> GetRBFSolverRawControlIndices(uint16 SolverIndex) const override;
	TArrayView<const uint16> GetRBFSolverPoseIndices(uint16 SolverIndex) const override;
	TArrayView<const float> GetRBFSolverRawControlValues(uint16 SolverIndex) const override;
	ERBFSolverType GetRBFSolverType(uint16 SolverIndex) const override;
	float GetRBFSolverRadius(uint16 SolverIndex) const override;
	EAutomaticRadius GetRBFSolverAutomaticRadius(uint16 SolverIndex) const override;
	float GetRBFSolverWeightThreshold(uint16 SolverIndex) const override;
	ERBFDistanceMethod GetRBFSolverDistanceMethod(uint16 SolverIndex) const override;
	ERBFNormalizeMethod GetRBFSolverNormalizeMethod(uint16 SolverIndex) const override;
	ERBFFunctionType GetRBFSolverFunctionType(uint16 SolverIndex) const override;
	ETwistAxis GetRBFSolverTwistAxis(uint16 SolverIndex) const override;
	// TwistSwingBehavior
	uint16 GetTwistCount() const override;
	ETwistAxis GetTwistSetupTwistAxis(uint16 TwistIndex) const override;
	TArrayView<const uint16> GetTwistInputControlIndices(uint16 TwistIndex) const override;
	TArrayView<const uint16> GetTwistOutputJointIndices(uint16 TwistIndex) const override;
	TArrayView<const float> GetTwistBlendWeights(uint16 TwistIndex) const override;
	uint16 GetSwingCount() const override;
	ETwistAxis GetSwingSetupTwistAxis(uint16 SwingIndex) const override;
	TArrayView<const uint16> GetSwingInputControlIndices(uint16 SwingIndex) const override;
	TArrayView<const uint16> GetSwingOutputJointIndices(uint16 SwingIndex) const override;
	TArrayView<const float> GetSwingBlendWeights(uint16 SwingIndex) const override;
	// MachineLearnedBehaviorExtReader
	uint16 GetMLTypeCount() const override;
	uint16 GetMLOperationSetCount(uint16 MLTypeIndex) const override;
	uint16 GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const override;
	EMachineLearnedBehaviorOperationType GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	TArrayView<const uint32> GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	TArrayView<const uint16> GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	TArrayView<const uint16> GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const override;
	uint16 GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const override;
	TArrayView<const uint16> GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const override;
	TArrayView<const uint16> GetMLJointsInputIndices() const override;
	TArrayView<const uint16> GetMLJointsOutputIndices() const override;
	TArrayView<const uint16> GetMLJointsParameterKeys() const override;
	TArrayView<const uint16> GetMLJointsParameterValues() const override;

	void Unload(EDNADataLayer Layer) override;

	const FDNAConfig& GetConfig() const override;

	bool IsLegacyWrapped() const override { return true; }

private:
	dna::Reader* Unwrap() const override;

private:
	TSharedPtr<FMemoryResource> MemoryResource;

	struct FWrappedReaderDeleter
	{
		void operator()(TWrappedReader* Pointer);
	};
	TUniquePtr<TWrappedReader, FWrappedReaderDeleter> ReaderPtr;
	FDNAConfig Config;
};

template <class TWrappedReader>
FLegacyDNAReader<TWrappedReader>::FLegacyDNAReader(TWrappedReader* Source, const FDNAConfig& InConfig) :
	MemoryResource{ FMemoryResource::SharedInstance() },
	ReaderPtr{ Source },
	Config{ InConfig }
{
}

template <class TWrappedReader>
const FDNAConfig& FLegacyDNAReader<TWrappedReader>::GetConfig() const
{
	return Config;
}

template <class TWrappedReader>
dna::Reader* FLegacyDNAReader<TWrappedReader>::Unwrap() const
{
	return ReaderPtr.Get();
}

template <class TWrappedReader>
void FLegacyDNAReader<TWrappedReader>::Unload(EDNADataLayer Layer)
{
	ReaderPtr->unload(CalculateDNADataLayerBitmask(Layer));
}

template <class TWrappedReader>
void FLegacyDNAReader<TWrappedReader>::FWrappedReaderDeleter::operator()(TWrappedReader* Pointer)
{
	if (Pointer != nullptr)
	{
		TWrappedReader::destroy(Pointer);
	}
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetFileFormatGeneration() const
{
	return ReaderPtr->getFileFormatGeneration();
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetFileFormatVersion() const
{
	return ReaderPtr->getFileFormatVersion();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetName() const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getName().data()));
}

template <class TWrappedReader>
EArchetype FLegacyDNAReader<TWrappedReader>::GetArchetype() const
{
	return static_cast<EArchetype>(ReaderPtr->getArchetype());
}

template <class TWrappedReader>
EGender FLegacyDNAReader<TWrappedReader>::GetGender() const
{
	return static_cast<EGender>(ReaderPtr->getGender());
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetAge() const
{
	return ReaderPtr->getAge();
}

template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetMetaDataCount() const
{
	return ReaderPtr->getMetaDataCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetMetaDataKey(uint32 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getMetaDataKey(Index).data()));
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetMetaDataValue(const FString& Key) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getMetaDataValue(TCHAR_TO_ANSI(*Key)).data()));
}

template <class TWrappedReader>
ETranslationUnit FLegacyDNAReader<TWrappedReader>::GetTranslationUnit() const
{
	return static_cast<ETranslationUnit>(ReaderPtr->getTranslationUnit());
}

template <class TWrappedReader>
ERotationUnit FLegacyDNAReader<TWrappedReader>::GetRotationUnit() const
{
	return static_cast<ERotationUnit>(ReaderPtr->getRotationUnit());
}

template <class TWrappedReader>
FCoordinateSystem FLegacyDNAReader<TWrappedReader>::GetCoordinateSystem() const
{
	const auto System = ReaderPtr->getCoordinateSystem();
	return FCoordinateSystem
	{
		static_cast<EDirection>(System.x),
		static_cast<EDirection>(System.y),
		static_cast<EDirection>(System.z)
	};
}

template <class TWrappedReader>
ERotationSequence FLegacyDNAReader<TWrappedReader>::GetRotationSequence() const
{
	return static_cast<ERotationSequence>(ReaderPtr->getRotationSequence());
}

template <class TWrappedReader>
FRotationSign FLegacyDNAReader<TWrappedReader>::GetRotationSign() const
{
	const auto ConvertRotationDir = [](dna::RotationDirection RotationDir)
		{
			return RotationDir == dna::RotationDirection::negative ? ERotationDirection::Negative : ERotationDirection::Positive;
		};
	const dna::RotationSign RotationSigns = ReaderPtr->getRotationSign();
	return FRotationSign{ ConvertRotationDir(RotationSigns.x), ConvertRotationDir(RotationSigns.y), ConvertRotationDir(RotationSigns.z) };
}

template <class TWrappedReader>
EFaceWindingOrder FLegacyDNAReader<TWrappedReader>::GetFaceWindingOrder() const
{
	return static_cast<EFaceWindingOrder>(ReaderPtr->getFaceWindingOrder());
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetLODCount() const
{
	return ReaderPtr->getLODCount();
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetDBMaxLOD() const
{
	return ReaderPtr->getDBMaxLOD();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetDBComplexity() const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getDBComplexity().data()));
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetDBName() const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getDBName().data()));
}


template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetGUIControlCount() const
{
	return ReaderPtr->getGUIControlCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetGUIControlName(uint16 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getGUIControlName(Index).data()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetRawControlCount() const
{
	return ReaderPtr->getRawControlCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetRawControlName(uint16 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getRawControlName(Index).data()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetJointCount() const
{
	return ReaderPtr->getJointCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetJointName(uint16 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getJointName(Index).data()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetJointIndexListCount() const
{
	return ReaderPtr->getJointIndexListCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetJointIndicesForLOD(uint16 LOD) const
{
	const auto Indices = ReaderPtr->getJointIndicesForLOD(LOD);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetJointParentIndex(uint16 Index) const
{
	return ReaderPtr->getJointParentIndex(Index);
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelCount() const
{
	return ReaderPtr->getBlendShapeChannelCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelName(uint16 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getBlendShapeChannelName(Index).data()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelIndexListCount() const
{
	return ReaderPtr->getBlendShapeChannelIndexListCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelIndicesForLOD(uint16 LOD) const
{
	const auto Indices = ReaderPtr->getBlendShapeChannelIndicesForLOD(LOD);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetAnimatedMapCount() const
{
	return ReaderPtr->getAnimatedMapCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetAnimatedMapName(uint16 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getAnimatedMapName(Index).data()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetAnimatedMapIndexListCount() const
{
	return ReaderPtr->getAnimatedMapIndexListCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapIndicesForLOD(uint16 LOD) const
{
	const auto Indices = ReaderPtr->getAnimatedMapIndicesForLOD(LOD);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMeshCount() const
{
	return ReaderPtr->getMeshCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetMeshName(uint16 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getMeshName(Index).data()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMeshIndexListCount() const
{
	return ReaderPtr->getMeshIndexListCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMeshIndicesForLOD(uint16 LOD) const
{
	const auto Indices = ReaderPtr->getMeshIndicesForLOD(LOD);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMeshBlendShapeChannelMappingCount() const
{
	return ReaderPtr->getMeshBlendShapeChannelMappingCount();
}

template <class TWrappedReader>
FMeshBlendShapeChannelMapping FLegacyDNAReader<TWrappedReader>::GetMeshBlendShapeChannelMapping(uint16 Index) const
{
	const auto Mapping = ReaderPtr->getMeshBlendShapeChannelMapping(Index);
	return FMeshBlendShapeChannelMapping{ Mapping.meshIndex, Mapping.blendShapeChannelIndex };
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMeshBlendShapeChannelMappingIndicesForLOD(uint16 LOD) const
{
	const auto Indices = ReaderPtr->getMeshBlendShapeChannelMappingIndicesForLOD(LOD);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
FVector FLegacyDNAReader<TWrappedReader>::GetNeutralJointTranslation(uint16 Index) const
{
	const auto Translation = ReaderPtr->getNeutralJointTranslation(Index);
	// X = X, Y = -Y, Z = Z
	return FVector(Translation.x, -Translation.y, Translation.z);
}

template <class TWrappedReader>
FVector FLegacyDNAReader<TWrappedReader>::GetNeutralJointRotation(uint16 Index) const
{
	const auto Rotation = ReaderPtr->getNeutralJointRotation(Index);
	// X = -Y, Y = -Z, Z = X
	return FVector(-Rotation.y, -Rotation.z, Rotation.x);
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetGUIToRawInputIndices() const
{
	const auto Indices = ReaderPtr->getGUIToRawInputIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetGUIToRawOutputIndices() const
{
	const auto Indices = ReaderPtr->getGUIToRawOutputIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetGUIToRawFromValues() const
{
	const auto Values = ReaderPtr->getGUIToRawFromValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetGUIToRawToValues() const
{
	const auto Values = ReaderPtr->getGUIToRawToValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetGUIToRawSlopeValues() const
{
	const auto Values = ReaderPtr->getGUIToRawSlopeValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetGUIToRawCutValues() const
{
	const auto Values = ReaderPtr->getGUIToRawCutValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetPSDCount() const
{
	return ReaderPtr->getPSDCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetPSDRowIndices() const
{
	const auto Indices = ReaderPtr->getPSDRowIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetPSDColumnIndices() const
{
	const auto Indices = ReaderPtr->getPSDColumnIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetPSDValues() const
{
	const auto Values = ReaderPtr->getPSDValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetJointRowCount() const
{
	return ReaderPtr->getJointRowCount();
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetJointColumnCount() const
{
	return ReaderPtr->getJointColumnCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetJointVariableAttributeIndices(uint16 LOD) const
{
	const auto Indices = ReaderPtr->getJointVariableAttributeIndices(LOD);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetJointGroupCount() const
{
	return ReaderPtr->getJointGroupCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetJointGroupLODs(uint16 JointGroupIndex) const
{
	const auto LODs = ReaderPtr->getJointGroupLODs(JointGroupIndex);
	return TArrayView<const uint16>(LODs.data(), static_cast<int32>(LODs.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetJointGroupInputIndices(uint16 JointGroupIndex) const
{
	const auto Indices = ReaderPtr->getJointGroupInputIndices(JointGroupIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetJointGroupOutputIndices(uint16 JointGroupIndex) const
{
	const auto Indices = ReaderPtr->getJointGroupOutputIndices(JointGroupIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetJointGroupValues(uint16 JointGroupIndex) const
{
	const auto Values = ReaderPtr->getJointGroupValues(JointGroupIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetJointGroupJointIndices(uint16 JointGroupIndex) const
{
	const auto Indices = ReaderPtr->getJointGroupJointIndices(JointGroupIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelLODs() const
{
	const auto LODs = ReaderPtr->getBlendShapeChannelLODs();
	return TArrayView<const uint16>(LODs.data(), static_cast<int32>(LODs.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelOutputIndices() const
{
	const auto Indices = ReaderPtr->getBlendShapeChannelOutputIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelInputIndices() const
{
	const auto Indices = ReaderPtr->getBlendShapeChannelInputIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapLODs() const
{
	const auto LODs = ReaderPtr->getAnimatedMapLODs();
	return TArrayView<const uint16>(LODs.data(), static_cast<int32>(LODs.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapInputIndices() const
{
	const auto Indices = ReaderPtr->getAnimatedMapInputIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapOutputIndices() const
{
	const auto Indices = ReaderPtr->getAnimatedMapOutputIndices();
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapFromValues() const
{
	const auto Values = ReaderPtr->getAnimatedMapFromValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapToValues() const
{
	const auto Values = ReaderPtr->getAnimatedMapToValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapSlopeValues() const
{
	const auto Values = ReaderPtr->getAnimatedMapSlopeValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetAnimatedMapCutValues() const
{
	const auto Values = ReaderPtr->getAnimatedMapCutValues();
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}


template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetVertexPositionCount(uint16 MeshIndex) const
{
	return ReaderPtr->getVertexPositionCount(MeshIndex);
}

template <class TWrappedReader>
FVector FLegacyDNAReader<TWrappedReader>::GetVertexPosition(uint16 MeshIndex, uint32 VertexIndex) const
{
	const auto Position = ReaderPtr->getVertexPosition(MeshIndex, VertexIndex);
	// X = X, Y = Z, Z = Y
	return FVector(Position.x, Position.z, Position.y);
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetVertexPositionXs(uint16 MeshIndex) const
{
	// X = X
	const auto Values = ReaderPtr->getVertexPositionXs(MeshIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetVertexPositionYs(uint16 MeshIndex) const
{
	// Y = Z
	const auto Values = ReaderPtr->getVertexPositionZs(MeshIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetVertexPositionZs(uint16 MeshIndex) const
{
	// Z = Y
	const auto Values = ReaderPtr->getVertexPositionYs(MeshIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetVertexTextureCoordinateCount(uint16 MeshIndex) const
{
	return ReaderPtr->getVertexTextureCoordinateCount(MeshIndex);
}

template <class TWrappedReader>
FTextureCoordinate FLegacyDNAReader<TWrappedReader>::GetVertexTextureCoordinate(uint16 MeshIndex, uint32 TextureCoordinateIndex) const
{
	const auto Coordinate = ReaderPtr->getVertexTextureCoordinate(MeshIndex, TextureCoordinateIndex);
	return FTextureCoordinate{ Coordinate.u, Coordinate.v };
}

template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetVertexNormalCount(uint16 MeshIndex) const
{
	return ReaderPtr->getVertexNormalCount(MeshIndex);
}

template <class TWrappedReader>
FVector FLegacyDNAReader<TWrappedReader>::GetVertexNormal(uint16 MeshIndex, uint32 NormalIndex) const
{
	const auto Normal = ReaderPtr->getVertexNormal(MeshIndex, NormalIndex);
	// X = X, Y = Z, Z = Y
	return FVector(Normal.x, Normal.z, Normal.y);
}

template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetFaceCount(uint16 MeshIndex) const
{
	return ReaderPtr->getFaceCount(MeshIndex);
}

template <class TWrappedReader>
TArrayView<const uint32> FLegacyDNAReader<TWrappedReader>::GetFaceVertexLayoutIndices(uint16 MeshIndex, uint32 FaceIndex) const
{
	const auto Indices = ReaderPtr->getFaceVertexLayoutIndices(MeshIndex, FaceIndex);
	return TArrayView<const uint32>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetVertexLayoutCount(uint16 MeshIndex) const
{
	return ReaderPtr->getVertexLayoutCount(MeshIndex);
}

template <class TWrappedReader>
FVertexLayout FLegacyDNAReader<TWrappedReader>::GetVertexLayout(uint16 MeshIndex, uint32 LayoutIndex) const
{
	const auto Layout = ReaderPtr->getVertexLayout(MeshIndex, LayoutIndex);
	return FVertexLayout{ static_cast<int32>(Layout.position), static_cast<int32>(Layout.textureCoordinate), static_cast<int32>(Layout.normal) };
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMaximumInfluencePerVertex(uint16 MeshIndex) const
{
	return ReaderPtr->getMaximumInfluencePerVertex(MeshIndex);
}

template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetSkinWeightsCount(uint16 MeshIndex) const
{
	return ReaderPtr->getSkinWeightsCount(MeshIndex);
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetSkinWeightsValues(uint16 MeshIndex, uint32 VertexIndex) const
{
	const auto Values = ReaderPtr->getSkinWeightsValues(MeshIndex, VertexIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetSkinWeightsJointIndices(uint16 MeshIndex, uint32 VertexIndex) const
{
	const auto Indices = ReaderPtr->getSkinWeightsJointIndices(MeshIndex, VertexIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetBlendShapeTargetCount(uint16 MeshIndex) const
{
	return ReaderPtr->getBlendShapeTargetCount(MeshIndex);
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetBlendShapeChannelIndex(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->getBlendShapeChannelIndex(MeshIndex, BlendShapeTargetIndex);
}

template <class TWrappedReader>
uint32 FLegacyDNAReader<TWrappedReader>::GetBlendShapeTargetDeltaCount(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	return ReaderPtr->getBlendShapeTargetDeltaCount(MeshIndex, BlendShapeTargetIndex);
}

template <class TWrappedReader>
FVector FLegacyDNAReader<TWrappedReader>::GetBlendShapeTargetDelta(uint16 MeshIndex, uint16 BlendShapeTargetIndex, uint32 DeltaIndex) const
{
	const auto Delta = ReaderPtr->getBlendShapeTargetDelta(MeshIndex, BlendShapeTargetIndex, DeltaIndex);
	// X = X, Y = Z, Z = Y
	return FVector(Delta.x, Delta.z, Delta.y);
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetBlendShapeTargetDeltaXs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	// X = X
	const auto Values = ReaderPtr->getBlendShapeTargetDeltaXs(MeshIndex, BlendShapeTargetIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetBlendShapeTargetDeltaYs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	// Y = Z
	const auto Values = ReaderPtr->getBlendShapeTargetDeltaZs(MeshIndex, BlendShapeTargetIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetBlendShapeTargetDeltaZs(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	// Z = Y
	const auto Values = ReaderPtr->getBlendShapeTargetDeltaYs(MeshIndex, BlendShapeTargetIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint32> FLegacyDNAReader<TWrappedReader>::GetBlendShapeTargetVertexIndices(uint16 MeshIndex, uint16 BlendShapeTargetIndex) const
{
	const auto Indices = ReaderPtr->getBlendShapeTargetVertexIndices(MeshIndex, BlendShapeTargetIndex);
	return TArrayView<const uint32>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMLControlCount() const
{
	return ReaderPtr->getMLControlCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetMLControlName(uint16 Index) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getMLControlName(Index).data()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkCount() const
{
	return ReaderPtr->getNeuralNetworkCount();
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkIndexListCount() const
{
	return ReaderPtr->getNeuralNetworkIndexListCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkIndicesForLOD(uint16 LOD) const
{
	const auto Values = ReaderPtr->getNeuralNetworkIndicesForLOD(LOD);
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMeshRegionCount(uint16 MeshIndex) const
{
	return ReaderPtr->getMeshRegionCount(MeshIndex);
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetMeshRegionName(uint16 MeshIndex, uint16 RegionIndex) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getMeshRegionName(MeshIndex, RegionIndex).data()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkIndicesForMeshRegion(uint16 MeshIndex, uint16 RegionIndex) const
{
	const auto Values = ReaderPtr->getNeuralNetworkIndicesForMeshRegion(MeshIndex, RegionIndex);
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkInputIndices(uint16 NetIndex) const
{
	const auto Values = ReaderPtr->getNeuralNetworkInputIndices(NetIndex);
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkOutputIndices(uint16 NetIndex) const
{
	const auto Values = ReaderPtr->getNeuralNetworkOutputIndices(NetIndex);
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkLayerCount(uint16 NetIndex) const
{
	return ReaderPtr->getNeuralNetworkLayerCount(NetIndex);
}

template <class TWrappedReader>
EActivationFunction FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkLayerActivationFunction(uint16 NetIndex, uint16 LayerIndex) const
{
	return static_cast<EActivationFunction>(ReaderPtr->getNeuralNetworkLayerActivationFunction(NetIndex, LayerIndex));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkLayerActivationFunctionParameters(uint16 NetIndex, uint16 LayerIndex) const
{
	const auto Values = ReaderPtr->getNeuralNetworkLayerActivationFunctionParameters(NetIndex, LayerIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkLayerBiases(uint16 NetIndex, uint16 LayerIndex) const
{
	const auto Values = ReaderPtr->getNeuralNetworkLayerBiases(NetIndex, LayerIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetNeuralNetworkLayerWeights(uint16 NetIndex, uint16 LayerIndex) const
{
	const auto Values = ReaderPtr->getNeuralNetworkLayerWeights(NetIndex, LayerIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
ETranslationRepresentation FLegacyDNAReader<TWrappedReader>::GetJointTranslationRepresentation(uint16 JointIndex) const
{
	return static_cast<ETranslationRepresentation>(ReaderPtr->getJointTranslationRepresentation(JointIndex));
}

template <class TWrappedReader>
ERotationRepresentation FLegacyDNAReader<TWrappedReader>::GetJointRotationRepresentation(uint16 JointIndex) const
{
	return static_cast<ERotationRepresentation>(ReaderPtr->getJointRotationRepresentation(JointIndex));
}

template <class TWrappedReader>
EScaleRepresentation FLegacyDNAReader<TWrappedReader>::GetJointScaleRepresentation(uint16 JointIndex) const
{
	return static_cast<EScaleRepresentation>(ReaderPtr->getJointScaleRepresentation(JointIndex));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetRBFPoseCount() const
{
	return ReaderPtr->getRBFPoseCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetRBFPoseName(uint16 PoseIndex) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getRBFPoseName(PoseIndex).data()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFPoseJointOutputIndices(uint16 PoseIndex) const
{
	const auto Indices = ReaderPtr->getRBFPoseJointOutputIndices(PoseIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFPoseBlendShapeChannelOutputIndices(uint16 PoseIndex) const
{
	const auto Indices = ReaderPtr->getRBFPoseBlendShapeChannelOutputIndices(PoseIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFPoseAnimatedMapOutputIndices(uint16 PoseIndex) const
{
	const auto Indices = ReaderPtr->getRBFPoseAnimatedMapOutputIndices(PoseIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetRBFPoseJointOutputValues(uint16 PoseIndex) const
{
	const auto Values = ReaderPtr->getRBFPoseJointOutputValues(PoseIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
float FLegacyDNAReader<TWrappedReader>::GetRBFPoseScale(uint16 PoseIndex) const
{
	return ReaderPtr->getRBFPoseScale(PoseIndex);
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetRBFPoseControlCount() const
{
	return ReaderPtr->getRBFPoseControlCount();
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetRBFPoseControlName(uint16 PoseControlIndex) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getRBFPoseControlName(PoseControlIndex).data()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFPoseInputControlIndices(uint16 PoseIndex) const
{
	const auto Indices = ReaderPtr->getRBFPoseInputControlIndices(PoseIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFPoseOutputControlIndices(uint16 PoseIndex) const
{
	const auto Indices = ReaderPtr->getRBFPoseOutputControlIndices(PoseIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetRBFPoseOutputControlWeights(uint16 PoseIndex) const
{
	const auto Values = ReaderPtr->getRBFPoseOutputControlWeights(PoseIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetRBFSolverCount() const
{
	return ReaderPtr->getRBFSolverCount();
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetRBFSolverIndexListCount() const
{
	return ReaderPtr->getRBFSolverIndexListCount();
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFSolverIndicesForLOD(uint16 LOD) const
{
	const auto Indices = ReaderPtr->getRBFSolverIndicesForLOD(LOD);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
FString FLegacyDNAReader<TWrappedReader>::GetRBFSolverName(uint16 SolverIndex) const
{
	return FString(ANSI_TO_TCHAR(ReaderPtr->getRBFSolverName(SolverIndex).data()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFSolverRawControlIndices(uint16 SolverIndex) const
{
	const auto Indices = ReaderPtr->getRBFSolverRawControlIndices(SolverIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetRBFSolverPoseIndices(uint16 SolverIndex) const
{
	const auto Indices = ReaderPtr->getRBFSolverPoseIndices(SolverIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetRBFSolverRawControlValues(uint16 SolverIndex) const
{
	const auto Values = ReaderPtr->getRBFSolverRawControlValues(SolverIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
ERBFSolverType FLegacyDNAReader<TWrappedReader>::GetRBFSolverType(uint16 SolverIndex) const
{
	return static_cast<ERBFSolverType>(ReaderPtr->getRBFSolverType(SolverIndex));
}

template <class TWrappedReader>
float FLegacyDNAReader<TWrappedReader>::GetRBFSolverRadius(uint16 SolverIndex) const
{
	return ReaderPtr->getRBFSolverRadius(SolverIndex);
}

template <class TWrappedReader>
EAutomaticRadius FLegacyDNAReader<TWrappedReader>::GetRBFSolverAutomaticRadius(uint16 SolverIndex) const
{
	return static_cast<EAutomaticRadius>(ReaderPtr->getRBFSolverAutomaticRadius(SolverIndex));
}

template <class TWrappedReader>
float FLegacyDNAReader<TWrappedReader>::GetRBFSolverWeightThreshold(uint16 SolverIndex) const
{
	return ReaderPtr->getRBFSolverWeightThreshold(SolverIndex);
}

template <class TWrappedReader>
ERBFDistanceMethod FLegacyDNAReader<TWrappedReader>::GetRBFSolverDistanceMethod(uint16 SolverIndex) const
{
	return static_cast<ERBFDistanceMethod>(ReaderPtr->getRBFSolverDistanceMethod(SolverIndex));
}

template <class TWrappedReader>
ERBFNormalizeMethod FLegacyDNAReader<TWrappedReader>::GetRBFSolverNormalizeMethod(uint16 SolverIndex) const
{
	return static_cast<ERBFNormalizeMethod>(ReaderPtr->getRBFSolverNormalizeMethod(SolverIndex));
}

template <class TWrappedReader>
ERBFFunctionType FLegacyDNAReader<TWrappedReader>::GetRBFSolverFunctionType(uint16 SolverIndex) const
{
	return static_cast<ERBFFunctionType>(ReaderPtr->getRBFSolverFunctionType(SolverIndex));
}

template <class TWrappedReader>
ETwistAxis FLegacyDNAReader<TWrappedReader>::GetRBFSolverTwistAxis(uint16 SolverIndex) const
{
	return static_cast<ETwistAxis>(ReaderPtr->getRBFSolverTwistAxis(SolverIndex));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetTwistCount() const
{
	return ReaderPtr->getTwistCount();
}

template <class TWrappedReader>
ETwistAxis FLegacyDNAReader<TWrappedReader>::GetTwistSetupTwistAxis(uint16 TwistIndex) const
{
	return static_cast<ETwistAxis>(ReaderPtr->getTwistSetupTwistAxis(TwistIndex));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetTwistInputControlIndices(uint16 TwistIndex) const
{
	const auto Indices = ReaderPtr->getTwistInputControlIndices(TwistIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetTwistOutputJointIndices(uint16 TwistIndex) const
{
	const auto Indices = ReaderPtr->getTwistOutputJointIndices(TwistIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetTwistBlendWeights(uint16 TwistIndex) const
{
	const auto Values = ReaderPtr->getTwistBlendWeights(TwistIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetSwingCount() const
{
	return ReaderPtr->getSwingCount();
}

template <class TWrappedReader>
ETwistAxis FLegacyDNAReader<TWrappedReader>::GetSwingSetupTwistAxis(uint16 SwingIndex) const
{
	return static_cast<ETwistAxis>(ReaderPtr->getSwingSetupTwistAxis(SwingIndex));

}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetSwingInputControlIndices(uint16 SwingIndex) const
{
	const auto Indices = ReaderPtr->getSwingInputControlIndices(SwingIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetSwingOutputJointIndices(uint16 SwingIndex) const
{
	const auto Indices = ReaderPtr->getSwingOutputJointIndices(SwingIndex);
	return TArrayView<const uint16>(Indices.data(), static_cast<int32>(Indices.size()));
}

template <class TWrappedReader>
TArrayView<const float> FLegacyDNAReader<TWrappedReader>::GetSwingBlendWeights(uint16 SwingIndex) const
{
	const auto Values = ReaderPtr->getSwingBlendWeights(SwingIndex);
	return TArrayView<const float>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMLTypeCount() const
{
	return ReaderPtr->getMLTypeCount();
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMLOperationSetCount(uint16 MLTypeIndex) const
{
	return ReaderPtr->getMLOperationSetCount(MLTypeIndex);
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return ReaderPtr->getMLOperationCount(MLTypeIndex, MLOperationSetIndex);
}

template <class TWrappedReader>
EMachineLearnedBehaviorOperationType FLegacyDNAReader<TWrappedReader>::GetMLOperationType(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	return static_cast<EMachineLearnedBehaviorOperationType>(ReaderPtr->getMLOperationType(MLTypeIndex, MLOperationSetIndex, MLOperationIndex));
}

template <class TWrappedReader>
TArrayView<const uint32> FLegacyDNAReader<TWrappedReader>::GetMLOperationParameters(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	const auto Values = ReaderPtr->getMLOperationParameters(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
	return TArrayView<const uint32>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMLOperationDependencyOperationSetIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	const auto Values = ReaderPtr->getMLOperationDependencyOperationSetIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMLOperationDependencyOperationIndices(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const
{
	const auto Values = ReaderPtr->getMLOperationDependencyOperationIndices(MLTypeIndex, MLOperationSetIndex, MLOperationIndex);
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
uint16 FLegacyDNAReader<TWrappedReader>::GetMLOperationIndexListCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const
{
	return ReaderPtr->getMLOperationIndexListCount(MLTypeIndex, MLOperationSetIndex);
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMLOperationIndicesForLOD(uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 LOD) const
{
	const auto Values = ReaderPtr->getMLOperationIndicesForLOD(MLTypeIndex, MLOperationSetIndex, LOD);
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMLJointsInputIndices() const
{
	const auto Values = ReaderPtr->getMLJointsInputIndices();
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMLJointsOutputIndices() const
{
	const auto Values = ReaderPtr->getMLJointsOutputIndices();
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMLJointsParameterKeys() const
{
	const auto Values = ReaderPtr->getMLJointsParameterKeys();
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}

template <class TWrappedReader>
TArrayView<const uint16> FLegacyDNAReader<TWrappedReader>::GetMLJointsParameterValues() const
{
	const auto Values = ReaderPtr->getMLJointsParameterValues();
	return TArrayView<const uint16>(Values.data(), static_cast<int32>(Values.size()));
}
