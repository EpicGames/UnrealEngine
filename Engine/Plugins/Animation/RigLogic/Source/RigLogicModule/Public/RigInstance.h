// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"

#include "FMemoryResource.h"

#define UE_API RIGLOGICMODULE_API

class FRigLogic;

namespace rl4
{

class RigInstance;

}  // namespace rl4

class FRigInstance
{
public:
	UE_API explicit FRigInstance(FRigLogic* RigLogic);
	UE_API ~FRigInstance();

	FRigInstance(const FRigInstance&) = delete;
	FRigInstance& operator=(const FRigInstance&) = delete;

	FRigInstance(FRigInstance&&) = default;
	FRigInstance& operator=(FRigInstance&&) = default;

	UE_API uint16 GetGUIControlCount() const;
	UE_API float GetGUIControl(uint16 Index) const;
	UE_API void SetGUIControl(uint16 Index, float Value);
	UE_API TArrayView<float> GetGUIControlValues();
	UE_API TArrayView<const float> GetGUIControlValues() const;

	UE_API uint16 GetRawControlCount() const;
	UE_API float GetRawControl(uint16 Index) const;
	UE_API void SetRawControl(uint16 Index, float Value);
	UE_API TArrayView<float> GetRawControlValues();
	UE_API TArrayView<const float> GetRawControlValues() const;

	UE_API uint16 GetPSDControlCount() const;
	UE_API float GetPSDControl(uint16 Index) const;
	UE_API TArrayView<const float> GetPSDControlValues() const;

	UE_API uint16 GetMLControlCount() const;
	UE_API float GetMLControl(uint16 Index) const;
	UE_API TArrayView<const float> GetMLControlValues() const;

	UE_API uint16 GetMLTypeCount() const;
	UE_API uint16 GetMLOperationSetCount(uint16 MLTypeIndex) const;
	UE_API uint16 GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const;
	UE_API TArrayView<float> GetMLMaskValues();
	UE_API TArrayView<const float> GetMLMaskValues() const;

	UE_API uint16 GetRBFControlCount() const;
	UE_API float GetRBFControl(uint16 Index) const;
	UE_API TArrayView<const float> GetRBFControlValues() const;

	UE_API TArrayView<const float> GetJointOutputs() const;
	UE_API TArrayView<const float> GetBlendShapeOutputs() const;
	UE_API TArrayView<const float> GetAnimatedMapOutputs() const;
	UE_API uint16 GetLOD() const;
	UE_API void SetLOD(uint16 LOD);

private:
	friend FRigLogic;
	UE_API rl4::RigInstance* Unwrap() const;

private:
	TSharedPtr<FMemoryResource> MemoryResource;

	struct FRigInstanceDeleter
	{
		void operator()(rl4::RigInstance* Pointer);
	};
	TUniquePtr<rl4::RigInstance, FRigInstanceDeleter> RigInstance;
};

#undef UE_API
