// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Math/Vector.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PerPlatformProperties.h"

#include "FMemoryResource.h"

#include "RigLogic.generated.h"

#define UE_API RIGLOGICMODULE_API

class FRigInstance;
class IDNAReader;

namespace rl4
{

class RigLogic;

}  // namespace rl4

UENUM(BlueprintType)
enum class ERigLogicCalculationType: uint8
{
	Scalar,
	SSE,
	AVX,
	NEON,
	AnyVector
};

UENUM(BlueprintType)
enum class ERigLogicFloatingPointType : uint8
{
	Float,
	HalfFloat,
	Auto
};

UENUM(BlueprintType)
enum class ERigLogicTranslationType : uint8 {
	None UMETA(Hidden),
	Vector = 3
};

UENUM(BlueprintType)
enum class ERigLogicRotationType : uint8 {
	None UMETA(Hidden),
	EulerAngles = 3,
	Quaternions = 4
};

UENUM(BlueprintType, meta = (Deprecated = "5.8", DeprecationMessage = "The value is read from the DNA instead."))
enum class ERigLogicRotationOrder : uint8 {
	XYZ,
	XZY,
	YXZ,
	YZX,
	ZXY,
	ZYX
};

UENUM(BlueprintType)
enum class ERigLogicScaleType : uint8 {
	None UMETA(Hidden),
	Vector = 3
};

/**
 * Per-platform wrapper for ERigLogicCalculationType.
 * Serializes as FPerPlatformInt but renders as enum dropdown in the editor.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Per Platform Calculation Type"))
struct FPerPlatformERigLogicCalculationType : public FPerPlatformInt
{
	GENERATED_BODY()

	FPerPlatformERigLogicCalculationType() : FPerPlatformInt(static_cast<int32>(ERigLogicCalculationType::AnyVector))
	{
	}

	explicit FPerPlatformERigLogicCalculationType(ERigLogicCalculationType InValue) : FPerPlatformInt(static_cast<int32>(InValue))
	{
	}

	ERigLogicCalculationType GetEnumValue() const
	{
		return static_cast<ERigLogicCalculationType>(GetValue());
	}

#if WITH_EDITOR
	ERigLogicCalculationType GetEnumValueForPlatform(FName PlatformName) const
	{
		return static_cast<ERigLogicCalculationType>(GetValueForPlatform(PlatformName));
	}
#endif  // WITH_EDITOR
};

template<>
struct TStructOpsTypeTraits<FPerPlatformERigLogicCalculationType> : public TStructOpsTypeTraitsBase2<FPerPlatformERigLogicCalculationType>
{
	enum
	{
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/**
 * Per-platform wrapper for ERigLogicFloatingPointType.
 * Serializes as FPerPlatformInt but renders as enum dropdown in the editor.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Per Platform Floating Point Type"))
struct FPerPlatformERigLogicFloatingPointType : public FPerPlatformInt
{
	GENERATED_BODY()

	FPerPlatformERigLogicFloatingPointType() : FPerPlatformInt(static_cast<int32>(ERigLogicFloatingPointType::Float))
	{
	}

	explicit FPerPlatformERigLogicFloatingPointType(ERigLogicFloatingPointType InValue) : FPerPlatformInt(static_cast<int32>(InValue))
	{
	}

	ERigLogicFloatingPointType GetEnumValue() const
	{
		return static_cast<ERigLogicFloatingPointType>(GetValue());
	}

#if WITH_EDITOR
	ERigLogicFloatingPointType GetEnumValueForPlatform(FName PlatformName) const
	{
		return static_cast<ERigLogicFloatingPointType>(GetValueForPlatform(PlatformName));
	}
#endif  // WITH_EDITOR
};

template<>
struct TStructOpsTypeTraits<FPerPlatformERigLogicFloatingPointType> : public TStructOpsTypeTraitsBase2<FPerPlatformERigLogicFloatingPointType>
{
	enum
	{
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

USTRUCT(BlueprintType)
struct FRigLogicConfiguration
{
	GENERATED_BODY()

	FRigLogicConfiguration() :
		CalculationTypePerPlatform(ERigLogicCalculationType::AnyVector),
		FloatingPointTypePerPlatform(ERigLogicFloatingPointType::Auto),
		EnableMultiThreadMLComputePerPlatform(true),
		LoadJoints(true),
		LoadBlendShapes(true),
		LoadAnimatedMaps(true),
		LoadMachineLearnedBehavior(true),
		LoadRBFBehavior(true),
		LoadTwistSwingBehavior(true),
		TranslationType(ERigLogicTranslationType::Vector),
		RotationType(ERigLogicRotationType::Quaternions),
		ScaleType(ERigLogicScaleType::Vector),
		TranslationPruningThreshold(0.0f),
		RotationPruningThreshold(0.0f),
		ScalePruningThreshold(0.0f)
	{
	}

	UE_DEPRECATED(5.8, "Use FRigLogicConfiguration overload.")
	FRigLogicConfiguration(ERigLogicCalculationType CalculationType,
							bool LoadJoints,
							bool LoadBlendShapes,
							bool LoadAnimatedMaps,
							bool LoadMachineLearnedBehavior,
							bool LoadRBFBehavior,
							bool LoadTwistSwingBehavior,
							ERigLogicTranslationType TranslationType,
							ERigLogicRotationType RotationType,
							ERigLogicRotationOrder RotationOrder,
							ERigLogicScaleType ScaleType,
							float TranslationPruningThreshold,
							float RotationPruningThreshold,
							float ScalePruningThreshold) :
		CalculationTypePerPlatform(CalculationType),
		FloatingPointTypePerPlatform(ERigLogicFloatingPointType::Auto),
		EnableMultiThreadMLComputePerPlatform(true),
		LoadJoints(LoadJoints),
		LoadBlendShapes(LoadBlendShapes),
		LoadAnimatedMaps(LoadAnimatedMaps),
		LoadMachineLearnedBehavior(LoadMachineLearnedBehavior),
		LoadRBFBehavior(LoadRBFBehavior),
		LoadTwistSwingBehavior(LoadTwistSwingBehavior),
		TranslationType(TranslationType),
		RotationType(RotationType),
		ScaleType(ScaleType),
		TranslationPruningThreshold(TranslationPruningThreshold),
		RotationPruningThreshold(RotationPruningThreshold),
		ScalePruningThreshold(ScalePruningThreshold)
	{
	}

	FRigLogicConfiguration(ERigLogicCalculationType CalculationType,
							ERigLogicFloatingPointType FloatingPointType,
							bool EnableMultiThreadMLCompute,
							bool LoadJoints,
							bool LoadBlendShapes,
							bool LoadAnimatedMaps,
							bool LoadMachineLearnedBehavior,
							bool LoadRBFBehavior,
							bool LoadTwistSwingBehavior,
							ERigLogicTranslationType TranslationType,
							ERigLogicRotationType RotationType,
							ERigLogicScaleType ScaleType,
							float TranslationPruningThreshold,
							float RotationPruningThreshold,
							float ScalePruningThreshold) :
		CalculationTypePerPlatform(CalculationType),
		FloatingPointTypePerPlatform(FloatingPointType),
		EnableMultiThreadMLComputePerPlatform(EnableMultiThreadMLCompute),
		LoadJoints(LoadJoints),
		LoadBlendShapes(LoadBlendShapes),
		LoadAnimatedMaps(LoadAnimatedMaps),
		LoadMachineLearnedBehavior(LoadMachineLearnedBehavior),
		LoadRBFBehavior(LoadRBFBehavior),
		LoadTwistSwingBehavior(LoadTwistSwingBehavior),
		TranslationType(TranslationType),
		RotationType(RotationType),
		ScaleType(ScaleType),
		TranslationPruningThreshold(TranslationPruningThreshold),
		RotationPruningThreshold(RotationPruningThreshold),
		ScalePruningThreshold(ScalePruningThreshold)
	{
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRigLogicConfiguration(FRigLogicConfiguration&&) = default;
	FRigLogicConfiguration(const FRigLogicConfiguration&) = default;
	FRigLogicConfiguration& operator=(FRigLogicConfiguration&&) = default;
	FRigLogicConfiguration& operator=(const FRigLogicConfiguration&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void PostSerialize(const FArchive& Ar);

	// --- Deprecated: plain enum, migrated to per-platform ---
	UE_DEPRECATED(5.8, "Use CalculationTypePerPlatform instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use CalculationTypePerPlatform instead"))
	ERigLogicCalculationType CalculationType_DEPRECATED = ERigLogicCalculationType::AnyVector;

	// --- New per-platform properties ---
	UPROPERTY(EditAnywhere, Category = "RigLogic", meta = (DisplayName = "Calculation Type"))
	FPerPlatformERigLogicCalculationType CalculationTypePerPlatform;

	UPROPERTY(EditAnywhere, Category = "RigLogic", meta = (DisplayName = "Floating Point Type"))
	FPerPlatformERigLogicFloatingPointType FloatingPointTypePerPlatform;

	UPROPERTY(EditAnywhere, Category = "RigLogic", meta = (DisplayName = "Enable multi-thread ML Compute"))
	FPerPlatformBool EnableMultiThreadMLComputePerPlatform;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	bool LoadJoints;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	bool LoadBlendShapes;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	bool LoadAnimatedMaps;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	bool LoadMachineLearnedBehavior;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	bool LoadRBFBehavior;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	bool LoadTwistSwingBehavior;
	
	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	ERigLogicTranslationType TranslationType;
	
	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	ERigLogicRotationType RotationType;
	
	UE_DEPRECATED(5.8, "The value is read from the DNA instead.")
	UPROPERTY(meta = (IgnoreForMemberInitializationTest, DeprecatedProperty, DeprecationMessage = "The value is read from the DNA instead."))
	ERigLogicRotationOrder RotationOrder_DEPRECATED;

	UPROPERTY(BlueprintReadOnly, Category = "RigLogic")
	ERigLogicScaleType ScaleType;

	/** The joint translation pruning threshold is used to eliminate joint translation deltas below
	  * the specified threshold from the joint matrix when the RigLogic instance is initialized.
	  * Use it with caution, as while it may reduce the amount of compute to be done, it may also erase
	  * important deltas that could introduce artifacts into the rig.
	  * A reasonably safe starting value to try translation pruning would be 0.0001f
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	float TranslationPruningThreshold;

	/** The joint rotation pruning threshold is used to eliminate joint rotation deltas below
	  * the specified threshold from the joint matrix when the RigLogic instance is initialized.
	  * Use it with caution, as while it may reduce the amount of compute to be done, it may also erase
	  * important deltas that could introduce artifacts into the rig.
	  * A reasonably safe starting value to try rotation pruning would be 0.1f
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	float RotationPruningThreshold;

	/** The joint scale pruning threshold is used to eliminate joint scale deltas below
	  * the specified threshold from the joint matrix when the RigLogic instance is initialized.
	  * Use it with caution, as while it may reduce the amount of compute to be done, it may also erase
	  * important deltas that could introduce artifacts into the rig.
	  * A reasonably safe starting value to try scale pruning would be 0.001f
	 **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "RigLogic")
	float ScalePruningThreshold;
};

template<>
struct TStructOpsTypeTraits<FRigLogicConfiguration> : public TStructOpsTypeTraitsBase2<FRigLogicConfiguration>
{
	enum
	{
		WithPostSerialize = true
	};
};

class FRigLogic
{
public:
	UE_API explicit FRigLogic(const IDNAReader* Reader, const FRigLogicConfiguration& Config = FRigLogicConfiguration());
	UE_API explicit FRigLogic(FArchive* Archive, const FRigLogicConfiguration& Config);
	UE_API ~FRigLogic();

	FRigLogic(const FRigLogic&) = delete;
	FRigLogic& operator=(const FRigLogic&) = delete;

	FRigLogic(FRigLogic&&) = default;
	FRigLogic& operator=(FRigLogic&&) = default;

	UE_API void Dump(FArchive* Archive) const;
	UE_API const FRigLogicConfiguration& GetConfiguration() const;
	UE_API uint16 GetLODCount() const;
	UE_API TArrayView<const uint16> GetRBFSolverIndicesForLOD(uint16_t LOD) const;
	UE_API TArrayView<const uint16> GetMLOperationIndicesForLOD(uint16_t LOD, uint16_t MLTypeIndex, uint16 MLOperationSetIndex) const;
	UE_API TArrayView<const uint16> GetBlendShapeChannelIndicesForLOD(uint16_t LOD) const;
	UE_API TArrayView<const uint16> GetAnimatedMapIndicesForLOD(uint16_t LOD) const;
	UE_API TArrayView<const uint16> GetJointIndicesForLOD(uint16_t LOD) const;
	UE_API TArrayView<const float> GetNeutralJointValues() const;
	UE_API TArrayView<const uint16> GetJointVariableAttributeIndices(uint16 LOD) const;
	UE_API uint16 GetJointGroupCount() const;
	UE_API uint16 GetRBFSolverCount() const;
	UE_API uint16 GetTwistCount() const;
	UE_API uint16 GetSwingCount() const;
	UE_API uint16 GetMeshCount() const;
	UE_API uint16 GetMeshRegionCount(uint16 MeshIndex) const;
	UE_API uint16 GetMLTypeCount() const;
	UE_API uint16 GetMLOperationSetCount(uint16 MLTypeIndex) const;
	UE_API uint16 GetMLOperationCount(uint16 MLTypeIndex, uint16 MLOperationSetIndex) const;

	UE_API void MapGUIToRawControls(FRigInstance* Instance) const;
	UE_API void MapRawToGUIControls(FRigInstance* Instance) const;
	UE_API void CalculateMLControls(FRigInstance* Instance) const;
	UE_API void CalculateMLControls(FRigInstance* Instance, uint16 MLTypeIndex, uint16 MLOperationSetIndex, uint16 MLOperationIndex) const;
	UE_API void CalculateRBFControls(FRigInstance* Instance) const;
	UE_API void CalculateRBFControls(FRigInstance* Instance, uint16 SolverIndex) const;
	UE_API void CalculatePSDControls(FRigInstance* Instance) const;
	UE_API void CalculateJoints(FRigInstance* Instance) const;
	UE_API void CalculateJoints(FRigInstance* Instance, uint16 JointGroupIndex) const;
	UE_API void CalculateBlendShapes(FRigInstance* Instance) const;
	UE_API void CalculateAnimatedMaps(FRigInstance* Instance) const;
	UE_API void Calculate(FRigInstance* Instance) const;
	UE_API void CollectCalculationStats(FRigInstance* Instance) const;

private:
	friend FRigInstance;
	UE_API rl4::RigLogic* Unwrap() const;

private:
	TSharedPtr<rl4::MemoryResource> MemoryResource;

	struct FRigLogicDeleter
	{
		void operator()(rl4::RigLogic* Pointer);
	};
	TUniquePtr<rl4::RigLogic, FRigLogicDeleter> RigLogic;
	FRigLogicConfiguration Configuration;
	bool CachedEnableMultiThreadMLCompute;
};

#undef UE_API
