// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include <dna/Reader.h>

#define UE_API RIGLOGICMODULE_API

class UE_DEPRECATED(5.8, "RigLogicDNAReader is obsolete and will be removed. A DNA reader can be used directly instead, e.g. see the FDNAReader template class") RigLogicDNAReader : public dna::Reader {
public:
	UE_API explicit RigLogicDNAReader(const dna::Reader* DNAReader);

	// Header
	UE_API uint16 getFileFormatGeneration() const override;
	UE_API uint16 getFileFormatVersion() const override;

	// Descriptor
	UE_API dna::StringView getName() const override;
	UE_API dna::Archetype getArchetype() const override;
	UE_API dna::Gender getGender() const override;
	UE_API uint16 getAge() const override;
	UE_API uint32 getMetaDataCount() const override;
	UE_API dna::StringView getMetaDataKey(uint32 index) const override;
	UE_API dna::StringView getMetaDataValue(const char* key) const override;
	UE_API dna::TranslationUnit getTranslationUnit() const override;
	UE_API dna::RotationUnit getRotationUnit() const override;
	UE_API dna::CoordinateSystem getCoordinateSystem() const override;
	UE_API uint16 getLODCount() const override;
	UE_API uint16 getDBMaxLOD() const override;
	UE_API dna::StringView getDBComplexity() const override;
	UE_API dna::StringView getDBName() const override;

	// Definition
	UE_API uint16 getGUIControlCount() const override;
	UE_API dna::StringView getGUIControlName(uint16 index) const override;
	UE_API uint16 getRawControlCount() const override;
	UE_API dna::StringView getRawControlName(uint16 index) const override;
	UE_API uint16 getJointCount() const override;
	UE_API dna::StringView getJointName(uint16 index) const override;
	UE_API uint16 getJointIndexListCount() const override;
	UE_API dna::ConstArrayView<uint16> getJointIndicesForLOD(uint16 lod) const override;
	UE_API uint16 getJointParentIndex(uint16 index) const override;
	UE_API uint16 getBlendShapeChannelCount() const override;
	UE_API dna::StringView getBlendShapeChannelName(uint16 index) const override;
	UE_API uint16 getBlendShapeChannelIndexListCount() const override;
	UE_API dna::ConstArrayView<uint16> getBlendShapeChannelIndicesForLOD(uint16 lod) const override;
	UE_API uint16 getAnimatedMapCount() const override;
	UE_API dna::StringView getAnimatedMapName(uint16 index) const override;
	UE_API uint16 getAnimatedMapIndexListCount() const override;
	UE_API dna::ConstArrayView<uint16> getAnimatedMapIndicesForLOD(uint16 lod) const override;
	UE_API uint16 getMeshCount() const override;
	UE_API dna::StringView getMeshName(uint16 index) const override;
	UE_API uint16 getMeshIndexListCount() const override;
	UE_API dna::ConstArrayView<uint16> getMeshIndicesForLOD(uint16 lod) const override;
	UE_API uint16 getMeshBlendShapeChannelMappingCount() const override;
	UE_API dna::MeshBlendShapeChannelMapping getMeshBlendShapeChannelMapping(uint16 index) const override;
	UE_API dna::ConstArrayView<uint16> getMeshBlendShapeChannelMappingIndicesForLOD(uint16 lod) const override;
	UE_API dna::Vector3 getNeutralJointTranslation(uint16 index) const override;
	UE_API dna::ConstArrayView<float> getNeutralJointTranslationXs() const override;
	UE_API dna::ConstArrayView<float> getNeutralJointTranslationYs() const override;
	UE_API dna::ConstArrayView<float> getNeutralJointTranslationZs() const override;
	UE_API dna::Vector3 getNeutralJointRotation(uint16 index) const override;
	UE_API dna::ConstArrayView<float> getNeutralJointRotationXs() const override;
	UE_API dna::ConstArrayView<float> getNeutralJointRotationYs() const override;
	UE_API dna::ConstArrayView<float> getNeutralJointRotationZs() const override;

	// Behavior
	UE_API dna::ConstArrayView<uint16> getGUIToRawInputIndices() const override;
	UE_API dna::ConstArrayView<uint16> getGUIToRawOutputIndices() const override;
	UE_API dna::ConstArrayView<float> getGUIToRawFromValues() const override;
	UE_API dna::ConstArrayView<float> getGUIToRawToValues() const override;
	UE_API dna::ConstArrayView<float> getGUIToRawSlopeValues() const override;
	UE_API dna::ConstArrayView<float> getGUIToRawCutValues() const override;
	UE_API uint16 getPSDCount() const override;
	UE_API dna::ConstArrayView<uint16> getPSDRowIndices() const override;
	UE_API dna::ConstArrayView<uint16> getPSDColumnIndices() const override;
	UE_API dna::ConstArrayView<float> getPSDValues() const override;
	UE_API uint16 getJointRowCount() const override;
	UE_API uint16 getJointColumnCount() const override;
	UE_API dna::ConstArrayView<uint16> getJointVariableAttributeIndices(uint16 lod) const override;
	UE_API uint16 getJointGroupCount() const override;
	UE_API dna::ConstArrayView<uint16> getJointGroupLODs(uint16 jointGroupIndex) const override;
	UE_API dna::ConstArrayView<uint16> getJointGroupInputIndices(uint16 jointGroupIndex) const override;
	UE_API dna::ConstArrayView<uint16> getJointGroupOutputIndices(uint16 jointGroupIndex) const override;
	UE_API dna::ConstArrayView<float> getJointGroupValues(uint16 jointGroupIndex) const override;
	UE_API dna::ConstArrayView<uint16> getJointGroupJointIndices(uint16 jointGroupIndex) const override;
	UE_API dna::ConstArrayView<uint16> getBlendShapeChannelLODs() const override;
	UE_API dna::ConstArrayView<uint16> getBlendShapeChannelInputIndices() const override;
	UE_API dna::ConstArrayView<uint16> getBlendShapeChannelOutputIndices() const override;
	UE_API dna::ConstArrayView<uint16> getAnimatedMapLODs() const override;
	UE_API dna::ConstArrayView<uint16> getAnimatedMapInputIndices() const override;
	UE_API dna::ConstArrayView<uint16> getAnimatedMapOutputIndices() const override;
	UE_API dna::ConstArrayView<float> getAnimatedMapFromValues() const override;
	UE_API dna::ConstArrayView<float> getAnimatedMapToValues() const override;
	UE_API dna::ConstArrayView<float> getAnimatedMapSlopeValues() const override;
	UE_API dna::ConstArrayView<float> getAnimatedMapCutValues() const override;

	// Geometry
	UE_API uint32 getVertexPositionCount(uint16 meshIndex) const override;
	UE_API dna::Position getVertexPosition(uint16 meshIndex, uint32 vertexIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexPositionXs(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexPositionYs(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexPositionZs(uint16 meshIndex) const override;
	UE_API uint32 getVertexTextureCoordinateCount(uint16 meshIndex) const override;
	UE_API dna::TextureCoordinate getVertexTextureCoordinate(uint16 meshIndex, uint32 textureCoordinateIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexTextureCoordinateUs(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexTextureCoordinateVs(uint16 meshIndex) const override;
	UE_API uint32 getVertexNormalCount(uint16 meshIndex) const override;
	UE_API dna::Normal getVertexNormal(uint16 meshIndex, uint32 normalIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexNormalXs(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexNormalYs(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<float> getVertexNormalZs(uint16 meshIndex) const override;
	UE_API uint32 getVertexLayoutCount(uint16 meshIndex) const override;
	UE_API dna::VertexLayout getVertexLayout(uint16 meshIndex, uint32 layoutIndex) const override;
	UE_API dna::ConstArrayView<uint32> getVertexLayoutPositionIndices(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<uint32> getVertexLayoutTextureCoordinateIndices(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<uint32> getVertexLayoutNormalIndices(uint16 meshIndex) const override;
	UE_API uint32 getFaceCount(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<uint32> getFaceVertexLayoutIndices(uint16 meshIndex, uint32 faceIndex) const override;
	UE_API uint16 getMaximumInfluencePerVertex(uint16 meshIndex) const override;
	UE_API uint32 getSkinWeightsCount(uint16 meshIndex) const override;
	UE_API dna::ConstArrayView<float> getSkinWeightsValues(uint16 meshIndex, uint32 vertexIndex) const override;
	UE_API dna::ConstArrayView<uint16> getSkinWeightsJointIndices(uint16 meshIndex, uint32 vertexIndex) const override;
	UE_API uint16 getBlendShapeTargetCount(uint16 meshIndex) const override;
	UE_API uint16 getBlendShapeChannelIndex(uint16 meshIndex, uint16 blendShapeTargetIndex) const override;
	UE_API uint32 getBlendShapeTargetDeltaCount(uint16 meshIndex, uint16 blendShapeTargetIndex) const override;
	UE_API dna::Delta getBlendShapeTargetDelta(uint16 meshIndex, uint16 blendShapeTargetIndex, uint32 deltaIndex) const override;
	UE_API dna::ConstArrayView<float> getBlendShapeTargetDeltaXs(uint16 meshIndex, uint16 blendShapeTargetIndex) const override;
	UE_API dna::ConstArrayView<float> getBlendShapeTargetDeltaYs(uint16 meshIndex, uint16 blendShapeTargetIndex) const override;
	UE_API dna::ConstArrayView<float> getBlendShapeTargetDeltaZs(uint16 meshIndex, uint16 blendShapeTargetIndex) const override;
	UE_API dna::ConstArrayView<uint32> getBlendShapeTargetVertexIndices(uint16 meshIndex, uint16 blendShapeTargetIndex) const override;

	// Machine Learned Behavior
	UE_API uint16 getMLControlCount() const override;
	UE_API dna::StringView getMLControlName(uint16 index) const override;
	UE_API uint16 getNeuralNetworkCount() const override;
	UE_API uint16 getNeuralNetworkIndexListCount() const override;
	UE_API dna::ConstArrayView<uint16> getNeuralNetworkIndicesForLOD(uint16 lod) const override;
	UE_API uint16 getMeshRegionCount(uint16 meshIndex) const override;
	UE_API dna::StringView getMeshRegionName(uint16 meshIndex, uint16 regionIndex) const override;
	UE_API dna::ConstArrayView<uint16> getNeuralNetworkIndicesForMeshRegion(uint16 meshIndex, uint16 regionIndex) const override;
	UE_API dna::ConstArrayView<uint16> getNeuralNetworkInputIndices(uint16 netIndex) const override;
	UE_API dna::ConstArrayView<uint16> getNeuralNetworkOutputIndices(uint16 netIndex) const override;
	UE_API uint16 getNeuralNetworkLayerCount(uint16 netIndex) const override;
	UE_API dna::ActivationFunction getNeuralNetworkLayerActivationFunction(uint16 netIndex, uint16 layerIndex) const override;
	UE_API dna::ConstArrayView<float> getNeuralNetworkLayerActivationFunctionParameters(uint16 netIndex, uint16 layerIndex) const override;
	UE_API dna::ConstArrayView<float> getNeuralNetworkLayerBiases(uint16 netIndex, uint16 layerIndex) const override;
	UE_API dna::ConstArrayView<float> getNeuralNetworkLayerWeights(uint16 netIndex, uint16 layerIndex) const override;

	// RBFBehaviorReader methods
	UE_API uint16 getRBFPoseCount() const override;
	UE_API dna::StringView getRBFPoseName(uint16 poseIndex) const override;
	UE_API dna::ConstArrayView<uint16> getRBFPoseJointOutputIndices(uint16 poseIndex) const override;
	UE_API dna::ConstArrayView<uint16> getRBFPoseBlendShapeChannelOutputIndices(uint16 poseIndex) const override;
	UE_API dna::ConstArrayView<uint16> getRBFPoseAnimatedMapOutputIndices(uint16 poseIndex) const override;
	UE_API dna::ConstArrayView<float> getRBFPoseJointOutputValues(uint16 poseIndex) const override;
	UE_API float getRBFPoseScale(uint16 poseIndex) const override;
	UE_API uint16 getRBFPoseControlCount() const override;
	UE_API dna::StringView getRBFPoseControlName(uint16 poseControlIndex) const override;
	UE_API dna::ConstArrayView<uint16> getRBFPoseInputControlIndices(uint16 poseIndex) const override;
	UE_API dna::ConstArrayView<uint16> getRBFPoseOutputControlIndices(uint16 poseIndex) const override;
	UE_API dna::ConstArrayView<float> getRBFPoseOutputControlWeights(uint16 poseIndex) const override;
	UE_API uint16 getRBFSolverCount() const override;
	UE_API uint16 getRBFSolverIndexListCount() const override;
	UE_API dna::ConstArrayView<uint16> getRBFSolverIndicesForLOD(uint16 lod) const override;
	UE_API dna::StringView getRBFSolverName(uint16 solverIndex) const override;
	UE_API dna::ConstArrayView<uint16> getRBFSolverRawControlIndices(uint16 solverIndex) const override;
	UE_API dna::ConstArrayView<uint16> getRBFSolverPoseIndices(uint16 solverIndex) const override;
	UE_API dna::ConstArrayView<float> getRBFSolverRawControlValues(uint16 solverIndex) const override;
	UE_API dna::RBFSolverType getRBFSolverType(uint16 solverIndex) const override;
	UE_API float getRBFSolverRadius(uint16 solverIndex) const override;
	UE_API dna::AutomaticRadius getRBFSolverAutomaticRadius(uint16 solverIndex) const override;
	UE_API float getRBFSolverWeightThreshold(uint16 solverIndex) const override;
	UE_API dna::RBFDistanceMethod getRBFSolverDistanceMethod(uint16 solverIndex) const override;
	UE_API dna::RBFNormalizeMethod getRBFSolverNormalizeMethod(uint16 solverIndex) const override;
	UE_API dna::RBFFunctionType getRBFSolverFunctionType(uint16 solverIndex) const override;
	UE_API dna::TwistAxis getRBFSolverTwistAxis(uint16 solverIndex) const override;

	// JointBehaviorMetadataReader methods
	UE_API dna::TranslationRepresentation getJointTranslationRepresentation(uint16 jointIndex) const override;
	UE_API dna::RotationRepresentation getJointRotationRepresentation(uint16 jointIndex) const override;
	UE_API dna::ScaleRepresentation getJointScaleRepresentation(uint16 jointIndex) const override;

	// TwistSwingBehaviorReader methods
	UE_API uint16 getTwistCount() const override;
	UE_API dna::TwistAxis getTwistSetupTwistAxis(uint16 twistIndex) const override;
	UE_API dna::ConstArrayView<uint16> getTwistInputControlIndices(uint16 twistIndex) const override;
	UE_API dna::ConstArrayView<uint16> getTwistOutputJointIndices(uint16 twistIndex) const override;
	UE_API dna::ConstArrayView<float> getTwistBlendWeights(uint16 twistIndex) const override;
	UE_API uint16 getSwingCount() const override;
	UE_API dna::TwistAxis getSwingSetupTwistAxis(uint16 swingIndex) const override;
	UE_API dna::ConstArrayView<uint16> getSwingInputControlIndices(uint16 swingIndex) const override;
	UE_API dna::ConstArrayView<uint16> getSwingOutputJointIndices(uint16 swingIndex) const override;
	UE_API dna::ConstArrayView<float> getSwingBlendWeights(uint16 swingIndex) const override;

	// MachineLearnedBehaviorExtReader methods
	UE_API uint16 getMLTypeCount() const override;
	UE_API uint16 getMLOperationSetCount(uint16 mlTypeIndex) const override;
	UE_API uint16 getMLOperationCount(uint16 mlTypeIndex, uint16 mlOperationSetIndex) const override;
	UE_API dna::MachineLearnedBehaviorOperationType getMLOperationType(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const override;
	UE_API dna::ConstArrayView<uint32> getMLOperationParameters(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const override;
	UE_API dna::ConstArrayView<uint16> getMLOperationDependencyOperationSetIndices(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const override;
	UE_API dna::ConstArrayView<uint16> getMLOperationDependencyOperationIndices(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 mlOperationIndex) const override;
	UE_API uint16 getMLOperationIndexListCount(uint16 mlTypeIndex, uint16 mlOperationSetIndex) const override;
	UE_API dna::ConstArrayView<uint16> getMLOperationIndicesForLOD(uint16 mlTypeIndex, uint16 mlOperationSetIndex, uint16 lod) const override;
	UE_API dna::ConstArrayView<uint16> getMLJointsInputIndices() const override;
	UE_API dna::ConstArrayView<uint16> getMLJointsOutputIndices() const override;
	UE_API dna::ConstArrayView<uint16> getMLJointsParameterKeys() const override;
	UE_API dna::ConstArrayView<uint16> getMLJointsParameterValues() const override;

	// Reader
	UE_API void unload(dna::DataLayer Layer) override;
	static UE_API void destroy(dna::Reader* Pointer);

private:
	const dna::Reader* Reader;
};

#undef UE_API
