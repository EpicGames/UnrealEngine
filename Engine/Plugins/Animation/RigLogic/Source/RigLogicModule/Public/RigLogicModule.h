// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Stats/Stats.h"

#define UE_API RIGLOGICMODULE_API

DECLARE_LOG_CATEGORY_EXTERN(LogRigLogic, Log, All);

#if STATS
DECLARE_STATS_GROUP(TEXT("RigLogic"), STATGROUP_RigLogic, STATCAT_Advanced)

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Calculation Backend [ 0 = Scalar, 1 = SSE, 2 = AVX, 3 = NEON ]"), STAT_RigLogic_CalculationType, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Floating Point Type [ 0 = Float, 1 = Half-Float ]"), STAT_RigLogic_FloatingPointType, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Multi-thread ML Compute [ 0 = False, 1 = True ]"), STAT_RigLogic_MultiThreadMLCompute, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Current LOD"), STAT_RigLogic_LOD, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("RBF Solver Count"), STAT_RigLogic_RBFSolverCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("ML Operation Count"), STAT_RigLogic_MLOperationCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("PSD Expression Count"), STAT_RigLogic_PSDCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Joint Count"), STAT_RigLogic_JointCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Joint Delta Value Count"), STAT_RigLogic_JointDeltaValueCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Blend Shape Channel Count"), STAT_RigLogic_BlendShapeChannelCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Animated Map Count"), STAT_RigLogic_AnimatedMapCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Vertex Count"), STAT_RigLogic_VertexCount, STATGROUP_RigLogic, RIGLOGICMODULE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Map GUI To Raw Controls Time"), STAT_RigLogic_MapGUIToRawControlsTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Map Raw To GUI Controls Time"), STAT_RigLogic_MapRawToGUIControlsTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calculate RBF Controls Time"), STAT_RigLogic_CalculateRBFControlsTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calculate ML Controls Time"), STAT_RigLogic_CalculateMLControlsTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calculate PSD Controls Time"), STAT_RigLogic_CalculatePSDControlsTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calculate Joints Time"), STAT_RigLogic_CalculateJointsTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calculate Blend Shapes Time"), STAT_RigLogic_CalculateBlendShapesTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calculate Animated Maps Time"), STAT_RigLogic_CalculateAnimatedMapsTime, STATGROUP_RigLogic, RIGLOGICMODULE_API);
#endif  // STATS

class FRigLogicModule: public IModuleInterface
{
public:
	UE_API void StartupModule() override;
	UE_API void ShutdownModule() override;
};

#undef UE_API
