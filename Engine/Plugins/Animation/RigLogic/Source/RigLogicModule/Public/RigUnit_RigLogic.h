// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigInstance.h"
#include "RigLogic.h"

#include "DNAAsset.h"
#include "Materials/Material.h"
#include "Units/RigUnit.h"

#include "RigUnit_RigLogic.generated.h"

class IBehaviorReader;
struct FSharedRigRuntimeContext;

DECLARE_LOG_CATEGORY_EXTERN(LogRigLogicUnit, Log, All);

/* A helper struct used inside of the RigUnit_RigLogic to map bone indices to DNA driver joint indices. */
USTRUCT(meta = (DocumentationPolicy = "Strict"))
struct FBoneIndexControlAttributeMapping
{
	GENERATED_BODY()

	int32 BoneIndex;
	int32 DNAJointIndex;
	int32 RotationX;
	int32 RotationY;
	int32 RotationZ;
	int32 RotationW;
};

/* A helper struct used inside of the RigUnit_RigLogic to store arrays of arrays of integers. */
USTRUCT(meta = (DocumentationPolicy = "Strict"))
struct FRigUnit_RigLogic_IntArray
{
	GENERATED_BODY()

	// The values stored within this array.
	UPROPERTY(transient)
	TArray<int32> Values;
};

/* A helper struct used inside of the RigUnit_RigLogic to store arrays of arrays of integers. */
USTRUCT(meta = (DocumentationPolicy = "Strict"))
struct FRigUnit_RigLogic_IntMatrix
{
	GENERATED_BODY()

	// The values stored within this array.
	UPROPERTY(transient)
	TArray<FRigUnit_RigLogic_IntArray> Values;
};

/* The work data used by the FRigUnit_RigLogic */
USTRUCT(meta = (DocumentationPolicy = "Strict"))
struct FRigUnit_RigLogic_Data
{
	GENERATED_BODY()

	FRigUnit_RigLogic_Data();
 	~FRigUnit_RigLogic_Data();
 	FRigUnit_RigLogic_Data(const FRigUnit_RigLogic_Data& Other);
 	FRigUnit_RigLogic_Data& operator=(const FRigUnit_RigLogic_Data& Other);

	/** Cached Skeletal Mesh Component **/
	UPROPERTY(transient)
	TWeakObjectPtr<USkeletalMeshComponent> SkelMeshComponent;

	TSharedPtr<FSharedRigRuntimeContext> LocalRigRuntimeContext;

	/** RigInstance is a thin class, containing only character instance specific data
	  * i.e. the output buffers where RigLogic writes the results of computations, specific
	  * to each instance of a rig.
	**/
	TUniquePtr<FRigInstance> RigInstance;

	/** Mapping RL indices to UE indices
	  * Note: we use int32 instead of uint32 to allow storing INDEX_NONE for missing elements
	  * if value is valid, it is cast to appropriate uint type
	**/

	/** RL input index to ControlRig's input curve index **/
	UPROPERTY(transient)
	TArray<int32> InputCurveIndices;

	/** RL neural network mask index to ControlRig's input curve index **/
	UPROPERTY(transient)
	TArray<int32> MLMaskCurveIndices;

	/** RL joint index to ControlRig's hierarchy bone index **/
	UPROPERTY(transient)
	TArray<int32> HierarchyBoneIndices;

	/** RL driver joint index to ControlRig's hierarchy bone index and RigLogic control attribute mapping **/
	UPROPERTY(transient)
	TArray<FBoneIndexControlAttributeMapping> DriverJointsToControlAttributesMap;

	/** RL mesh blend shape index to ControlRig's output blendshape curve index for each LOD **/
	UPROPERTY(transient)
	TArray<FRigUnit_RigLogic_IntArray> MorphTargetCurveIndices;

	/** RL mesh+blend shape array index to RL blend shape index for each LOD **/
	UPROPERTY(transient)
	TArray<FRigUnit_RigLogic_IntArray> BlendShapeIndices;

	/** RL animated map index to ControlRig's output anim map curve index for each LOD **/
	UPROPERTY(transient)
	TArray<FRigUnit_RigLogic_IntArray> CurveElementIndicesForAnimMaps;

	/** RL animated map index to RL anim map curve index for each LOD **/
	UPROPERTY(transient)
	TArray<FRigUnit_RigLogic_IntArray> RigLogicIndicesForAnimMaps;

	/** LOD for which the model is rendered **/
	UPROPERTY(transient)
	uint32 CurrentLOD;

	static const uint8 MAX_ATTRS_PER_JOINT;

	bool IsRigLogicInitialized();
	void InitializeRigLogic(const URigHierarchy* InHierarchy, TSharedPtr<FSharedRigRuntimeContext> NewContext);

	/** Makes a map of input curve indices from DNA file to the control rig curves **/
	void MapInputCurveIndices(const URigHierarchy* InHierarchy);
	/** Makes a map of neural net mask indices from DNA file to the control rig curves **/
	void MapMLMaskCurveIndices(const URigHierarchy* InHierarchy);
	/** Uses names to map joint indices from DNA file to the indices of bones in control rig hierarchy **/
	void MapJoints(const URigHierarchy* Hierarchy);
	void MapDriverJoints(const URigHierarchy* Hierarchy);
	/** Uses names of blend shapes and meshes in DNA file, for all LODs, to map their indices to the indices of
	  * morph target curves in the curve container; curve name format is <mesh>__<blendshape> **/
	void MapMorphTargets(const URigHierarchy* InHierarchy);
	/** Uses names to map mask multiplier indices from DNA file, for all LODs, to the indices of curves in the
	  * control rig's curve container **/
	void MapMaskMultipliers(const URigHierarchy* InHierarchy);

	/** Calculates joint positions, orientation and scale based on inputs curves of the control rig **/
	void CalculateRigLogic(const URigHierarchy* InHierarchy, TArrayView<const float> NeutralJointValues);
	/** Updates joint positions in the hierarchy based on inputs curves of the control rig **/
	void UpdateJoints(URigHierarchy* Hierarchy, TArrayView<const float> NeutralJointValues, TArrayView<const float> DeltaJointValues);
	/** Updates morph target curve values based on values of input curves of the control rig **/
	void UpdateBlendShapeCurves(URigHierarchy* InHierarchy, TArrayView<const float> BlendShapeValues);
	/** Updates anim map curve values based on values of input curves of the control rig **/
	void UpdateAnimMapCurves(URigHierarchy* InHierarchy, TArrayView<const float> AnimMapOutputs);
};


/** RigLogic is used to translate control input curves into bone transforms and values for blend shape and
  *  animated map multiplier curves */

USTRUCT(meta = (DisplayName = "RigLogic", Category = "RigLogic", DocumentationPolicy = "Strict", Keywords = "Rig,RigLogic"))
struct FRigUnit_RigLogic: public FRigUnitMutable
{
	GENERATED_BODY()

	FRigUnit_RigLogic()
		: Data()
		, bIsInitialized(false)
	{}

public:
#if WITH_DEV_AUTOMATION_TESTS
	/** Allows accessing private Data property from unit tests **/
	struct TestAccessor;
	friend TestAccessor;
#endif

	RIGVM_METHOD()
	virtual void Execute() override;

private:
	static TSharedPtr<FSharedRigRuntimeContext> GetSharedRigRuntimeContext(USkeletalMesh* SkelMesh);

private:
	// internal work data for the unit
	UPROPERTY(transient)
	FRigUnit_RigLogic_Data Data;

	// flag to determine if the node has been initialized
	UPROPERTY(transient)
	bool bIsInitialized;
};
