// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DNAReader.h"
#include "RigLogic.h"

struct FSharedRigRuntimeContext
{
	template <typename T>
	struct TNestedArray
	{
		TArray<T> Values;
	};

	// Redundant but stored here as well so the runtime context can be updated / queried atomically
	TSharedPtr<IDNAReader> DNAReader;

	/** RigLogic itself is stateless, and is designed to be shared between
	  * multiple rig instances based on the same DNA.
	**/
	TSharedPtr<FRigLogic> RigLogic;

	/** Cached joint indices that need to be updated for each LOD **/
	TArray<TNestedArray<uint16>> VariableJointIndicesPerLOD;

	TArray<FQuat> InverseNeutralJointRotations;

	// Deprecated default constructor for ABI compatibility
	UE_DEPRECATED(5.8, "Default constructor is deprecated. Use the explicit constructor instead.")
	FSharedRigRuntimeContext();

	FSharedRigRuntimeContext(TSharedPtr<IDNAReader> InBehaviorReader, TSharedPtr<FRigLogic> InRigLogic);
	void CacheVariableJointIndices();
	void CacheInverseNeutralJointRotations();

};
