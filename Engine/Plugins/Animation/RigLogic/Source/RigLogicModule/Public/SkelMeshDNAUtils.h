// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DNAToSkelMeshMap.h"
#include "Engine/SkeletalMesh.h"
#include "DNAAsset.h"
#include "SkelMeshDNAUtils.generated.h"

#define UE_API RIGLOGICMODULE_API

DECLARE_LOG_CATEGORY_EXTERN(LogDNAUtils, Log, All);

class IDNAReader;
class FDNAToSkelMeshMap;
class UDNA;

UENUM()
enum class ELodUpdateOption : uint8
{
	LOD0Only, 		// LOD0 Only
	LOD0AndLOD1Only,// LOD0 and LOD1 Only
	LOD1AndHigher, 	// LOD1 and higher
	All				// All LODs
};

/** A utility class for updating SkeletalMesh joints, base mesh, morph targets and skin weights according to DNA data.
  * After the update, the render data is re-chunked.
 **/
UCLASS(MinimalAPI, transient)
class USkelMeshDNAUtils: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Prepare context object that will allow mapping of DNA structures to SkelMesh ones for updating **/
	static UE_API FDNAToSkelMeshMap* CreateMapForUpdatingNeutralMesh(IDNAReader* InDNAReader, USkeletalMesh* InSkelMesh);
	/** Prepare context object that will allow mapping of DNA structures extracted from DNAAsset to SkelMesh ones for updating **/
	static UE_API FDNAToSkelMeshMap* CreateMapForUpdatingNeutralMesh(USkeletalMesh* InSkelMesh);

#if WITH_EDITORONLY_DATA
	/** Updates the positions, orientation and scale in the joint hierarchy using the data from DNA file **/
	static UE_API void UpdateJoints(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap);
	/** Updates the base mesh vertex positions for all mesh sections of all LODs, using the data from DNA file
	  * NOTE: Not calling RebuildRenderData automatically, it needs to be called explicitly after the first update
	  *       As the topology doesn't change, for subsequent updates it can be ommited to gain performance **/
	static UE_API void UpdateBaseMesh(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap, ELodUpdateOption InUpdateOption = ELodUpdateOption::LOD0Only);
	/** Updates the morph targets for all mesh sections of LODs, using the data from DNA file **/
	static UE_API void UpdateMorphTargets(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap, ELodUpdateOption InUpdateOption = ELodUpdateOption::LOD0Only);
	/** Updates the skin weights for all LODs using the data from DNA file **/
	static UE_API void UpdateSkinWeights(USkeletalMesh* InSkelMesh, IDNAReader* InDNAReader, FDNAToSkelMeshMap* InDNAToSkelMeshMap, ELodUpdateOption InUpdateOption = ELodUpdateOption::LOD0Only);
	/** Rechunks the mesh after the update **/
	static UE_API void RebuildRenderData(USkeletalMesh* InSkelMesh);
	/** Re-initialize vertex positions for rendering after the update, and optionally tangents **/
	static UE_API void RebuildRenderData_VertexPosition(USkeletalMesh* InSkelMesh, bool InRebuildTangents = false);
	/** Update joint behavior **/
	/*  NOTE: DNAAsset->SetBehaviorReader needs to be called before this */
	static UE_API void UpdateJointBehavior(USkeletalMeshComponent* InSkelMeshComponent);

	/** Gets the DNA asset embedded in the mesh */
	//UE_DEPRECATED(5.8, "Use GetMeshDNAAsset instead which returns the new UDNA asset type")
	static UE_API UDNAAsset* GetMeshDNA(USkeletalMesh* InSkelMesh);

	/** Gets the DNA asset embedded in the mesh */
	static UE_API UDNA* GetMeshDNAAsset(USkeletalMesh* InSkelMesh);

	/** Gets DNA Reader either from old version of UDNAAsset or new UDNA, because the reader is widely used so having that in function can be useful*/
	static UE_API TSharedPtr<IDNAReader> GetDNAReader(USkeletalMesh* InSkelMesh);

	/** Sets DNA Reader either from old version of UDNAAsset or new UDNA, because the reader is widely used so having that in function can be useful.
	 *
	 *  Pass ERigLogicInitPolicy::Defer when this call is immediately followed by another
	 *  operation that rebuilds the runtime context (e.g. RestoreLegacyUEMHCCompatibility).
	 */
	static UE_API void SetDNAReader(USkeletalMesh* InSkelMesh, TSharedPtr<IDNAReader> InDNAReader,
									EDNACopyPolicy CopyPolicy = EDNACopyPolicy::Copy,
									ERigLogicInitPolicy InitPolicy = ERigLogicInitPolicy::Initialize);

#endif // WITH_EDITORONLY_DATA
	/** Converts DNA vertex coordinates to UE4 coordinate system **/
	inline static FVector ConvertDNAVertexToUE4CoordSystem(FVector InVertexPositionInDNA)
	{
		return FVector{ -InVertexPositionInDNA.X, InVertexPositionInDNA.Y, -InVertexPositionInDNA.Z };
	}

	/** Converts UE4 coordinate system to DNA vertex coordinates **/
	inline static FVector ConvertUE4CoordSystemToDNAVertex(FVector InVertexPositionInUE4)
	{
		return FVector{ -InVertexPositionInUE4.X, InVertexPositionInUE4.Y, -InVertexPositionInUE4.Z };
	}
	
private:

	inline static void GetLODRange(ELodUpdateOption InUpdateOption, const int32& InLODNum, int32& OutLODStart, int32& OutLODRangeSize)
	{
		OutLODStart = 0;
		OutLODRangeSize = InLODNum;
		if (InUpdateOption == ELodUpdateOption::LOD1AndHigher)
		{
			OutLODStart = 1;
		}
		else if (InUpdateOption == ELodUpdateOption::LOD0Only && OutLODRangeSize > 0)
		{
			OutLODRangeSize = 1;
		}
		else if (InUpdateOption == ELodUpdateOption::LOD0AndLOD1Only && OutLODRangeSize > 1)
		{
			OutLODRangeSize = 2;
		}
	}
};

#undef UE_API
