// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RigLogicModule : ModuleRules
	{
		public RigLogicModule(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"ControlRig",
					"RigLogicLib",
					"RigVM",
					"Projects"
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PublicDependencyModuleNames.Add("UnrealEd");
				PublicDependencyModuleNames.Add("EditorFramework");
				PublicDependencyModuleNames.Add("MessageLog");

				PrivateDependencyModuleNames.Add("SkeletalMeshUtilitiesCommon");
				PrivateDependencyModuleNames.Add("RHI");
				PrivateDependencyModuleNames.Add("RenderCore");
				PrivateDependencyModuleNames.Add("AssetRegistry");
			}

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AnimationCore",
					"AnimGraphRuntime"
				}
			);

			bool bDefaultHalfFloats = true;
			if (Target.Platform == UnrealTargetPlatform.Win64
				|| Target.Platform == UnrealTargetPlatform.Linux
				|| Target.Platform == UnrealTargetPlatform.Mac)
			{
				// Desktop x86/x64 - Float by default, everything else defaults to half-floats
				if (!Target.Architectures.Contains(UnrealArch.Arm64))
				{
					bDefaultHalfFloats = false;
				}
			}

			if (bDefaultHalfFloats)
			{
				PrivateDefinitions.Add("RIGLOGIC_DEFAULT_HALF_FLOATS");
			}
		}
	}
}
